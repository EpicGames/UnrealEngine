// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class CurveEditor : ModuleRules
{
	public CurveEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "ApplicationCore",
                "AppFramework",
                "Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
                "InputCore",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"TimeManagement",
				"UMG",
				"UnrealEd",
				"ToolMenus",
				"TraceLog"
			}
		);

        PublicDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "SequencerWidgets"
	        }
	    );
	}
}
