// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditor.h"

#include "Algo/Transform.h"
#include "Containers/SparseArray.h"
#include "CoreGlobals.h"
#include "CurveEditorCommands.h"
#include "CurveEditorCopyBuffer.h"
#include "CurveEditorSettings.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorAxis.h"
#include "CurveEditorZoomScaleConfig.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ICurveEditorExtension.h"
#include "ICurveEditorModule.h"
#include "ICurveEditorToolExtension.h"
#include "ITimeSlider.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Geometry.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/KeyPasteArgs.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "SCurveEditor.h" // for access to LogCurveEditor
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UnrealExporter.h"
#include "Misc/SmartSnap.h"
#include "Modification/Resolution/CurveModelIdCache.h"
#include "Modification/Resolution/CurveModelLookUpInfo.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Selection/SelectionCleanser.h"
#include "Selection/SelectionUtils.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "CurveEditor"

FCurveModelID FCurveModelID::Unique()
{
	static uint32 CurrentID = 1;

	FCurveModelID ID;
	ID.ID = CurrentID++;
	return ID;
}

FCurveEditor::FCurveEditor()
	: Bounds(new FStaticCurveEditorBounds)
	, bBoundTransformUpdatesSuppressed(false)
	, ActiveCurvesSerialNumber(0)
	, SuspendBroadcastCount(0)
{
	Settings = GetMutableDefault<UCurveEditorSettings>();
	CommandList = MakeShared<FUICommandList>();

	OutputSnapEnabledAttribute = true;
	InputSnapEnabledAttribute  = true;
	InputSnapRateAttribute = FFrameRate(10, 1);

	GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}s");
	GridLineLabelFormatYAttribute = LOCTEXT("GridYLabelFormat", "{0}");
	
	Settings->GetOnCustomColorsChanged().AddRaw(this, &FCurveEditor::OnCustomColorsChanged);
	Settings->GetOnAxisSnappingChanged().AddRaw(this, &FCurveEditor::OnAxisSnappingChanged);
}

FCurveEditor::~FCurveEditor()
{
	if (!IsEngineExitRequested() && Settings)
	{
		Settings->GetOnCustomColorsChanged().RemoveAll(this);
		Settings->GetOnAxisSnappingChanged().RemoveAll(this);
	}
}

void FCurveEditor::InitCurveEditor(const FCurveEditorInitParams& InInitParams)
{
	ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>("CurveEditor");

	Selection = FCurveEditorSelection(SharedThis(this));
	ZoomScalingAttr = InInitParams.ZoomScalingAttr;
	ResolveCurveModelDelegate = InInitParams.ResolveCurveModelDelegate;

	// Editor Extensions can be registered in the Curve Editor module. To allow users to derive from FCurveEditor
	// we have to manually reach out to the module and get a list of extensions to create an instance of them.
	// If none of your extensions are showing up, it's because you forgot to call this function after construction
	// We're not allowed to use SharedThis(...) in a Constructor so it must exist as a separate function call.
	EditorExtensions.Append(InInitParams.AdditionalEditorExtensions);
	TArrayView<const FOnCreateCurveEditorExtension> Extensions = CurveEditorModule.GetEditorExtensions();
	for (int32 DelegateIndex = 0; DelegateIndex < Extensions.Num(); ++DelegateIndex)
	{
		check(Extensions[DelegateIndex].IsBound());

		// We call a delegate and have the delegate create the instance to cover cross-module
		TSharedRef<ICurveEditorExtension> NewExtension = Extensions[DelegateIndex].Execute(SharedThis(this));
		EditorExtensions.Add(NewExtension);
	}


	TArrayView<const FOnCreateCurveEditorToolExtension> Tools = CurveEditorModule.GetToolExtensions();
	for (int32 DelegateIndex = 0; DelegateIndex < Tools.Num(); ++DelegateIndex)
	{
		check(Tools[DelegateIndex].IsBound());

		// We call a delegate and have the delegate create the instance to cover cross-module
		AddTool(Tools[DelegateIndex].Execute(SharedThis(this)));
	}
	
	SuspendBroadcastCount = 0;
	// Listen to global undo so we can fix up our selection state for keys that no longer exist.
	GEditor->RegisterForUndo(this);
	TransactionManager = MakeUnique<UE::CurveEditor::FTransactionManager>(SharedThis(this));

	SelectionCleanser = MakePimpl<UE::CurveEditor::FSelectionCleanser>(SharedThis(this));

	CurveModelIdCache = MakePimpl<UE::CurveEditor::FCurveModelIdCache>();
}

int32 FCurveEditor::GetSupportedTangentTypes()
{
	return ((int32)ECurveEditorTangentTypes::InterpolationConstant |
		(int32)ECurveEditorTangentTypes::InterpolationLinear |
		(int32)ECurveEditorTangentTypes::InterpolationCubicAuto |
		(int32)ECurveEditorTangentTypes::InterpolationCubicUser |
		(int32)ECurveEditorTangentTypes::InterpolationCubicBreak |
		(int32)ECurveEditorTangentTypes::InterpolationCubicWeighted);
		//nope we don't support smart auto by default, FRichCurve doesn't support i
}

void FCurveEditor::SetPanel(TSharedPtr<SCurveEditorPanel> InPanel)
{
	WeakPanel = InPanel;
}

TSharedPtr<SCurveEditorPanel> FCurveEditor::GetPanel() const
{
	return WeakPanel.Pin();
}

void FCurveEditor::SetView(TSharedPtr<SCurveEditorView> InView)
{
	WeakView = InView;
}

TSharedPtr<SCurveEditorView> FCurveEditor::GetView() const
{
	return WeakView.Pin();
}

FCurveModel* FCurveEditor::FindCurve(FCurveModelID CurveID) const
{
	const TUniquePtr<FCurveModel>* Ptr = CurveData.Find(CurveID);
	return Ptr ? Ptr->Get() : nullptr;
}

const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& FCurveEditor::GetCurves() const
{
	return CurveData;
}

FCurveEditorToolID FCurveEditor::AddTool(TUniquePtr<ICurveEditorToolExtension>&& InTool)
{
	FCurveEditorToolID NewID = FCurveEditorToolID::Unique();
	ToolExtensions.Add(NewID, MoveTemp(InTool));
	ToolExtensions[NewID]->SetToolID(NewID);
	return NewID;
}

void FCurveEditor::AddAxis(const FName& InIdentifier, TSharedPtr<FCurveEditorAxis> InAxis)
{
	// Allow overwrites
	CustomAxes.Add(InIdentifier, InAxis);
}

TSharedPtr<FCurveEditorAxis> FCurveEditor::FindAxis(const FName& InIdentifier) const
{
	return CustomAxes.FindRef(InIdentifier);
}

void FCurveEditor::RemoveAxis(const FName& InIdentifier)
{
	CustomAxes.Remove(InIdentifier);
}

void FCurveEditor::ClearAxes()
{
	CustomAxes.Empty();
}

FCurveModelID FCurveEditor::AddCurve(TUniquePtr<FCurveModel>&& InCurve)
{
	check(InCurve);
	
	// The curve ID is relevant e.g. for undo / redo.
	// If InCurve has no ID set, InitCurveModelIdWithReusedOrNewId will set it here.
	// If the caller actually specifies an ID and that ID is added already, they have a mistake in their business logic.
	const FCurveModelID& CurveId = UE::CurveEditor::InitCurveModelIdWithReusedOrNewId(*CurveModelIdCache, *InCurve, CurveData);
	if (!ensureMsgf(!CurveData.Contains(CurveId), TEXT("Investigate what caused the double-addition and fix it!")))
	{
		return FCurveModelID();
	}
	
	FCurveModel *Curve = InCurve.Get();

	CurveData.Add(CurveId, MoveTemp(InCurve));

	// Add child curves
	TArray<TUniquePtr<FCurveModel>> ChildCurvesArray;
	Curve->MakeChildCurves(ChildCurvesArray);
	for (TUniquePtr<FCurveModel>& Child : ChildCurvesArray)
	{
		ChildCurves.Add(CurveId, AddCurve(MoveTemp(Child)));
	}

	// Trigger OnCurveColorsChangedDelegate when any of the colors are changed...
	// Convenience so external systems do not need to listen to all curves individually.
	Curve->OnPostColorChanged().AddSP(this, &FCurveEditor::HandleCurveColorChanged, CurveId);

	++ActiveCurvesSerialNumber;
	if (IsBroadcasting())
	{
		OnCurveArrayChanged.Broadcast(Curve, true, this);
	}
	return CurveId;
}

void FCurveEditor::BroadcastCurveChanged(FCurveModel* InCurve)
{
	if (IsBroadcasting())
	{
		OnCurveArrayChanged.Broadcast(InCurve, true, this);
	}
}

FCurveModelID FCurveEditor::AddCurveForTreeItem(TUniquePtr<FCurveModel>&& InCurve, FCurveEditorTreeItemID TreeItemID)
{
	FCurveModelID NewID = AddCurve(MoveTemp(InCurve));
	TreeIDByCurveID.Add(NewID, TreeItemID);
	return NewID;
}

void FCurveEditor::ResetMinMaxes()
{
	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	if (Panel.IsValid())
	{
		Panel->ResetMinMaxes();
	}
}
void FCurveEditor::RemoveCurve(FCurveModelID InCurveID)
{
	for (auto ChildID = ChildCurves.CreateConstKeyIterator(InCurveID); ChildID; ++ChildID)
	{
		RemoveCurve(ChildID.Value());
	}
	ChildCurves.Remove(InCurveID);

	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	if (Panel.IsValid())
	{
		Panel->RemoveCurveFromViews(InCurveID);
	}

	if(IsBroadcasting())
	{
		OnCurveArrayChanged.Broadcast(FindCurve(InCurveID), false,this);
	}


	CurveData.Remove(InCurveID);
	Selection.Remove(InCurveID);
	PinnedCurves.Remove(InCurveID);


	++ActiveCurvesSerialNumber;
}

void FCurveEditor::RemoveAllCurves()
{
	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	if (Panel.IsValid())
	{
		for (TPair<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : CurveData)
		{
			Panel->RemoveCurveFromViews(CurvePair.Key);
		}
	}

	CurveData.Empty();
	Selection.Clear();
	PinnedCurves.Empty();
	ChildCurves.Empty();

	++ActiveCurvesSerialNumber;
}

bool FCurveEditor::IsCurvePinned(FCurveModelID InCurveID) const
{
	return PinnedCurves.Contains(InCurveID);
}

void FCurveEditor::PinCurve(FCurveModelID InCurveID)
{
	PinnedCurves.Add(InCurveID);
	++ActiveCurvesSerialNumber;
}

void FCurveEditor::UnpinCurve(FCurveModelID InCurveID)
{
	PinnedCurves.Remove(InCurveID);
	++ActiveCurvesSerialNumber;
}

const SCurveEditorView* FCurveEditor::FindFirstInteractiveView(FCurveModelID InCurveID) const
{
	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	if (Panel.IsValid())
	{
		for (auto ViewIt = Panel->FindViews(InCurveID); ViewIt; ++ViewIt)
		{
			if (ViewIt.Value()->IsInteractive())
			{
				return &ViewIt.Value().Get();
			}
		}
	}
	return nullptr;
}

FCurveEditorTreeItem& FCurveEditor::GetTreeItem(FCurveEditorTreeItemID ItemID)
{
	return Tree.GetItem(ItemID);
}

const FCurveEditorTreeItem& FCurveEditor::GetTreeItem(FCurveEditorTreeItemID ItemID) const
{
	return Tree.GetItem(ItemID);
}

FCurveEditorTreeItem* FCurveEditor::FindTreeItem(FCurveEditorTreeItemID ItemID)
{
	return Tree.FindItem(ItemID);
}

const FCurveEditorTreeItem* FCurveEditor::FindTreeItem(FCurveEditorTreeItemID ItemID) const
{
	return Tree.FindItem(ItemID);
}

const TArray<FCurveEditorTreeItemID>& FCurveEditor::GetRootTreeItems() const
{
	return Tree.GetRootItems();
}

FCurveEditorTreeItemID FCurveEditor::GetTreeIDFromCurveID(FCurveModelID CurveID) const
{
	if (TreeIDByCurveID.Contains(CurveID))
	{
		return TreeIDByCurveID[CurveID];
	}

	return FCurveEditorTreeItemID();
}

FCurveEditorTreeItem* FCurveEditor::AddTreeItem(FCurveEditorTreeItemID ParentID)
{
	return Tree.AddItem(ParentID);
}

void FCurveEditor::RemoveTreeItem(FCurveEditorTreeItemID ItemID)
{
	FCurveEditorTreeItem* Item = Tree.FindItem(ItemID);
	if (!Item)
	{
		return;
	}

	Tree.RemoveItem(ItemID, this);
	++ActiveCurvesSerialNumber;
}

void FCurveEditor::RemoveAllTreeItems()
{
	TArray<FCurveEditorTreeItemID> RootItems = Tree.GetRootItems();
	for(FCurveEditorTreeItemID ItemID : RootItems)
	{
		Tree.RemoveItem(ItemID, this);
	}
	++ActiveCurvesSerialNumber;
}

void FCurveEditor::SetTreeSelection(TArray<FCurveEditorTreeItemID>&& TreeItems)
{
	Tree.SetDirectSelection(MoveTemp(TreeItems), this);
}

void FCurveEditor::RemoveFromTreeSelection(TArrayView<const FCurveEditorTreeItemID> TreeItems)
{
	Tree.RemoveFromSelection(TreeItems, this);
}

ECurveEditorTreeSelectionState FCurveEditor::GetTreeSelectionState(FCurveEditorTreeItemID InTreeItemID) const
{
	return Tree.GetSelectionState(InTreeItemID);
}

const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& FCurveEditor::GetTreeSelection() const
{
	return Tree.GetSelection();
}

void FCurveEditor::SetBounds(TUniquePtr<ICurveEditorBounds>&& InBounds)
{
	check(InBounds.IsValid());
	Bounds = MoveTemp(InBounds);
}

bool FCurveEditor::ShouldAutoFrame() const
{
	return Settings->GetAutoFrameCurveEditor();
}


void FCurveEditor::BindCommands()
{
	using namespace UE::CurveEditor;
	UCurveEditorSettings* CurveSettings = Settings;
		
	CommandList->MapAction(FGenericCommands::Get().Delete, FExecuteAction::CreateSP(this, &FCurveEditor::DeleteSelection));

	CommandList->MapAction(FGenericCommands::Get().Cut, FExecuteAction::CreateSP(this, &FCurveEditor::CutSelection));
	CommandList->MapAction(FGenericCommands::Get().Copy, FExecuteAction::CreateSP(this, &FCurveEditor::CopySelection));
	CommandList->MapAction(FGenericCommands::Get().Paste, FExecuteAction::CreateSP(this, &FCurveEditor::PasteKeys,
		FKeyPasteArgs{ .Mode = ECurveEditorPasteMode::OverwriteRange,	.Flags = ECurveEditorPasteFlags::Default }
		));
	CommandList->MapAction(FCurveEditorCommands::Get().PasteAndMerge, FExecuteAction::CreateSP(this, &FCurveEditor::PasteKeys,
		FKeyPasteArgs{ .Mode = ECurveEditorPasteMode::Merge,			.Flags = ECurveEditorPasteFlags::Default }
		));
	CommandList->MapAction(FCurveEditorCommands::Get().PasteRelative, FExecuteAction::CreateSP(this, &FCurveEditor::PasteKeys,
		FKeyPasteArgs{ .Mode = ECurveEditorPasteMode::OverwriteRange,	.Flags = ECurveEditorPasteFlags::Default | ECurveEditorPasteFlags::Relative }
		));

	CommandList->MapAction(FCurveEditorCommands::Get().ZoomToFit, FExecuteAction::CreateSP(this, &FCurveEditor::ZoomToFit, EAxisList::All));
	CommandList->MapAction(FCurveEditorCommands::Get().ZoomToFitHorizontal, FExecuteAction::CreateSP(this, &FCurveEditor::ZoomToFit, EAxisList::X));
	CommandList->MapAction(FCurveEditorCommands::Get().ZoomToFitVertical, FExecuteAction::CreateSP(this, &FCurveEditor::ZoomToFit, EAxisList::Y));
	CommandList->MapAction(FCurveEditorCommands::Get().ZoomToFitAll, FExecuteAction::CreateSP(this, &FCurveEditor::ZoomToFitAll, EAxisList::All));

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleExpandCollapseNodes, FExecuteAction::CreateSP(this, &FCurveEditor::ToggleExpandCollapseNodes, false));
	CommandList->MapAction(FCurveEditorCommands::Get().ToggleExpandCollapseNodesAndDescendants, FExecuteAction::CreateSP(this, &FCurveEditor::ToggleExpandCollapseNodes, true));
	CommandList->MapAction(FCurveEditorCommands::Get().ExpandAllNodes, FExecuteAction::CreateSP(this, &FCurveEditor::ExpandAllNodes));
	CommandList->MapAction(FCurveEditorCommands::Get().CollapseAllNodes, FExecuteAction::CreateSP(this, &FCurveEditor::CollapseAllNodes));
	CommandList->MapAction(FCurveEditorCommands::Get().SortAllNodes, FExecuteAction::CreateSP(this, &FCurveEditor::SortAllNodes));

	CommandList->MapAction(FCurveEditorCommands::Get().TranslateSelectedKeysLeft, FExecuteAction::CreateSP(this, &FCurveEditor::TranslateSelectedKeysLeft));
	CommandList->MapAction(FCurveEditorCommands::Get().TranslateSelectedKeysRight, FExecuteAction::CreateSP(this, &FCurveEditor::TranslateSelectedKeysRight));

	CommandList->MapAction(FCurveEditorCommands::Get().SelectAllKeys, FExecuteAction::CreateSP(this, &FCurveEditor::SelectAllKeys));
	CommandList->MapAction(FCurveEditorCommands::Get().SelectForward, FExecuteAction::CreateSP(this, &FCurveEditor::SelectForward));
	CommandList->MapAction(FCurveEditorCommands::Get().SelectBackward, FExecuteAction::CreateSP(this, &FCurveEditor::SelectBackward));
	CommandList->MapAction(FCurveEditorCommands::Get().SelectNone, FExecuteAction::CreateSP(this, &FCurveEditor::SelectNone));
	CommandList->MapAction(FCurveEditorCommands::Get().InvertSelection, FExecuteAction::CreateSP(this, &FCurveEditor::InvertSelection));

	CommandList->MapAction(FCurveEditorCommands::Get().MatchLastTangentToFirst, FExecuteAction::CreateSP(this, &FCurveEditor::MatchLastTangentToFirst,true));
	CommandList->MapAction(FCurveEditorCommands::Get().MatchFirstTangentToLast, FExecuteAction::CreateSP(this, &FCurveEditor::MatchLastTangentToFirst,false));

	{
		FExecuteAction   ToggleInputSnapping     = FExecuteAction::CreateSP(this,   &FCurveEditor::ToggleInputSnapping);
		FIsActionChecked IsInputSnappingEnabled  = FIsActionChecked::CreateSP(this, &FCurveEditor::IsInputSnappingEnabled);
		FExecuteAction   ToggleOutputSnapping    = FExecuteAction::CreateSP(this,   &FCurveEditor::ToggleOutputSnapping);
		FIsActionChecked IsOutputSnappingEnabled = FIsActionChecked::CreateSP(this, &FCurveEditor::IsOutputSnappingEnabled);

		CommandList->MapAction(FCurveEditorCommands::Get().ToggleInputSnapping, ToggleInputSnapping, FCanExecuteAction(), IsInputSnappingEnabled);
		CommandList->MapAction(FCurveEditorCommands::Get().ToggleOutputSnapping, ToggleOutputSnapping, FCanExecuteAction(), IsOutputSnappingEnabled);
	}

	// Flip Curve
	CommandList->MapAction(FCurveEditorCommands::Get().FlipCurveHorizontal, FExecuteAction::CreateSP(this, &FCurveEditor::FlipCurve, ECurveFlipDirection::Horizontal));
	CommandList->MapAction(FCurveEditorCommands::Get().FlipCurveVertical, FExecuteAction::CreateSP(this, &FCurveEditor::FlipCurve, ECurveFlipDirection::Vertical));

	// Flatten and Straighten Tangents
	{
		CommandList->MapAction(FCurveEditorCommands::Get().FlattenTangents, FExecuteAction::CreateSP(this, &FCurveEditor::FlattenSelection), FCanExecuteAction::CreateSP(this, &FCurveEditor::CanFlattenOrStraightenSelection) );
		CommandList->MapAction(FCurveEditorCommands::Get().StraightenTangents, FExecuteAction::CreateSP(this, &FCurveEditor::StraightenSelection), FCanExecuteAction::CreateSP(this, &FCurveEditor::CanFlattenOrStraightenSelection) );
	}

	CommandList->MapAction(FCurveEditorCommands::Get().SmartSnapKeys, FExecuteAction::CreateSP(this, &FCurveEditor::SmartSnapSelection), FCanExecuteAction::CreateSP(this, &FCurveEditor::CanSmartSnapSelection));

	// Curve Colors
	{
		CommandList->MapAction(FCurveEditorCommands::Get().SetRandomCurveColorsForSelected, FExecuteAction::CreateSP(this, &FCurveEditor::SetRandomCurveColorsForSelected), FCanExecuteAction());
		CommandList->MapAction(FCurveEditorCommands::Get().SetCurveColorsForSelected, FExecuteAction::CreateSP(this, &FCurveEditor::SetCurveColorsForSelected), FCanExecuteAction());
	}

	// Tangent Visibility
	{
		FExecuteAction SetAllTangents          = FExecuteAction::CreateUObject(Settings, &UCurveEditorSettings::SetTangentVisibility, ECurveEditorTangentVisibility::AllTangents);
		FExecuteAction SetSelectedKeyTangents  = FExecuteAction::CreateUObject(Settings, &UCurveEditorSettings::SetTangentVisibility, ECurveEditorTangentVisibility::SelectedKeys);
		FExecuteAction SetUserTangents			= FExecuteAction::CreateUObject(Settings, &UCurveEditorSettings::SetTangentVisibility, ECurveEditorTangentVisibility::UserTangents);
		FExecuteAction SetNoTangents           = FExecuteAction::CreateUObject(Settings, &UCurveEditorSettings::SetTangentVisibility, ECurveEditorTangentVisibility::NoTangents);

		FIsActionChecked IsAllTangents         = FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetTangentVisibility() == ECurveEditorTangentVisibility::AllTangents; } );
		FIsActionChecked IsSelectedKeyTangents = FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetTangentVisibility() == ECurveEditorTangentVisibility::SelectedKeys; } );
		FIsActionChecked IsUserTangents			= FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetTangentVisibility() == ECurveEditorTangentVisibility::UserTangents; } );
		FIsActionChecked IsNoTangents          = FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetTangentVisibility() == ECurveEditorTangentVisibility::NoTangents; } );

		CommandList->MapAction(FCurveEditorCommands::Get().SetAllTangentsVisibility, SetAllTangents, FCanExecuteAction(), IsAllTangents);
		CommandList->MapAction(FCurveEditorCommands::Get().SetSelectedKeysTangentVisibility, SetSelectedKeyTangents, FCanExecuteAction(), IsSelectedKeyTangents);
		CommandList->MapAction(FCurveEditorCommands::Get().SetUserTangentsVisibility, SetUserTangents, FCanExecuteAction(), IsUserTangents);
		CommandList->MapAction(FCurveEditorCommands::Get().SetNoTangentsVisibility, SetNoTangents, FCanExecuteAction(), IsNoTangents);
	}

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleAutoFrameCurveEditor,
		FExecuteAction::CreateLambda( [CurveSettings]{ CurveSettings->SetAutoFrameCurveEditor( !CurveSettings->GetAutoFrameCurveEditor() ); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetAutoFrameCurveEditor(); } )
	);

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleShowTransportControls,
		FExecuteAction::CreateLambda([this, CurveSettings] { CurveSettings->SetShowTransportControls(!CurveSettings->GetShowTransportControls()); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([CurveSettings] { return CurveSettings->GetShowTransportControls(); })
	);

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleShowBars,
		FExecuteAction::CreateLambda([this, CurveSettings] { CurveSettings->SetShowBars(!CurveSettings->GetShowBars()); Tree.RecreateModelsFromExistingSelection(this); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([CurveSettings] { return CurveSettings->GetShowBars(); })
	);

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleSnapTimeToSelection,
		FExecuteAction::CreateLambda( [CurveSettings]{ CurveSettings->SetSnapTimeToSelection( !CurveSettings->GetSnapTimeToSelection() ); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetSnapTimeToSelection(); } )
	);

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleShowBufferedCurves,
		FExecuteAction::CreateLambda( [CurveSettings]{ CurveSettings->SetShowBufferedCurves( !CurveSettings->GetShowBufferedCurves() ); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetShowBufferedCurves(); } )
		);

	CommandList->MapAction(FCurveEditorCommands::Get().ToggleShowCurveEditorCurveToolTips,
		FExecuteAction::CreateLambda( [CurveSettings]{ CurveSettings->SetShowCurveEditorCurveToolTips( !CurveSettings->GetShowCurveEditorCurveToolTips() ); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetShowCurveEditorCurveToolTips(); } )
		);
	CommandList->MapAction(FCurveEditorCommands::Get().ToggleShowValueIndicatorLines,
		FExecuteAction::CreateLambda( [CurveSettings]{ CurveSettings->SetShowValueIndicators( !CurveSettings->GetShowValueIndicators() ); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [CurveSettings]{ return CurveSettings->GetShowValueIndicators(); } )
	);

	// Deactivate Current Tool
	CommandList->MapAction(FCurveEditorCommands::Get().DeactivateCurrentTool,
		FExecuteAction::CreateSP(this, &FCurveEditor::MakeToolActive, FCurveEditorToolID::Unset()),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [this]{ return ActiveTool.IsSet() == false; } ) );

	// Bind commands for Editor Extensions
	for (TSharedRef<ICurveEditorExtension> Extension : EditorExtensions)
	{
		Extension->BindCommands(CommandList.ToSharedRef());
	}

	// Bind Commands for Tool Extensions
	for (TPair<FCurveEditorToolID, TUniquePtr<ICurveEditorToolExtension>>& Pair : ToolExtensions)
	{
		Pair.Value->BindCommands(CommandList.ToSharedRef());
	}
}

TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> FCurveEditor::GetToolbarPromotedFilters() const
{
	TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> Result = ICurveEditorModule::Get().GetGlobalToolbarPromotedFilters();
	checkf(Result, TEXT("Should be valid for the lifetime of the module"));

	// In the future, we could extend FCurveEditor to have its own override for the globally promoted filters.
	return Result;
}

FCurveSnapMetrics FCurveEditor::GetCurveSnapMetrics(FCurveModelID CurveModel) const
{
	FCurveSnapMetrics CurveMetrics;

	const SCurveEditorView* View = FindFirstInteractiveView(CurveModel);
	if (!View)
	{
		return CurveMetrics;
	}

	// get the grid lines in view space
	TArray<float> ViewSpaceGridLines;
	View->GetGridLinesY(SharedThis(this), ViewSpaceGridLines, ViewSpaceGridLines);

	// convert the grid lines from view space
	TArray<double> CurveSpaceGridLines;
	ViewSpaceGridLines.Reserve(ViewSpaceGridLines.Num());
	FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveModel);
	Algo::Transform(ViewSpaceGridLines, CurveSpaceGridLines, [&CurveSpace](float VSVal) { return CurveSpace.ScreenToValue(VSVal); });
	
	// create metrics struct;
	CurveMetrics.bSnapOutputValues = OutputSnapEnabledAttribute.Get();
	CurveMetrics.bSnapInputValues = InputSnapEnabledAttribute.Get();
	CurveMetrics.AllGridLines = CurveSpaceGridLines;
	CurveMetrics.InputSnapRate = InputSnapRateAttribute.Get();

	return CurveMetrics;
}

void FCurveEditor::ZoomToFit(EAxisList::Type Axes)
{
	// If they have keys selected, we fit the specific keys.
	if (Selection.Count() > 0)
	{
		ZoomToFitSelection(Axes);
	}
	else
	{
		ZoomToFitAll(Axes);
	}
}

void FCurveEditor::ZoomToFitAll(EAxisList::Type Axes)
{
	TMap<FCurveModelID, FKeyHandleSet> AllCurves;
	for (FCurveModelID ID : GetEditedCurves())
	{
		AllCurves.Add(ID);
	}
	ZoomToFitInternal(Axes, AllCurves);
}

void FCurveEditor::ZoomToFitCurves(TArrayView<const FCurveModelID> CurveModelIDs, EAxisList::Type Axes)
{
	TMap<FCurveModelID, FKeyHandleSet> AllCurves;
	for (FCurveModelID ID : CurveModelIDs)
	{
		AllCurves.Add(ID);
	}
	ZoomToFitInternal(Axes, AllCurves);
}

void FCurveEditor::ZoomToFitSelection(EAxisList::Type Axes)
{
	ZoomToFitInternal(Axes, Selection.GetAll());
}

const FCurveEditorZoomScaleConfig& FCurveEditor::GetZoomScaleConfig() const
{
	const bool bCanCallGet = ZoomScalingAttr.IsBound() || ZoomScalingAttr.IsSet();
	const FCurveEditorZoomScaleConfig* OverrideConfig = bCanCallGet ? ZoomScalingAttr.Get() : nullptr;
	
	static FCurveEditorZoomScaleConfig Default;
	return OverrideConfig ? *OverrideConfig : Default;
}

void FCurveEditor::ZoomToFitInternal(EAxisList::Type Axes, const TMap<FCurveModelID, FKeyHandleSet>& CurveKeySet)
{
	TArray<FKeyPosition> KeyPositionsScratch;

	TMap<TTuple<TSharedRef<SCurveEditorView>, FCurveEditorViewAxisID>, TTuple<double, double>> ViewAndAxisToInputBounds;
	TMap<TTuple<TSharedRef<SCurveEditorView>, FCurveEditorViewAxisID>, TTuple<double, double>> ViewAndAxisToOutputBounds;

	auto TrackHorizontalBoundsForView = [&ViewAndAxisToInputBounds, Axes](const TSharedRef<SCurveEditorView>& View, FCurveModelID InCurveID, double InputMin, double InputMax)
	{
		if (Axes & EAxisList::X)
		{
			FCurveEditorViewAxisID HorizontalAxis = View->GetAxisForCurve(InCurveID, ECurveEditorAxisOrientation::Horizontal);
			if (HorizontalAxis)  // Only track horizontal axis zoom for custom axes since every view is implicitly linked to the global curve editor bounds
			{
				TTuple<double, double>* ViewBounds = ViewAndAxisToInputBounds.Find(MakeTuple(View, HorizontalAxis));
				if (ViewBounds)
				{
					ViewBounds->Get<0>() = FMath::Min(ViewBounds->Get<0>(), InputMin);
					ViewBounds->Get<1>() = FMath::Max(ViewBounds->Get<1>(), InputMax);
				}
				else
				{
					ViewAndAxisToInputBounds.Add(MakeTuple(View, HorizontalAxis), MakeTuple(InputMin, InputMax));
				}
			}
		}
	};

	auto TrackVerticalBoundsForView = [&ViewAndAxisToOutputBounds, Axes](const TSharedRef<SCurveEditorView>& View, FCurveModelID InCurveID, double OutputMin, double OutputMax)
	{
		if (Axes & EAxisList::Y)
		{
			FCurveEditorViewAxisID VerticalAxis = View->GetAxisForCurve(InCurveID, ECurveEditorAxisOrientation::Vertical);

			TTuple<double, double>* ViewBounds = ViewAndAxisToOutputBounds.Find(MakeTuple(View, VerticalAxis));
			if (ViewBounds)
			{
				ViewBounds->Get<0>() = FMath::Min(ViewBounds->Get<0>(), OutputMin);
				ViewBounds->Get<1>() = FMath::Max(ViewBounds->Get<1>(), OutputMax);
			}
			else
			{
				ViewAndAxisToOutputBounds.Add(MakeTuple(View, VerticalAxis), MakeTuple(OutputMin, OutputMax));
			}
		}
	};

	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();

	const TRange<FFrameNumber> TimeBounds       = TimeSliderController.IsValid() ? TimeSliderController->GetTimeBounds() : TRange<FFrameNumber>();
	const FFrameRate           TickResolution   = TimeSliderController.IsValid() ? TimeSliderController->GetTickResolution() : FFrameRate();
	const bool                 bHasTimeBounds   = TimeBounds.HasLowerBound() && TimeBounds.HasUpperBound();

	double AllInputMin = TNumericLimits<double>::Max(), AllInputMax = TNumericLimits<double>::Lowest();

	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	TSharedPtr<SCurveEditorView>  View  = WeakView.Pin();

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveKeySet)
	{
		FCurveModelID CurveID = Pair.Key;
		const FCurveModel* Curve = FindCurve(CurveID);
		if (!Curve)
		{
			continue;
		}

		double InputMin  = TNumericLimits<double>::Max(), InputMax  = TNumericLimits<double>::Lowest();
		double OutputMin = TNumericLimits<double>::Max(), OutputMax = TNumericLimits<double>::Lowest();

		int32 NumKeys = Pair.Value.AsArray().Num();
		if (NumKeys == 0)
		{
			double LocalMin = 0.0, LocalMax = 1.0;

			// If there are no keys selected, frame on the time slider's bounds (selection range when active,
			// otherwise playback range). Fall back to the curve's own time range only when no time slider is
			// wired up - keeps an undefined infinite-curve case from collapsing to (0, 0).
			if (bHasTimeBounds)
			{
				InputMin = TickResolution.AsSeconds(TimeBounds.GetLowerBoundValue());
				InputMax = TickResolution.AsSeconds(TimeBounds.GetUpperBoundValue());
			}
			else if (Curve->GetNumKeys())
			{
				Curve->GetTimeRange(LocalMin, LocalMax);
				InputMin = FMath::Min(InputMin, LocalMin);
				InputMax = FMath::Max(InputMax, LocalMax);
			}

			// Most curve types we know about support default values, so we can zoom to that even if there are no keys
			Curve->GetValueRange(LocalMin, LocalMax);
			OutputMin = FMath::Min(OutputMin, LocalMin);
			OutputMax = FMath::Max(OutputMax, LocalMax);
		}
		else
		{
			// Zoom to the min/max of the specified key set
			KeyPositionsScratch.SetNum(NumKeys, EAllowShrinking::No);
			Curve->GetKeyPositions(Pair.Value.AsArray(), KeyPositionsScratch);
			for (const FKeyPosition& Key : KeyPositionsScratch)
			{
				InputMin  = FMath::Min(InputMin, Key.InputValue);
				InputMax  = FMath::Max(InputMax, Key.InputValue);
				OutputMin = FMath::Min(OutputMin, Key.OutputValue);
				OutputMax = FMath::Max(OutputMax, Key.OutputValue);
			}
		}

		AllInputMin = FMath::Min(InputMin, AllInputMin);
		AllInputMax = FMath::Max(InputMax, AllInputMax);

		if (Panel)
		{
			// Store the min max for each view
			for (auto ViewIt = Panel->FindViews(CurveID); ViewIt; ++ViewIt)
			{
				TrackHorizontalBoundsForView(ViewIt.Value(), CurveID, InputMin, InputMax);
				TrackVerticalBoundsForView(ViewIt.Value(), CurveID, OutputMin, OutputMax);
			}
		}
		else if(View.IsValid())
		{
			TrackHorizontalBoundsForView(View.ToSharedRef(), CurveID, InputMin, InputMax);
			TrackVerticalBoundsForView(View.ToSharedRef(), CurveID, OutputMin, OutputMax);
		}
	}

	// CurveKeySet was empty (e.g. ZoomToFitSelection with nothing selected): fall back to the time slider's bounds.
	if (AllInputMin == TNumericLimits<double>::Max() && AllInputMax == TNumericLimits<double>::Lowest() && bHasTimeBounds)
	{
		AllInputMin = TickResolution.AsSeconds(TimeBounds.GetLowerBoundValue());
		AllInputMax = TickResolution.AsSeconds(TimeBounds.GetUpperBoundValue());
	}

	auto AdjustHorizontalBounds = [this, Panel, View](TSharedPtr<SCurveEditorView> InView, double CurrentInputMin, double CurrentInputMax, double& NewInputMin, double& NewInputMax)
	{
		// If zooming to the same (or invalid) min/max, keep the same zoom scale and center within the timeline
		if (NewInputMin >= NewInputMax)
		{
			const double HalfInputScale = (CurrentInputMax - CurrentInputMin) * 0.5;
			NewInputMin -= HalfInputScale;
			NewInputMax += HalfInputScale;
		}
		else
		{
			double PanelHeight = 0;
			if (Panel)
			{
				PanelHeight = Panel->GetViewContainerGeometry().GetLocalSize().Y;
			}
			else
			{
				PanelHeight = InView->GetViewSpace().GetPhysicalHeight();
			}

			double InputPercentage = PanelHeight != 0 ? FMath::Min(Settings->GetFrameInputPadding() / PanelHeight, 0.5) : 0.1; // Cannot pad more than half the height

			constexpr double MinInputZoom = 0.00001;
			const double InputPadding = FMath::Max((NewInputMax - NewInputMin) * InputPercentage, MinInputZoom);

			NewInputMin -= InputPadding;
			NewInputMax = FMath::Max(NewInputMin + MinInputZoom, NewInputMax) + InputPadding;
		}
	};

	// Perform per-view input zoom for custom axes
	for (const TPair<TTuple<TSharedRef<SCurveEditorView>, FCurveEditorViewAxisID>, TTuple<double, double>>& ViewAndAxisToBounds : ViewAndAxisToInputBounds)
	{
		FCurveEditorViewAxisID       AxisID   = ViewAndAxisToBounds.Key.Value;
		TSharedRef<SCurveEditorView> AxisView = ViewAndAxisToBounds.Key.Key;

		check(AxisID);

		FCurveEditorScreenSpaceH AxisSpace = AxisView->GetHorizontalAxisSpace(AxisID);

		double InputMin = ViewAndAxisToBounds.Value.Get<0>();
		double InputMax = ViewAndAxisToBounds.Value.Get<1>();

		AdjustHorizontalBounds(AxisView, AxisSpace.GetInputMin(), AxisSpace.GetInputMax(), InputMin, InputMax);

		AxisView->FrameHorizontal(InputMin, InputMax, AxisID);
	}

	if (Axes & EAxisList::X && AllInputMin != TNumericLimits<double>::Max() && AllInputMax != TNumericLimits<double>::Lowest())
	{
		double CurrentInputMin = 0.0, CurrentInputMax = 1.0;
		Bounds->GetInputBounds(CurrentInputMin, CurrentInputMax);

		AdjustHorizontalBounds(View, CurrentInputMin, CurrentInputMax, AllInputMin, AllInputMax);

		Bounds->SetInputBounds(AllInputMin, AllInputMax);
	}

	// Perform per-view output zoom for any computed ranges
	for (const TPair<TTuple<TSharedRef<SCurveEditorView>, FCurveEditorViewAxisID>, TTuple<double, double>>& ViewAndAxisToBounds : ViewAndAxisToOutputBounds)
	{
		FCurveEditorViewAxisID       AxisID   = ViewAndAxisToBounds.Key.Value;
		TSharedRef<SCurveEditorView> AxisView = ViewAndAxisToBounds.Key.Key;

		double OutputMin = ViewAndAxisToBounds.Value.Get<0>();
		double OutputMax = ViewAndAxisToBounds.Value.Get<1>();

		// If zooming to the same (or invalid) min/max, keep the same zoom scale and center within the timeline
		if (OutputMin >= OutputMax)
		{
			const double HalfOutputScale = (AxisView->GetOutputMax() - AxisView->GetOutputMin()) * 0.5;
			OutputMin -= HalfOutputScale;
			OutputMax += HalfOutputScale;
		}
		else
		{
			double PanelHeight = 0;
			if (Panel)
			{
				PanelHeight = Panel->GetViewContainerGeometry().GetLocalSize().Y;
			}
			else
			{
				PanelHeight = AxisView->GetViewSpace().GetPhysicalHeight();
			}

			double OutputPercentage = PanelHeight != 0 ? FMath::Min(Settings->GetFrameOutputPadding() / PanelHeight, 0.5) : 0.1; // Cannot pad more than half the height

			constexpr double MinOutputZoom = 0.00001;
			const double OutputPadding = FMath::Max((OutputMax - OutputMin) * OutputPercentage, MinOutputZoom);

			OutputMin -= OutputPadding;
			OutputMax = FMath::Max(OutputMin + MinOutputZoom, OutputMax) + OutputPadding;
		}

		AxisView->FrameVertical(OutputMin, OutputMax, AxisID);
	}
}

void FCurveEditor::TranslateSelectedKeys(double SecondsToAdd)
{
	using namespace UE::CurveEditor;

	const TMap<FCurveModelID, FKeyHandleSet> CurvesToKeys = ResolveSelectedOrRelativeKeys(*this);
	if (CurvesToKeys.IsEmpty())
	{
		return;
	}

	const FScopedCurveChange CurveChange(FCurvesSnapshotBuilder(SharedThis(this), CurvesToKeys, ECurveChangeFlags::MoveKeys));
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurvesToKeys)
	{
		if (FCurveModel* Curve = FindCurve(Pair.Key))
		{
			int32 NumKeys = Pair.Value.Num();

			if (NumKeys > 0)
			{
				TArrayView<const FKeyHandle> KeyHandles = Pair.Value.AsArray();
				TArray<FKeyPosition> KeyPositions;
				KeyPositions.SetNum(KeyHandles.Num());

				Curve->GetKeyPositions(KeyHandles, KeyPositions);

				for (int KeyIndex = 0; KeyIndex < KeyPositions.Num(); ++KeyIndex)
				{
					KeyPositions[KeyIndex].InputValue += SecondsToAdd;
				}
				Curve->SetKeyPositions(KeyHandles, KeyPositions);
			}
		}
	}
}

void FCurveEditor::TranslateSelectedKeysLeft()
{
	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("TranslateKeysLeft", "Translate Keys Left"));
	FFrameRate FrameRate = TimeSliderController->GetDisplayRate();
	double SecondsToAdd =  -FrameRate.AsInterval();
	TranslateSelectedKeys(SecondsToAdd);
}

void FCurveEditor::TranslateSelectedKeysRight()
{
	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("TranslateKeyRight", "Translate Keys Right"));
	FFrameRate FrameRate = TimeSliderController->GetDisplayRate();
	double SecondsToAdd = FrameRate.AsInterval();

	TranslateSelectedKeys(SecondsToAdd);
}

void FCurveEditor::SnapToSelectedKey()
{
	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}

	FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	TOptional<double> MinTime;

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		if (FCurveModel* Curve = FindCurve(Pair.Key))
		{
			int32 NumKeys = Pair.Value.Num();

			if (NumKeys > 0)
			{
				TArrayView<const FKeyHandle> KeyHandles = Pair.Value.AsArray();
				TArray<FKeyPosition> KeyPositions;
				KeyPositions.SetNum(KeyHandles.Num());

				Curve->GetKeyPositions(KeyHandles, KeyPositions);

				for (const FKeyPosition& KeyPosition : KeyPositions)
				{
					if (MinTime.IsSet())
					{
						MinTime = FMath::Min(KeyPosition.InputValue, MinTime.GetValue());
					}
					else
					{
						MinTime = KeyPosition.InputValue;
					}
				}
			}
		}
	}

	if (MinTime.IsSet())
	{
		TimeSliderController->SetScrubPosition(MinTime.GetValue() * TickResolution,/*bEvaluate*/ true);		
	}
}

void FCurveEditor::SelectAllKeys()
{
	const UE::CurveEditor::FScopedSelectionChange Transaction(SharedThis(this), LOCTEXT("SelectAllKeys", "Select all keys"));

	TMap<FCurveModelID, TArray<FKeyHandle>> SelectedKeysByCurve;

	for (FCurveModelID ID : GetEditedCurves())
	{
		if (FCurveModel* Curve = FindCurve(ID))
		{
			SelectedKeysByCurve.Add(ID, Curve->GetAllKeys());
		}
	}

	Selection.Replace(SelectedKeysByCurve, ECurvePointType::Key);
}

void FCurveEditor::SelectForward()
{
	const UE::CurveEditor::FScopedSelectionChange Transaction(SharedThis(this), LOCTEXT("SelectForward", "Select forward"));

	TMap<FCurveModelID, TArray<FKeyHandle>> SelectedKeysByCurve;

	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}

	FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	double CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());

	for (FCurveModelID ID : GetEditedCurves())
	{
		if (FCurveModel* Curve = FindCurve(ID))
		{
			TArray<FKeyHandle> KeyHandles;
			Curve->GetKeys(CurrentTime, TNumericLimits<double>::Max(), TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);
			SelectedKeysByCurve.Add(ID, MoveTemp(KeyHandles));
		}
	}

	Selection.Replace(SelectedKeysByCurve, ECurvePointType::Key);
}

void FCurveEditor::SelectBackward()
{
	const UE::CurveEditor::FScopedSelectionChange Transaction(SharedThis(this), LOCTEXT("SelectBackward", "Select backward"));

	TMap<FCurveModelID, TArray<FKeyHandle>> SelectedKeysByCurve;

	TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}

	FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	double CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());

	for (FCurveModelID ID : GetEditedCurves())
	{
		if (FCurveModel* Curve = FindCurve(ID))
		{
			TArray<FKeyHandle> KeyHandles;
			Curve->GetKeys(TNumericLimits<double>::Min(), CurrentTime, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);
			SelectedKeysByCurve.Add(ID, MoveTemp(KeyHandles));
		}
	}

	Selection.Replace(SelectedKeysByCurve, ECurvePointType::Key);
}

void FCurveEditor::SelectNone()
{
	const UE::CurveEditor::FScopedSelectionChange Transaction(SharedThis(this));
	Selection.Clear();
}

void FCurveEditor::InvertSelection()
{
	const UE::CurveEditor::FScopedSelectionChange Transaction(SharedThis(this), LOCTEXT("InvertSelection", "Invert selection"));
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		FCurveModelID CurveModelID = Pair.Key;
		if (FCurveModel* Curve = FindCurve(CurveModelID))
		{
			TArray<FKeyHandle> KeyHandles = Curve->GetAllKeys();
			
			TArrayView<const FKeyHandle> SelectedKeyHandles = Pair.Value.AsArray();
				
			if (SelectedKeyHandles.Num() > 0)
			{
				for (const FKeyHandle& SelectedKeyHandle : SelectedKeyHandles)
				{
					KeyHandles.Remove(SelectedKeyHandle);
				}

				Selection.Remove(CurveModelID);
				Selection.Add(CurveModelID, ECurvePointType::Key, KeyHandles);
			}
		}
	}	
}

bool FCurveEditor::IsInputSnappingEnabled() const
{
	return InputSnapEnabledAttribute.Get();
}

void FCurveEditor::ToggleInputSnapping()
{
	bool NewValue = !InputSnapEnabledAttribute.Get();

	if (!InputSnapEnabledAttribute.IsBound())
	{
		InputSnapEnabledAttribute = NewValue;
	}
	else
	{
		OnInputSnapEnabledChanged.ExecuteIfBound(NewValue);
	}
}

bool FCurveEditor::IsOutputSnappingEnabled() const
{
	return OutputSnapEnabledAttribute.Get();
}

void FCurveEditor::ToggleOutputSnapping()
{
	bool NewValue = !OutputSnapEnabledAttribute.Get();

	if (!OutputSnapEnabledAttribute.IsBound())
	{
		OutputSnapEnabledAttribute = NewValue;
	}
	else
	{
		OnOutputSnapEnabledChanged.ExecuteIfBound(NewValue);
	}
}
void FCurveEditor::FlipCurveHorizontal(TArray<FKeyPosition>& AllKeyPositions, TArray<FKeyAttributes>& AllKeyAttributes, ECurveFlipRangeType RangeType,
											  float InRangeMin, float InRangeMax, double CurveMinTime, double CurveMaxTime)
{
	float RangeMin = FLT_MAX;
	float RangeMax = -FLT_MAX;

	if (RangeType == ECurveFlipRangeType::CurveRange)
	{
		RangeMin = FMath::Min(CurveMinTime, RangeMin);
		RangeMax = FMath::Max(CurveMaxTime, RangeMax);
	}
	else
	{
		RangeMin = InRangeMin;
		RangeMax = InRangeMax;
	}

	// Loop through all keys to adjust positions and tangents
	for (int32 Index = AllKeyPositions.Num() - 1; Index >= 0; --Index)
	{
		FKeyPosition& Position = AllKeyPositions[Index];
		FKeyAttributes& Attributes = AllKeyAttributes[Index];

		// Mirror x value
		Position.InputValue = RangeMax - Position.InputValue + RangeMin;

		// Mirror tangent 
		if (Attributes.HasArriveTangent() && Attributes.HasLeaveTangent())
		{
			float ArriveTemp = Attributes.GetArriveTangent();
			float LeaveTemp = Attributes.GetLeaveTangent();
			Attributes.SetArriveTangent(-LeaveTemp);
			Attributes.SetLeaveTangent(-ArriveTemp);
		}

		if (Attributes.HasTangentMode())
		{
			// Mirror tangent weight
			if (Attributes.HasArriveTangentWeight() && Attributes.HasLeaveTangentWeight())
			{
				float ArriveWeightTemp = Attributes.GetArriveTangentWeight();
				float LeaveWeightTemp = Attributes.GetLeaveTangentWeight();
				Attributes.SetArriveTangentWeight(LeaveWeightTemp);
				Attributes.SetLeaveTangentWeight(ArriveWeightTemp);
			}
			ERichCurveTangentMode TangentMode = Attributes.GetTangentMode();
			Attributes.SetTangentMode(TangentMode);
		}
	}
}

void FCurveEditor::FlipCurveVertical(TArray<FKeyPosition>& AllKeyPositions, TArray<FKeyAttributes>& AllKeyAttributes, ECurveFlipRangeType RangeType,
											float InRangeMin, float InRangeMax, double CurveMinVal, double CurveMaxVal)
{
	float RangeMin = FLT_MAX;
	float RangeMax = -FLT_MAX;

	if (RangeType == ECurveFlipRangeType::CurveRange)
	{
		RangeMin = FMath::Min(CurveMinVal, RangeMin);
		RangeMax = FMath::Max(CurveMaxVal, RangeMax);
	}
	else if (RangeType == ECurveFlipRangeType::KeyRange)
	{
		for (int32 Index = AllKeyPositions.Num() - 1; Index >= 0; --Index)
		{
			FKeyPosition& Position = AllKeyPositions[Index];

			RangeMin = FMath::Min(Position.OutputValue, RangeMin);
			RangeMax = FMath::Max(Position.OutputValue, RangeMax);
		}
	}
	else
	{
		RangeMin = InRangeMin;
		RangeMax = InRangeMax;
	}

	// Loop through all keys to adjust positions and tangents
	for (int32 Index = AllKeyPositions.Num() - 1; Index >= 0; --Index)
	{
		FKeyPosition& Position = AllKeyPositions[Index];
		FKeyAttributes& Attributes = AllKeyAttributes[Index];

		// Mirror y value
		Position.OutputValue = RangeMax - Position.OutputValue + RangeMin;

		// Mirror tangent
		if (Attributes.HasArriveTangent())
		{
			float ArriveTemp = Attributes.GetArriveTangent();
			Attributes.SetArriveTangent(-ArriveTemp);
		}
		if (Attributes.HasLeaveTangent())
		{
			float LeaveTemp = Attributes.GetLeaveTangent();
			Attributes.SetLeaveTangent(-LeaveTemp);
		}
		if (Attributes.HasTangentMode())
		{
			ERichCurveTangentMode TangentMode = Attributes.GetTangentMode();
			Attributes.SetTangentMode(TangentMode);
		}
	}
}

void FCurveEditor::FlipCurve(ECurveFlipDirection Direction)
{
	using namespace UE::CurveEditor;
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), CurveData, ECurveChangeFlags::MoveKeys | ECurveChangeFlags::KeyAttributes),
		LOCTEXT("FlipCurve", "Flip Curve")
		);
	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : CurveData)
	{
		if (FCurveModel* Curve = Pair.Value.Get())
		{
			// Init key handles
			const TArray<FKeyHandle> KeyHandles = Curve->GetAllKeys();

			// Init key positions
			TArray<FKeyPosition> AllKeyPositions;
			AllKeyPositions.SetNum(KeyHandles.Num());
			Curve->GetKeyPositions(KeyHandles, AllKeyPositions);

			// Init key attributes
			TArray<FKeyAttributes> AllKeyAttributes;
			AllKeyAttributes.SetNum(KeyHandles.Num());
			Curve->GetKeyAttributes(KeyHandles, AllKeyAttributes);

			// If flipping horizontally
			if (Direction == ECurveFlipDirection::Horizontal)
			{
				double MinTime, MaxTime;
				Curve->GetTimeRange(MinTime, MaxTime);
				FlipCurveHorizontal(AllKeyPositions, AllKeyAttributes, HorizontalCurveFlipRangeSettings.RangeType,
										   HorizontalCurveFlipRangeSettings.MinRange, HorizontalCurveFlipRangeSettings.MaxRange, MinTime, MaxTime);
			}

			// If flipping vertically
			if (Direction == ECurveFlipDirection::Vertical)
			{
				double MinVal, MaxVal;
				Curve->GetValueRange(MinVal, MaxVal);
				FlipCurveVertical(AllKeyPositions, AllKeyAttributes, VerticalCurveFlipRangeSettings.RangeType,
										 VerticalCurveFlipRangeSettings.MinRange, VerticalCurveFlipRangeSettings.MaxRange, MinVal, MaxVal);
			}

			if (AllKeyPositions.Num() > 0)
			{
				Curve->SetKeyPositions(KeyHandles, AllKeyPositions);
				Curve->SetKeyAttributes(KeyHandles, AllKeyAttributes);
			}
		}
	}
}

void FCurveEditor::ToggleExpandCollapseNodes(bool bRecursive)
{
	Tree.ToggleExpansionState(bRecursive);
}

void FCurveEditor::ExpandAllNodes()
{
	Tree.ExpandAllNodes();
}

void FCurveEditor::CollapseAllNodes()
{
	Tree.CollapseAllNodes();
}

void FCurveEditor::SortAllNodes()
{
	Tree.SortTreeItems();
}

FCurveEditorScreenSpaceH FCurveEditor::GetPanelInputSpace() const
{
	const float PanelWidth = FMath::Max(1.f, WeakPanel.Pin()->GetViewContainerGeometry().GetLocalSize().X);

	double InputMin = 0.0, InputMax = 1.0;
	Bounds->GetInputBounds(InputMin, InputMax);

	InputMax = FMath::Max(InputMax, InputMin + 1e-10);
	return FCurveEditorScreenSpaceH(PanelWidth, InputMin, InputMax);
}

void FCurveEditor::ConstructXGridLines(TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{
	FCurveEditorScreenSpaceH InputSpace = GetPanelInputSpace();

	double MajorGridStep  = 0.0;
	int32  MinorDivisions = 0;
	if (InputSnapRateAttribute.Get().ComputeGridSpacing(InputSpace.PixelsPerInput(), MajorGridStep, MinorDivisions))
	{
		FText GridLineLabelFormatX = GridLineLabelFormatXAttribute.Get();
		const double FirstMajorLine = FMath::FloorToDouble(InputSpace.GetInputMin() / MajorGridStep) * MajorGridStep;
		const double LastMajorLine  = FMath::CeilToDouble(InputSpace.GetInputMax() / MajorGridStep) * MajorGridStep;

		for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
		{
			MajorGridLines.Add( (CurrentMajorLine - InputSpace.GetInputMin()) * InputSpace.PixelsPerInput() );
			if (MajorGridLabels)
			{
				MajorGridLabels->Add(FText::Format(GridLineLabelFormatX, FText::AsNumber(CurrentMajorLine)));
			}

			for (int32 Step = 1; Step < MinorDivisions; ++Step)
			{
				float MinorLine = CurrentMajorLine + Step*MajorGridStep/MinorDivisions;
				MinorGridLines.Add( (MinorLine - InputSpace.GetInputMin()) * InputSpace.PixelsPerInput() );
			}
		}
	}
}

void FCurveEditor::CutSelection()
{
	FScopedTransaction Transaction(LOCTEXT("CutKeys", "Cut Keys"));

	CopySelection();
	DeleteSelection();
}

void FCurveEditor::GetChildCurveModelIDs(const FCurveEditorTreeItemID TreeItemID, TSet<FCurveModelID>& OutCurveModelIDs) const
{
	const FCurveEditorTreeItem& TreeItem = GetTreeItem(TreeItemID);
	for (const FCurveModelID& CurveModelID : TreeItem.GetCurves())
	{
		OutCurveModelIDs.Add(CurveModelID);
	}

	for (const FCurveEditorTreeItemID& ChildTreeItem : TreeItem.GetChildren())
	{
		GetChildCurveModelIDs(ChildTreeItem, OutCurveModelIDs);
	}
}

void FCurveEditor::CopySelection() const
{
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	TOptional<double> KeyOffset;

	UCurveEditorCopyBuffer* CopyableBuffer = NewObject<UCurveEditorCopyBuffer>(GetTransientPackage(), UCurveEditorCopyBuffer::StaticClass(), NAME_None, RF_Transient);

	if (Selection.Count() > 0)
	{
		for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
		{
			if (FCurveModel* Curve = FindCurve(Pair.Key))
			{
				int32 NumKeys = Pair.Value.Num();

				if (NumKeys > 0)
				{
					UCurveEditorCopyableCurveKeys *CopyableCurveKeys = NewObject<UCurveEditorCopyableCurveKeys>(CopyableBuffer, UCurveEditorCopyableCurveKeys::StaticClass(), NAME_None, RF_Transient);

					CopyableCurveKeys->ShortDisplayName = Curve->GetShortDisplayName().ToString();
					CopyableCurveKeys->LongDisplayName = Curve->GetLongDisplayName().ToString();
					CopyableCurveKeys->LongIntentionName = Curve->GetLongIntentionName();
					CopyableCurveKeys->IntentionName = Curve->GetIntentionName();
					CopyableCurveKeys->KeyPositions.SetNum(NumKeys, EAllowShrinking::No);
					CopyableCurveKeys->KeyAttributes.SetNum(NumKeys, EAllowShrinking::No);

					TArrayView<const FKeyHandle> KeyHandles = Pair.Value.AsArray();

					Curve->GetKeyPositions(KeyHandles, CopyableCurveKeys->KeyPositions);
					// We need to get the attributes as they were specified by the user: so call the version that skips auto-computed values.
					Curve->GetKeyAttributesExcludingAutoComputed(KeyHandles, CopyableCurveKeys->KeyAttributes);

					for (int KeyIndex = 0; KeyIndex < CopyableCurveKeys->KeyPositions.Num(); ++KeyIndex)
					{
						if (!KeyOffset.IsSet() || CopyableCurveKeys->KeyPositions[KeyIndex].InputValue < KeyOffset.GetValue())
						{
							KeyOffset = CopyableCurveKeys->KeyPositions[KeyIndex].InputValue;
						}
					}

					CopyableBuffer->Curves.Add(CopyableCurveKeys);
				}
			}
		}
	}
	else
	{
		TSet<FCurveModelID> CurveModelIDs;

		for (const TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : GetTreeSelection())
		{
			if (Pair.Value == ECurveEditorTreeSelectionState::Explicit)
			{
				GetChildCurveModelIDs(Pair.Key, CurveModelIDs);
			}
		}

		for(const FCurveModelID& CurveModelID : CurveModelIDs)
		{
			if (FCurveModel* Curve = FindCurve(CurveModelID))
			{
				TUniquePtr<IBufferedCurveModel> CurveModelCopy = Curve->CreateBufferedCurveCopy();
				if (CurveModelCopy)
				{
					TArray<FKeyPosition> KeyPositions;
					CurveModelCopy->GetKeyPositions(KeyPositions);
					if (KeyPositions.Num() > 0)
					{
						UCurveEditorCopyableCurveKeys *CopyableCurveKeys = NewObject<UCurveEditorCopyableCurveKeys>(CopyableBuffer, UCurveEditorCopyableCurveKeys::StaticClass(), NAME_None, RF_Transient);

						CopyableCurveKeys->ShortDisplayName = Curve->GetShortDisplayName().ToString();
						CopyableCurveKeys->LongDisplayName = Curve->GetLongDisplayName().ToString();
						CopyableCurveKeys->IntentionName = Curve->GetIntentionName();

						CopyableCurveKeys->KeyPositions = KeyPositions;
						CurveModelCopy->GetKeyAttributes(CopyableCurveKeys->KeyAttributes);

						CopyableBuffer->Curves.Add(CopyableCurveKeys);
					}
				}
			}
		}

		// When copying entire curve objects we want absolute positions, so reset the detected offset
		KeyOffset.Reset();
	}

	if (KeyOffset.IsSet())
	{
		for (UCurveEditorCopyableCurveKeys* Curve : CopyableBuffer->Curves)
		{
			for (int Index = 0; Index < Curve->KeyPositions.Num(); ++Index)
			{
				Curve->KeyPositions[Index].InputValue -= KeyOffset.GetValue();
			}
		}

		CopyableBuffer->TimeOffset = KeyOffset.GetValue();
	}
	else
	{
		CopyableBuffer->bAbsolutePosition = true;
	}


	UExporter::ExportToOutputDevice(&Context, CopyableBuffer, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, CopyableBuffer);
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

class FCurveEditorCopyableCurveKeysObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FCurveEditorCopyableCurveKeysObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UCurveEditorCopyBuffer::StaticClass()))
		{
			return true;
		}
		return false;
	}


	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		NewCopyBuffers.Add(Cast<UCurveEditorCopyBuffer>(NewObject));
	}

public:
	TArray<UCurveEditorCopyBuffer*> NewCopyBuffers;
};

void FCurveEditor::MatchLastTangentToFirst(bool bMatchLastToFirst)
{
	using namespace UE::CurveEditor;
	// FScopedKeyChange won't generate any transaction if no tangents are changed
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), CurveData, ECurveChangeFlags::KeyAttributes),
		LOCTEXT("MatchTangents", "Match Tangents")
		);

	TArray<FKeyHandle, TInlineAllocator<2>> KeyHandles;
	KeyHandles.SetNum(2);
	TArray<FKeyAttributes, TInlineAllocator<2>> KeyAttributes;
	KeyAttributes.SetNum(2);
	
	TArray<FKeyHandle> AllKeyHandles;
	for (const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : CurveData)
	{
		FCurveModel* CurveModel = Pair.Value.Get();

		if (CurveModel)
		{
			// Get all of the key handles from this curve.
			CurveModel->GetAllKeys(AllKeyHandles);
			//need 2
			if (AllKeyHandles.Num() < 2)
			{
				continue;
			}
			KeyHandles[0] = AllKeyHandles[0];
			KeyHandles[1] = AllKeyHandles[AllKeyHandles.Num() - 1];
			
			CurveModel->GetKeyAttributes(KeyHandles, KeyAttributes);
			if (bMatchLastToFirst)
			{
				KeyAttributes[1] = KeyAttributes[0];
			}
			else
			{
				KeyAttributes[0] = KeyAttributes[1];
			}
			CurveModel->SetKeyAttributes(KeyHandles, KeyAttributes);
		}
	}
}

bool FCurveEditor::CanPaste(const FString& TextToImport) const
{
	FCurveEditorCopyableCurveKeysObjectTextFactory CopyableCurveKeysFactory;
	if (CopyableCurveKeysFactory.CanCreateObjectsFromText(TextToImport))
	{
		return true;
	}
	return false;
}

void FCurveEditor::ImportCopyBufferFromText(const FString& TextToImport, /*out*/ TArray<UCurveEditorCopyBuffer*>& ImportedCopyBuffers) const
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/CurveEditor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FCurveEditorCopyableCurveKeysObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

	ImportedCopyBuffers = Factory.NewCopyBuffers;

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();
}

TSet<FCurveModelID> FCurveEditor::GetTargetCurvesForPaste() const
{
	TSet<FCurveModelID> TargetCurves;

	TArray<FCurveEditorTreeItemID> NodesToSearch;

	// Try nodes with selected keys
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		TargetCurves.Add(Pair.Key);
	}

	// Try selected nodes
	if (TargetCurves.Num() == 0)
	{
		for (const TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : GetTreeSelection())
		{
			NodesToSearch.Add(Pair.Key);
		}
	}

	for (const FCurveEditorTreeItemID& TreeItemID : NodesToSearch)
	{
		const FCurveEditorTreeItem& TreeItem = GetTreeItem(TreeItemID);
		for (const FCurveModelID& CurveModelID : TreeItem.GetCurves())
		{
			TargetCurves.Add(CurveModelID);
		}
	}

	return TargetCurves;
}

bool FCurveEditor::CopyBufferCurveToCurveID(const UCurveEditorCopyableCurveKeys* InSourceCurve, const FCurveModelID InTargetCurve, TOptional<double> InTimeOffset, const bool bInAddToSelection, const bool bInOverwriteRange)
{
	using namespace UE::CurveEditor;
	return CopyBufferCurveToCurveID(InSourceCurve, InTargetCurve, InTimeOffset,
		bInOverwriteRange ? ECurveEditorPasteMode::OverwriteRange : ECurveEditorPasteMode::Merge,
		bInAddToSelection ? ECurveEditorPasteFlags::Default | ECurveEditorPasteFlags::SetSelection : ECurveEditorPasteFlags::Default
		);
}

namespace UE::CurveEditor::PasteDetail
{
static void RemovePastedKeysInRange(
	const UCurveEditorCopyableCurveKeys* InSourceCurve, FCurveModel* InTargetCurveModel, const TOptional<double>& InTimeOffset,
	double InCurrentTime
	)
{
	TArray<FKeyHandle> KeysToRemove;
	double MinKeyTime = TNumericLimits<double>::Max();
	double MaxKeyTime = TNumericLimits<double>::Lowest(); 
	for (int32 Index = 0; Index < InSourceCurve->KeyPositions.Num(); ++Index)
	{
		FKeyPosition KeyPosition = InSourceCurve->KeyPositions[Index];
		if (InTimeOffset.IsSet())
		{
			KeyPosition.InputValue += InTimeOffset.GetValue();
		}
		if (KeyPosition.InputValue < MinKeyTime)
		{
			MinKeyTime = KeyPosition.InputValue;
		}
		if (KeyPosition.InputValue > MaxKeyTime)
		{
			MaxKeyTime = KeyPosition.InputValue;
		}
	}

	// Just double checking we actually set a Min/Max time so we don't wipe out every key to infinity.
	if (InSourceCurve->KeyPositions.Num() > 0)
	{
		InTargetCurveModel->GetKeys(MinKeyTime, MaxKeyTime, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeysToRemove);
	}

	InTargetCurveModel->RemoveKeys(KeysToRemove, InCurrentTime);
}
/** @return Height to add to each pasted key to make the range relative to the closest key to the left of the pasted range. */
static double FindRelativeKeyPasteInset(const UCurveEditorCopyableCurveKeys* InSourceCurve, FCurveModel* InTargetCurveModel, double InCurrentTime)
{
	const FKeyPosition* MinElement = Algo::MinElementBy(InSourceCurve->KeyPositions, [](const FKeyPosition& Position)
	{
		return Position.InputValue;
	});
	if (!MinElement) 
	{
		// Nothing to paste
		return 0.0;
	}

	TOptional<FKeyHandle> ClosestPrevious, ClosestNext;
	InTargetCurveModel->GetClosestKeysTo(InCurrentTime, ClosestPrevious, ClosestNext);
	
	// If any key is at or in front of the scrubber, paste relative to that key.
	// If there is no previous key, then the user would expect the first key to be pasted where the scrubber intersects the curve.
	double KeyOutputValue;
	if (ClosestPrevious)
	{		
		FKeyPosition Position{};
		InTargetCurveModel->GetKeyPositions(
			TConstArrayView<FKeyHandle>(&ClosestPrevious.GetValue(), 1), TArrayView<FKeyPosition>(&Position, 1)
			);
		KeyOutputValue = Position.OutputValue;
	}
	else
	{
		InTargetCurveModel->Evaluate(InCurrentTime, KeyOutputValue);
	}
	
	// Bring all values down to the closest value to the left.
	return KeyOutputValue - MinElement->OutputValue; 
}
}

bool FCurveEditor::CopyBufferCurveToCurveID(
	const UCurveEditorCopyableCurveKeys* InSourceCurve, const FCurveModelID InTargetCurve,
	TOptional<double> InTimeOffset, UE::CurveEditor::ECurveEditorPasteMode InMode, UE::CurveEditor::ECurveEditorPasteFlags InFlags
	)
{
	using namespace UE::CurveEditor;
	
	FCurveModel* TargetCurveModel = FindCurve(InTargetCurve);
	if (!InSourceCurve || !TargetCurveModel)
	{
		return false;
	}
	
	double CurrentTime = 0.0;
	if (TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin())
	{
		FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());
	}

	// If pasting relative, bring all pasted values down to the first key to the left of the scrubber.
	const double ValueInset = EnumHasAnyFlags(InFlags, ECurveEditorPasteFlags::Relative)
		? PasteDetail::FindRelativeKeyPasteInset(InSourceCurve, TargetCurveModel, CurrentTime) : 0.0;
	
	// Sometimes when you paste you want to delete any keys that already exist in the timerange you'll be replacing
	// because mixing the pasted results with the original results wouldn't make any sense.
	if (InMode == ECurveEditorPasteMode::OverwriteRange)
	{
		PasteDetail::RemovePastedKeysInRange(InSourceCurve, TargetCurveModel, InTimeOffset, CurrentTime);
	}

	for (int32 Index = 0; Index < InSourceCurve->KeyPositions.Num(); ++Index)
	{
		FKeyPosition KeyPosition = InSourceCurve->KeyPositions[Index];
		if (InTimeOffset.IsSet())
		{
			KeyPosition.InputValue += InTimeOffset.GetValue();
		}

		KeyPosition.OutputValue += ValueInset;

		TOptional<FKeyHandle> KeyHandle = TargetCurveModel->AddKey(KeyPosition, InSourceCurve->KeyAttributes[Index]);
		if (KeyHandle.IsSet() && EnumHasAnyFlags(InFlags, ECurveEditorPasteFlags::SetSelection))
		{
			Selection.Add(FCurvePointHandle(InTargetCurve, ECurvePointType::Key, KeyHandle.GetValue()));
		}
	}

	return true;
}

void FCurveEditor::PasteKeys(TSet<FCurveModelID> CurveModelIDs, const bool bInOverwriteRange)
{
	using namespace UE::CurveEditor;

	FKeyPasteArgs Args;
	Args.CurveModelIds = CurveModelIDs;
	Args.Mode = bInOverwriteRange ? ECurveEditorPasteMode::OverwriteRange : ECurveEditorPasteMode::Merge;
	PasteKeys(Args);
}

void FCurveEditor::PasteKeys(UE::CurveEditor::FKeyPasteArgs InArgs)
{
	using namespace UE::CurveEditor;
	
	// Grab the text to paste from the clipboard
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	TArray<UCurveEditorCopyBuffer*> ImportedCopyBuffers;
	ImportCopyBufferFromText(TextToImport, ImportedCopyBuffers);

	if (ImportedCopyBuffers.Num() == 0)
	{
		return;
	}

	// There are numerous scenarios that Copy/Paste needs to handle.
	// 1:1				 - Copying a single curve to another single curve should always work.
	// 1:Multiple		 - Copying a single curve with multiple target curves should always work, the value will just be written into each one.
	// Multiple (Related): Multiple (Related) 
	//					 - Copying multiple curves between related controls, ie: fk_foot_l and fk_foot_r from one rig to another.
	//					 - If their long intent name matches, we consider them to be related controls. If their intent name doesn't match
	//					 - then we consider them unrelated controls.
	// Multiple (Unrelated):Multiple (Unrelated)
	//					 - If the long name doesn't match then we fall back to just the intent name. We want to handle copying both from one
	//					 - group of controls to multiple groups of controls, matching each by short intent name. This lets you copy fk_foot_l
	//					 - onto fk_foot_r and fk_spine_1 at the same time. We also handle trying to copy from multiple groups of controls
	//					 - onto multiple groups of controls - this falls back to a index-in-array order based copy and tries to ensure that
	//					 - the intent for each one (ie: transform.x) copies onto the first target transform.x, and then the next source that
	//					 - has a transform.x intent gets copied onto the *second* target transform.x.
	// Multiple (Unrelated):1
	//					 - This one is mostly an unhandled case and the last source intent will win on the target group, so fk_foot_l and fk_foot_r
	// 					 - pasted onto fk_spine_1, fk_spine_1 will just get the intents from fk_foot_r and fk_foot_l is ignored. This order isn't
	//					 - guranteed though because it's using the order the curves are in the internal arrays.
	
	// There should only be one copy buffer, but the way the import works returns an array.
	ensureMsgf(ImportedCopyBuffers.Num() == 1, TEXT("Multiple copy buffers pasted at one time, only the first one will be used!"));
	UCurveEditorCopyBuffer* SourceBuffer = ImportedCopyBuffers[0];

	// Figure out which CurveModelIDs we're trying to paste to. If they're not already specified, we try to find hovered curves,
	// and failing that we try to find all curves.
	TSet<FCurveModelID> TargetCurves = InArgs.CurveModelIds.Num() > 0 ? InArgs.CurveModelIds : GetTargetCurvesForPaste();
	
	if (TargetCurves.Num() == 0)
	{
		return;
	}

	// When we're pasting keys, we want the first key to paste where the timeslider is
	TOptional<double> TimeOffset;
	bool bApplyOffset = !SourceBuffer->bAbsolutePosition;

	if (bApplyOffset)
	{
		TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if (TimeSliderController.IsValid())
		{
			FFrameRate TickResolution = TimeSliderController->GetTickResolution();

			TimeOffset = TimeSliderController->GetScrubPosition() / TickResolution;
		}
		else
		{
			TimeOffset = SourceBuffer->TimeOffset;
		}
	}
	
	const FScopedTransaction Transaction(InArgs.OverrideTransactionName); // Selection and key change should be part of same undo / redo op.
	const FScopedSelectionChange SelectionChange(SharedThis(this));
	const FScopedCurveChange KeyChange(FCurvesSnapshotBuilder(SharedThis(this), TargetCurves, ECurveChangeFlags::KeyData));
	Selection.Clear();

	// Two simple cases, 1 to 1 and 1 to many.
	TArray<TPair<UCurveEditorCopyableCurveKeys*, FCurveModelID>> CopyPairs;

	if (SourceBuffer->Curves.Num() == 1)
	{
		for (FCurveModelID TargetCurveID : TargetCurves)
		{
			CopyPairs.Add(TPair<UCurveEditorCopyableCurveKeys*, FCurveModelID>(SourceBuffer->Curves[0], TargetCurveID));
		}
	}
	else
	{
		// The more complicated is the Multiple:Multiple / Multiple:1 (which is really just the same). We want to
		// prioritize matching up longer names if possible - this allows us to copy multiple controls to multiple
		// controls, such as starting with fk_foot_l and fk_foot_r and pasting to fk_foot_l, fk_foot_r, fk_neck_01.
		// We will match up the transform/scale/rotation for the fk_foot_l/fk_foot_r and don't touch fk_neck_01 in this
		// example. If no matches are made, then we fall back to the shorter intent string - where we just copy
		// transform.xyz to transform.xyz even though the source may be fk_foot_l and the target is fk_foot_r.

		// If any of the long names match (ie: fk_foot_l.transform.x) then we'll use long name matching for all.
		bool bUseLongNameForMatches = false;
		for (const UCurveEditorCopyableCurveKeys* SourceCurveKeys : SourceBuffer->Curves)
		{
			for (const FCurveModelID& TargetCurveID : TargetCurves)
			{
				FCurveModel* TargetCurve = FindCurve(TargetCurveID);
				if (TargetCurve)
				{
					if (SourceCurveKeys->LongIntentionName == TargetCurve->GetLongIntentionName())
					{
						bUseLongNameForMatches = true;
						break;
					}
				}
			}

			// Exit out of the outer loop too if we've got a match.
			if (bUseLongNameForMatches)
			{
				break;
			}
		}

		// Multiple to Multiple curve copying can get complicated when we only have the short intent name to deal with it, so
		// this creates an edge case where you're copying one set of intents (ie: transform.x, transform.y, transform.z) onto
		// multiple objects with those intents... we want to support this, but we don't support copying from multiple objects
		// onto multiple objects unless their LongIntentionName matches as it gets too confusing to match up.
		bool bOnlyOneSetOfSourceIntentions = true;
		{
			TMap<FString, int32> IntentionUseCounts;
			for (UCurveEditorCopyableCurveKeys* SourceCurveKeys : SourceBuffer->Curves)
			{
				IntentionUseCounts.FindOrAdd(SourceCurveKeys->IntentionName)++;
			}

			for (TPair<FString, int32>& Pair : IntentionUseCounts)
			{
				if (Pair.Value > 1)
				{
					bOnlyOneSetOfSourceIntentions = false;
					break;
				}
			}
		}
		
		TSet<FCurveModelID> CurvesToMatchTo = TargetCurves;
		for (UCurveEditorCopyableCurveKeys* SourceCurveKeys : SourceBuffer->Curves)
		{
			TArray<FCurveModelID> CurvesToRemove;
			for (const FCurveModelID& TargetCurveID : CurvesToMatchTo)
			{
				FCurveModel* TargetCurve = FindCurve(TargetCurveID);
				if (TargetCurve)
				{
					const bool bNameMatches = bUseLongNameForMatches ?
						SourceCurveKeys->LongIntentionName == TargetCurve->GetLongIntentionName() :
						SourceCurveKeys->IntentionName == TargetCurve->GetIntentionName();

					if (bNameMatches)
					{
						CopyPairs.Add(TPair<UCurveEditorCopyableCurveKeys*, FCurveModelID>(SourceCurveKeys, TargetCurveID));

						// Don't try to match to this curve again. This lets us try to handle the case where we have
						// multiple source objects (fk_foot_l, fk_foot_r) trying to copy to unrelated objects (cube1, cube2).
						// They will fail the LongDisplayName check but get the IntentionName check, but we need to remove
						// cube1 after the first time we match it so that fk_foot_r has a chance to paste into cube2 instead of cube1.
						CurvesToRemove.Add(TargetCurveID);

						// If we're copying from one object with multiple curves (ie: fk_foot_l) but we have multiple destination
						// objects, we loop through all of the target curves and apply them using the IntentionName matches check.
						// This only happens when using short intention names (as it's the more vague logic case), and we only
						// do this when you have multiple source curves, but only one of each kind. If you have multiple source
						// curves with multiple copies of the same intention, then we only apply it once to the first curve
						// who's intention matches and then remove it from the pool so that the next source with the same
						// intention (such as the second foot in the above example) gets a chance to write to the second
						// target curve with the same destination.
						bool bCopyToMultipleDestCurves = bOnlyOneSetOfSourceIntentions && !bUseLongNameForMatches;
						if (!bCopyToMultipleDestCurves)
						{
							break;
						}
					}
				}
			}

			for (FCurveModelID Curve : CurvesToRemove)
			{
				CurvesToMatchTo.Remove(Curve);
			}
		}

		// If the primary pass used long-name matching and some target curves were never
		// matched (e.g. an unrelated control selected alongside the exact-match one),
		// do a second pass using IntentionName only so those curves still receive the paste. 
		// (e.g foot_R can be pasted onto both foot_R and foot_L of another control)
		if (bUseLongNameForMatches && CurvesToMatchTo.Num() > 0)
		{
			for (UCurveEditorCopyableCurveKeys* SourceCurveKeys : SourceBuffer->Curves)
			{
				TArray<FCurveModelID> CurvesToRemove;
				for (const FCurveModelID& TargetCurveID : CurvesToMatchTo)
				{
					FCurveModel* TargetCurve = FindCurve(TargetCurveID);
					if (TargetCurve && SourceCurveKeys->IntentionName == TargetCurve->GetIntentionName())
					{
						CopyPairs.Add(TPair<UCurveEditorCopyableCurveKeys*, FCurveModelID>(SourceCurveKeys, TargetCurveID));
						CurvesToRemove.Add(TargetCurveID);

						if (!bOnlyOneSetOfSourceIntentions)
						{
							break;
						}
					}
				}

				for (const FCurveModelID& Curve : CurvesToRemove)
				{
					CurvesToMatchTo.Remove(Curve);
				}
			}
		}
	}

	// Now that we've calculated the source curve for each destination curve, copy them over.
	for (const TPair<UCurveEditorCopyableCurveKeys*, FCurveModelID>& Pair : CopyPairs)
	{
		CopyBufferCurveToCurveID(Pair.Key, Pair.Value, TimeOffset, InArgs.Mode, InArgs.Flags);
	}

	if (ShouldAutoFrame())
	{
		ZoomToFitSelection();
	}
}

void FCurveEditor::DeleteSelection()
{

	using namespace UE::CurveEditor;
	
	const FScopedTransaction Transaction(LOCTEXT("DeleteKeys", "Delete Keys"));
	const FScopedSelectionChange SelectionChange(SharedThis(this));
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), ECurveChangeFlags::RemoveKeys).TrackSelectedCurves()
		);
	
	double CurrentTime = 0.0;
	if (TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin())
	{
		FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());
	}

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		if (FCurveModel* Curve = FindCurve(Pair.Key))
		{
			Curve->RemoveKeys(Pair.Value.AsArray(), CurrentTime);
		}
	}

	Selection.Clear();
}

void FCurveEditor::FlattenSelection()
{
	using namespace UE::CurveEditor;
	// FScopedKeyChange won't generate any transaction if no tangents are changed
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), CurveData, ECurveChangeFlags::KeyAttributes),
		LOCTEXT("FlattenTangents", "Flatten Tangents")
		);

	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyAttributes> AllKeyPositions;
	//Since we don't have access here to the Section to get Tick Resolution if we flatten a weighted tangent we
	//do so by converting it to non-weighted and then back again.
	TArray<FKeyHandle>  KeyHandlesWeighted;
	TArray<FKeyAttributes> KeyAttributesWeighted;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		if (FCurveModel* Curve = FindCurve(Pair.Key))
		{
			KeyHandles.Reset(Pair.Value.Num());
			KeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

			AllKeyPositions.SetNum(KeyHandles.Num());
			Curve->GetKeyAttributes(KeyHandles, AllKeyPositions);

			KeyHandlesWeighted.Reset(Pair.Value.Num());
			KeyHandlesWeighted.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

			KeyAttributesWeighted.SetNum(KeyHandlesWeighted.Num());
			Curve->GetKeyAttributes(KeyHandlesWeighted, KeyAttributesWeighted);


			// Straighten tangents, ignoring any keys that we can't set tangents on
			for (int32 Index = AllKeyPositions.Num()-1 ; Index >= 0; --Index)
			{
				FKeyAttributes& Attributes = AllKeyPositions[Index];
				if (Attributes.HasTangentMode() && (Attributes.HasArriveTangent() || Attributes.HasLeaveTangent()))
				{
					Attributes.SetArriveTangent(0.f).SetLeaveTangent(0.f);
					if (Attributes.GetTangentMode() == RCTM_Auto || Attributes.GetTangentMode() == RCTM_SmartAuto)
					{
						Attributes.SetTangentMode(RCTM_User);
					}
					//if any weighted convert and convert back to both (which is what only support other modes are not really used).,
					if (Attributes.GetTangentWeightMode() == RCTWM_WeightedBoth || Attributes.GetTangentWeightMode() == RCTWM_WeightedArrive
						|| Attributes.GetTangentWeightMode() == RCTWM_WeightedLeave)
					{
						Attributes.SetTangentWeightMode(RCTWM_WeightedNone);
						FKeyAttributes& WeightedAttributes = KeyAttributesWeighted[Index];
						WeightedAttributes.UnsetArriveTangent();
						WeightedAttributes.UnsetLeaveTangent();
						WeightedAttributes.UnsetArriveTangentWeight();
						WeightedAttributes.UnsetLeaveTangentWeight();
						WeightedAttributes.SetTangentWeightMode(RCTWM_WeightedBoth);

					}
					else
					{
						KeyAttributesWeighted.RemoveAtSwap(Index, EAllowShrinking::No);
						KeyHandlesWeighted.RemoveAtSwap(Index, EAllowShrinking::No);
					}
				}
				else
				{
					AllKeyPositions.RemoveAtSwap(Index, EAllowShrinking::No);
					KeyHandles.RemoveAtSwap(Index, EAllowShrinking::No);
					KeyAttributesWeighted.RemoveAtSwap(Index, EAllowShrinking::No);
					KeyHandlesWeighted.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}

			if (AllKeyPositions.Num() > 0)
			{
				Curve->SetKeyAttributes(KeyHandles, AllKeyPositions);
				if (KeyAttributesWeighted.Num() > 0)
				{
					Curve->SetKeyAttributes(KeyHandlesWeighted, KeyAttributesWeighted);
				}
			}
		}
	}
}

void FCurveEditor::StraightenSelection()
{
	using namespace UE::CurveEditor;
	// FScopedKeyChange won't generate any transaction if no tangents are changed
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), CurveData, ECurveChangeFlags::KeyAttributes),
		LOCTEXT("StraightenTangents", "Straighten Tangents")
		);

	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyAttributes> AllKeyPositions;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		if (FCurveModel* Curve = FindCurve(Pair.Key))
		{
			KeyHandles.Reset(Pair.Value.Num());
			KeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

			AllKeyPositions.SetNum(KeyHandles.Num());
			Curve->GetKeyAttributes(KeyHandles, AllKeyPositions);

			// Straighten tangents, ignoring any keys that we can't set tangents on
			for (int32 Index = AllKeyPositions.Num()-1 ; Index >= 0; --Index)
			{
				FKeyAttributes& Attributes = AllKeyPositions[Index];
				if (Attributes.HasTangentMode() && Attributes.HasArriveTangent() && Attributes.HasLeaveTangent())
				{
					float NewTangent = (Attributes.GetLeaveTangent() + Attributes.GetArriveTangent()) * 0.5f;
					Attributes.SetArriveTangent(NewTangent).SetLeaveTangent(NewTangent);
					if (Attributes.GetTangentMode() == RCTM_Auto || Attributes.GetTangentMode() == RCTM_SmartAuto)
					{
						Attributes.SetTangentMode(RCTM_User);
					}
				}
				else
				{
					AllKeyPositions.RemoveAtSwap(Index, EAllowShrinking::No);
					KeyHandles.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}

			if (AllKeyPositions.Num() > 0)
			{
				Curve->SetKeyAttributes(KeyHandles, AllKeyPositions);
			}
		}
	}
}

bool FCurveEditor::CanFlattenOrStraightenSelection() const
{
	return Selection.Count() > 0;
}

void FCurveEditor::SmartSnapSelection()
{
	using namespace UE::CurveEditor;
	const FScopedTransaction Transaction(LOCTEXT("SmartSnapKeys", "Smart Snap")); // Selection and key change should be part of same undo / redo op.
	const FScopedSelectionChange SelectionChange(SharedThis(this));
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), ECurveChangeFlags::MoveKeysAndRemoveStackedKeys).TrackSelectedCurves()
		);
	
	TMap<FCurveModelID, FKeyHandleSet> OutKeysToSelect;
	EnumerateSmartSnappableKeys(*this, Selection.GetAll(), OutKeysToSelect,
		[](const FCurveModelID&, FCurveModel& CurveModel, const FSmartSnapResult& SnapResult)
		{
			ApplySmartSnap(CurveModel, SnapResult);
		});

	// Some keys might have been removed. Clean up the selection.
	Selection.Replace(OutKeysToSelect);
}

bool FCurveEditor::CanSmartSnapSelection() const
{
	return UE::CurveEditor::CanSmartSnapSelection(Selection);
}

void FCurveEditor::UpdateGeometry(const FGeometry& CurrentGeometry)
{
}

void FCurveEditor::SetRandomCurveColorsForSelected()
{
	const TArray<FCurveModelID> CurveModelIDs = GetSelectionFromTreeAndKeys().Array();
	if (CurveModelIDs.Num() == 0)
	{
		return;
	}

	// Calling FCurveModel::SetColor below triggers FCurveEditor's own delegate (set up in AddCurve) so skip those and trigger
	// OnCurveColorsChangedDelegate once at the end.
	SuspendBroadcast();
	ON_SCOPE_EXIT{ ResumeBroadcast(); };
	
	for (const FCurveModelID& CurveModelID : CurveModelIDs)
	{
		if (FCurveModel* Curve = FindCurve(CurveModelID))
		{
			const FLinearColor Color = UCurveEditorSettings::GetNextRandomColor();
			Curve->SetColor(Color);
			
			UObject* Object = nullptr;
			FString Name;
			Curve->GetCurveColorObjectAndName(&Object, Name);
			if (Object)
			{
				Settings->SetCustomColor(Object->GetClass(), Name, Color);
			}
		}
	}

	OnCurveColorsChangedDelegate.Broadcast(CurveModelIDs);
}

void FCurveEditor::SetCurveColorsForSelected()
{
	const TArray<FCurveModelID> CurveModelIDs = GetSelectionFromTreeAndKeys().Array();
	if (CurveModelIDs.Num() == 0)
	{
		return;
	}
	
	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.InitialColor = FindCurve(CurveModelIDs[0])->GetColor();
	PickerArgs.OnColorCommitted.BindLambda([WeakSelf = AsShared().ToWeakPtr(), CurveModelIDs](FLinearColor NewColor)
	{
		if (const TSharedPtr<FCurveEditor> Self = WeakSelf.Pin())
		{
			for (const FCurveModelID& CurveModelID : CurveModelIDs)
			{
				if (FCurveModel* Curve = Self->FindCurve(CurveModelID))
				{
					Curve->SetColor(NewColor);
					
					UObject* Object = nullptr;
					FString Name;
					Curve->GetCurveColorObjectAndName(&Object, Name);
					if (Object)
					{
						Self->Settings->SetCustomColor(Object->GetClass(), Name, NewColor);
					}
				}
				
				// This causes a redraw
				Self->OnCurveColorsChangedDelegate.Broadcast(CurveModelIDs);
			}
		}
	});

	// Calling FCurveModel::SetColor below triggers FCurveEditor's own delegate (set up in AddCurve) so skip those and trigger
	// OnCurveColorsChangedDelegate once at the end.
	SuspendBroadcast();
	ON_SCOPE_EXIT{ ResumeBroadcast(); };

	OpenColorPicker(PickerArgs);
}

bool FCurveEditor::IsToolActive(const FCurveEditorToolID InToolID) const
{
	if (ActiveTool.IsSet())
	{
		return ActiveTool == InToolID;
	}

	return false;
}

void FCurveEditor::MakeToolActive(const FCurveEditorToolID InToolID)
{
	if (ActiveTool.IsSet())
	{
		// Early out in the event that they're trying to switch to the same tool. This avoids
		// unwanted activation/deactivation calls.
		if (ActiveTool == InToolID)
		{
			return;
		}

		// Deactivate the current tool before we activate the new one.
		ToolExtensions[ActiveTool.GetValue()]->OnToolDeactivated();
	}

	ActiveTool.Reset();

	// Notify anyone listening that we've switched tools (possibly to an inactive one)
	OnActiveToolChangedDelegate.Broadcast(InToolID);

	if (InToolID != FCurveEditorToolID::Unset())
	{
		ActiveTool = InToolID;
		ToolExtensions[ActiveTool.GetValue()]->OnToolActivated();
	}
}

ICurveEditorToolExtension* FCurveEditor::GetCurrentTool() const
{
	if (ActiveTool.IsSet())
	{
		return ToolExtensions[ActiveTool.GetValue()].Get();
	}

	// If there is no active tool we return nullptr.
	return nullptr;
}

TSet<FCurveModelID> FCurveEditor::GetEditedCurves() const
{
	TArray<FCurveModelID> AllCurves;
	GetCurves().GenerateKeyArray(AllCurves);
	return TSet<FCurveModelID>(AllCurves);
}

void FCurveEditor::AddBufferedCurves(const TSet<FCurveModelID>& InCurves)
{
	// We make a copy of the curve data and store it.
	for (FCurveModelID CurveID : InCurves)
	{
		FCurveModel* CurveModel = FindCurve(CurveID);
		check(CurveModel);

		// Add a buffered curve copy if the curve model supports buffered curves
		TUniquePtr<IBufferedCurveModel> CurveModelCopy = CurveModel->CreateBufferedCurveCopy();
		if (CurveModelCopy) 
		{
			// Remove any existing buffered curves
			for (int32 BufferedCurveIndex = 0; BufferedCurveIndex < BufferedCurves.Num(); )
			{
				if (BufferedCurves[BufferedCurveIndex]->GetLongDisplayName() == CurveModel->GetLongDisplayName().ToString())
				{
					BufferedCurves.RemoveAt(BufferedCurveIndex);
				}
				else
				{
					++BufferedCurveIndex;
				}
			}

			BufferedCurves.Add(MoveTemp(CurveModelCopy)); 
		}
		else
		{
			UE_LOGF(LogCurveEditor, Warning, "Failed to buffer curve, curve model did not provide a copy.")
		}
	}
}


void FCurveEditor::ApplyBufferedCurveToTarget(const IBufferedCurveModel* BufferedCurve, FCurveModel* TargetCurve)
{
	check(TargetCurve);
	check(BufferedCurve);

	TArray<FKeyPosition> KeyPositions;
	TArray<FKeyAttributes> KeyAttributes;
	BufferedCurve->GetKeyPositions(KeyPositions);
	BufferedCurve->GetKeyAttributes(KeyAttributes);


	// Copy the data from the Buffered curve into the target curve. This just does wholesale replacement.
	const TArray<FKeyHandle> TargetKeyHandles = TargetCurve->GetAllKeys();

	double CurrentTime = 0.0;
	if (TSharedPtr<ITimeSliderController> TimeSliderController = WeakTimeSliderController.Pin())
	{
		FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());
	}
	// Clear our current keys from the target curve
	TargetCurve->RemoveKeys(TargetKeyHandles, CurrentTime);

	// Now put our buffered keys into the target curve
	TargetCurve->AddKeys(KeyPositions, KeyAttributes);
}

bool FCurveEditor::ApplyBufferedCurves(const TSet<FCurveModelID>& InCurvesToApplyTo, const bool bSwapBufferCurves)
{
	using namespace UE::CurveEditor;
	// Fyi, FScopedKeyChange won't make any entry in the Undo/Redo buffer if nothing was changed.
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(SharedThis(this), InCurvesToApplyTo),
		bSwapBufferCurves ? LOCTEXT("SwapBufferedCurves", "Swap Buffered Curves") : LOCTEXT("ApplyBufferedCurves", "Apply Buffered Curves")
		);

	// Each curve can specify an "Intention" name. This gives a little bit of context about how the curve is intended to be used,
	// without locking anyone into a specific set of intentions. When you go to apply the buffered curves, for each curve that you
	// want to apply it to, we can look in our stored curves to see if someone has the same intention. If there isn't a matching intention
	// then we skip and consider a fallback method (such as 1:1 copy). There is a lot of guessing still involved as there are complex
	// situations that users may try to use it in (such as buffering two sets of transform curves and applying it to two destination transform curves)
	// or trying to copy something with a name like "Focal Length" and pasting it onto a different track. We don't handle these cases for now,
	// but attempt to communicate it to the user via  toast notification when pasting fails for whatever reason.
	int32 NumCurvesMatchedByIntent = 0;
	int32 NumCurvesNoMatchedIntent = 0;
	bool bFoundAnyMatchedIntent = false;

	TMap<FString, int32> IntentMatchIndexes;
	for (const FCurveModelID& CurveModelID : InCurvesToApplyTo)
	{
		FCurveModel* TargetCurve = FindCurve(CurveModelID);
		check(TargetCurve);

		// Figure out what our destination thinks it's supposed to be used for, ie "Location.X"
		FString TargetIntent = TargetCurve->GetLongDisplayName().ToString();
		if (TargetIntent.IsEmpty())
		{
			// We don't try to match curves with no intent as that's just chaos.
			NumCurvesNoMatchedIntent++;
			continue;
		}

		// In an attempt to support buffering multiple curves with the same intention, we'll try to match them up in pairs. This means
		// for the first curve that we're trying to apply to, if the intention is "Location.X" we will search the buffered curves for a
		// "Location.X". Upon finding one, we store the index that it was found at, so the next time we try to find a curve with the same
		// intention, we look for the second "Location.X" and so forth. If we don't find a second "Location.X" in our buffered curves we'll
		// fall back to the first buffered one so you can 1:Many copy a curve.
		int32 BufferedCurveSearchIndexStart = 0;
		const int32* PreviouslyFoundIntent = IntentMatchIndexes.Find(TargetIntent);
		if (PreviouslyFoundIntent)
		{
			// Start our search on the next item in the array. If we don't find one, we'll fall back to the last one.
			BufferedCurveSearchIndexStart = IntentMatchIndexes[TargetIntent] + 1;
		}

		int32 MatchedBufferedCurveIndex = -1;
		for (int32 BufferedCurveIndex = BufferedCurveSearchIndexStart; BufferedCurveIndex < BufferedCurves.Num(); BufferedCurveIndex++)
		{
			if (BufferedCurves[BufferedCurveIndex]->GetLongDisplayName() == TargetIntent)
			{
				MatchedBufferedCurveIndex = BufferedCurveIndex;

				// Update our previously found intent to the latest one.
				IntentMatchIndexes.FindOrAdd(TargetIntent) = MatchedBufferedCurveIndex;
				break;
			}
		}

		// The Intent Match Indexes stores the latest index to find a valid curve, or the last one if no new valid one was found.
		// If there is an entry in the match indexes now, we can use that to figure out which buffered curve we'll pull from.
		// If we didn't find any more with the same intention, we fall back to the existing one (if it exists!)
		if (IntentMatchIndexes.Find(TargetIntent))
		{
			MatchedBufferedCurveIndex = IntentMatchIndexes[TargetIntent];
		}

		// Finally, we can try to use the matched curve if one was found.
		if (MatchedBufferedCurveIndex >= 0)
		{
			// We successfully matched, so count that one up!
			NumCurvesMatchedByIntent++;
			bFoundAnyMatchedIntent = true;

			const IBufferedCurveModel* BufferedCurve = BufferedCurves[MatchedBufferedCurveIndex].Get();

			TUniquePtr<IBufferedCurveModel> CurveModelCopy;
			if (bSwapBufferCurves)
			{
				CurveModelCopy = TargetCurve->CreateBufferedCurveCopy();
			}

			ApplyBufferedCurveToTarget(BufferedCurve, TargetCurve);

			if (bSwapBufferCurves)
			{
				BufferedCurves[MatchedBufferedCurveIndex] = MoveTemp(CurveModelCopy);
			}
		}
		else
		{
			// We couldn't find a match despite our best efforts
			NumCurvesNoMatchedIntent++;
		}
	}

	// If we managed to match any by intent, we're going to early out and assume that's what their intent was.
	if (bFoundAnyMatchedIntent)
	{
		const FText NotificationText = FText::Format(LOCTEXT("MatchedBufferedCurvesByIntent", "Applied {0}/{1} buffered curves to {2}/{3} target curves."),
			FText::AsNumber(IntentMatchIndexes.Num()), FText::AsNumber(BufferedCurves.Num()),		// We used X of Y total buffered curves
			FText::AsNumber(NumCurvesMatchedByIntent), FText::AsNumber(InCurvesToApplyTo.Num()));	// To apply to Z of W target curves,

		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 6.f;
		Info.bUseLargeFont = false;
		Info.bUseSuccessFailIcons = false;
		FSlateNotificationManager::Get().AddNotification(Info);

		if (NumCurvesNoMatchedIntent > 0)
		{
			const FText FailedNotificationText = FText::Format(LOCTEXT("NumCurvesNotMatchedByIntent", "Failed to find a buffered curve with the same intent for {0} target curves, skipping..."),
				FText::AsNumber(NumCurvesNoMatchedIntent));												// Leaving V many target curves unaffected due to no intent match.

			FNotificationInfo FailInfo(FailedNotificationText);
			FailInfo.ExpireDuration = 6.f;
			FailInfo.bUseLargeFont = false;
			FailInfo.bUseSuccessFailIcons = true;
			FSlateNotificationManager::Get().AddNotification(FailInfo);
		}

		// Early out
		return true;
	}

	// If we got this far, it means that the buffered curves have no recognizable relation to the target curves.
	// If the number of curves match, we'll just do a 1:1 mapping. This works for most cases where you're trying
	// to paste an unrelated curve onto another as it's likely that there's only one curve. We don't limit it to
	// one curve though, we'll just warn...
	if (InCurvesToApplyTo.Num() == BufferedCurves.Num())
	{
		// This will work great in the case there's only one curve. It'll guess if there's more than one, relying on
		// sets with no guaranteed order.
		TArray<FCurveModelID> CurvesToApplyTo = InCurvesToApplyTo.Array();
		
		for (int32 CurveIndex = 0; CurveIndex < InCurvesToApplyTo.Num(); CurveIndex++)
		{
			FCurveModel* TargetCurve = FindCurve(CurvesToApplyTo[CurveIndex]);

			TUniquePtr<IBufferedCurveModel> CurveModelCopy;
			if (bSwapBufferCurves)
			{
				CurveModelCopy = TargetCurve->CreateBufferedCurveCopy();
			}

			ApplyBufferedCurveToTarget(BufferedCurves[CurveIndex].Get(), TargetCurve);

			if (bSwapBufferCurves)
			{
				BufferedCurves[CurveIndex] = MoveTemp(CurveModelCopy);
			}
		}

		FText NotificationText;
		if (InCurvesToApplyTo.Num() == 1)
		{
			NotificationText = LOCTEXT("MatchedBufferedCurvesBySolo", "Applied buffered curve to target curve with no intention matching.");
		}
		else
		{
			NotificationText = LOCTEXT("MatchedBufferedCurvesByIndex", "Applied buffered curves with no intention matching. Order not guranteed.");
		}

		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 6.f;
		Info.bUseLargeFont = false;
		Info.bUseSuccessFailIcons = false;
		FSlateNotificationManager::Get().AddNotification(Info);

		// Early out
		return true;
	}

	// If we got this far, we have no idea what to do. They're trying to match a bunch of curves with no intention and different amounts. 
	// Warn of failure and give up.
	{
		const FText FailedNotificationText = LOCTEXT("NoBufferedCurvesMatched", "Failed to apply buffered curves, apply them one at a time instead.");

		FNotificationInfo FailInfo(FailedNotificationText);
		FailInfo.ExpireDuration = 6.f;
		FailInfo.bUseLargeFont = false;
		FailInfo.bUseSuccessFailIcons = true;
		FSlateNotificationManager::Get().AddNotification(FailInfo);
	}

	return false;
}

TSet<FCurveModelID> FCurveEditor::GetSelectionFromTreeAndKeys() const
{
	TSet<FCurveModelID> CurveModelIDs;

	// Buffer curves operates on the selected curves (tree selection or key selection)
	for (const TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : GetTreeSelection())
	{
		if (Pair.Value == ECurveEditorTreeSelectionState::Explicit)
		{
			const FCurveEditorTreeItem& TreeItem = GetTreeItem(Pair.Key);
			for (const FCurveModelID& CurveModelID : TreeItem.GetCurves())
			{
				CurveModelIDs.Add(CurveModelID);
			}
		}
	}

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		CurveModelIDs.Add(Pair.Key);
	}

	return CurveModelIDs;
}

bool FCurveEditor::IsActiveBufferedCurve(const TUniquePtr<IBufferedCurveModel>& BufferedCurve) const
{
	TSet<FCurveModelID> CurveModelIDs = GetSelectionFromTreeAndKeys();
	for (const FCurveModelID& CurveModelID : CurveModelIDs)
	{
		if (FCurveModel* Curve = FindCurve(CurveModelID))
		{
			if (Curve->GetLongDisplayName().ToString() == BufferedCurve.Get()->GetLongDisplayName())
			{
				return true;
			}
		}
	}

	return false;
}

void FCurveEditor::PostUndo(bool bSuccess)
{
	if (WeakPanel.IsValid())
	{
		WeakPanel.Pin()->PostUndo();
	}
}

void FCurveEditor::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void FCurveEditor::OnCustomColorsChanged()
{
	for (TPair<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : CurveData)
	{
		if (FCurveModel* Curve = CurvePair.Value.Get())
		{
			UObject* Object = nullptr;
			FString Name;
			Curve->GetCurveColorObjectAndName(&Object, Name);

			TOptional<FLinearColor> Color = Settings->GetCustomColor(Object->GetClass(), Name);
			if (Color.IsSet())
			{
				Curve->SetColor(Color.GetValue());
			}
			else
			{
				// Note: If the color is no longer defined, there's no way to update with the previously defined 
				// default color. The curve models would need to be rebuilt, but would cause selection/framing and 
				// other things to change. So, this is intentionally not implemented.
			}
		}
	}
}

void FCurveEditor::OnAxisSnappingChanged()
{
	TSharedPtr<SCurveEditorPanel> Panel = WeakPanel.Pin();
	if (Panel.IsValid())
	{
		Panel->UpdateAxisSnapping();
	}
}

TUniquePtr<FCurveModel> FCurveEditor::ResolveCurve(const FCurveModelID& InCurveID) const
{
	return ResolveCurveModelDelegate.IsBound()
	? ResolveCurveModelDelegate.Execute(UE::CurveEditor::FCurveModelLookUpArgs(*this, InCurveID))
	: nullptr;
}

void FCurveEditor::HandleCurveColorChanged(FCurveModelID CurveId)
{
	if (IsBroadcasting())
	{
		OnCurveColorsChangedDelegate.Broadcast(TConstArrayView<FCurveModelID>(&CurveId, 1));
	}
}

#undef LOCTEXT_NAMESPACE
