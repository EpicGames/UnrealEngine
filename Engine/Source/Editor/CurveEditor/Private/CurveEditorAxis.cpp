// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorAxis.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "CurveEditor.h"
#include "SCurveEditorView.h"
#include "CurveEditorAxis.h"


bool FCurveEditorAxis::HasLabels() const
{
	return NumericTypeInterface.IsValid();
}

FText FCurveEditorAxis::MakeLabel(double Value) const
{
	if (NumericTypeInterface)
	{
		return FText::FromString(NumericTypeInterface->ToString(Value));
	}
	return FText();
}

void FCurveEditorAxis::GetGridLines(const FCurveEditor& CurveEditor, const SCurveEditorView& View, FCurveEditorViewAxisID AxisID, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines, ECurveEditorAxisOrientation Axis) const
{

}

void FLinearCurveEditorAxis::GetGridLines(const FCurveEditor& CurveEditor, const SCurveEditorView& View, FCurveEditorViewAxisID AxisID, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines, ECurveEditorAxisOrientation Axis) const
{
	FCurveEditorScreenSpace ViewSpace = View.GetViewSpace();

	float Size = 1.0;
	float Min  = 0.0;
	float Max  = 1.0;

	if (Axis == ECurveEditorAxisOrientation::Horizontal)
	{
		FCurveEditorScreenSpaceH AxisSpace = View.GetHorizontalAxisSpace(AxisID);
		Size = AxisSpace.GetPhysicalWidth();
		Min  = AxisSpace.GetInputMin();
		Max  = AxisSpace.GetInputMax();


		CurveEditor::PopulateGridLineValues(
			ViewSpace.GetPhysicalWidth(),
			ViewSpace.GetInputMin(),
			ViewSpace.GetInputMax(),
			4,
			OutMajorGridLines,
			OutMinorGridLines);
	}
	else
	{
		FCurveEditorScreenSpaceV AxisSpace = View.GetVerticalAxisSpace(AxisID);
		Size = ViewSpace.GetPhysicalHeight();
		Min  = ViewSpace.GetOutputMin();
		Max  = ViewSpace.GetOutputMax();
	}

	CurveEditor::PopulateGridLineValues(Size, Min, Max, 4, OutMajorGridLines, OutMinorGridLines);
}
