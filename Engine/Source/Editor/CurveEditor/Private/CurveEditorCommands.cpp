// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "CurveEditorCommands"

UE_DEFINE_TCOMMANDS(FCurveEditorCommands)

void FCurveEditorCommands::RegisterCommands()
{
	UI_COMMAND(ZoomToFitHorizontal, "Fit Horizontal", "Zoom to Fit - Horizontal", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZoomToFitVertical, "Fit Vertical", "Zoom to Fit - Vertical", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZoomToFit, "Fit", "Zoom to Fit", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ZoomToFitAll, "FitAll", "Zoom to Fit All", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleInputSnapping, "Input Snapping", "Toggle Time Snapping", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleOutputSnapping, "Output Snapping", "Toggle Value Snapping", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(FlipCurveHorizontal, "Flip Curve Horizontal", "Flip curve horizontally", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FlipCurveVertical, "Flip Curve Vertical", "Flip curve vertically", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleExpandCollapseNodes, "Expand/Collapse Nodes", "Toggle expand or collapse selected nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::V) );
	UI_COMMAND(ToggleExpandCollapseNodesAndDescendants, "Expand/Collapse Nodes and Descendants", "Toggle expand or collapse selected nodes and descendants", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::V) );
	UI_COMMAND(ExpandAllNodes, "Expand All Nodes", "Expand all nodes in the curve editor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CollapseAllNodes, "Collapse All Nodes", "Collapse all nodes in the curve editor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SortAllNodes, "Sort Nodes", "Sort all nodes in the curve editor", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(InterpolationConstant, "Constant", "Constant interpolation", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Five));
	UI_COMMAND(InterpolationLinear, "Linear", "Linear interpolation", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Four));
	UI_COMMAND(InterpolationCubicSmartAuto, "Smart Auto", "Cubic interpolation - Smart Automatic tangents", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Zero));
	UI_COMMAND(InterpolationCubicAuto, "Auto", "Cubic interpolation - Automatic tangents", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::One));
	UI_COMMAND(InterpolationCubicUser, "User", "Cubic interpolation - User flat tangents", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Two));
	UI_COMMAND(InterpolationCubicBreak, "Break", "Cubic interpolation - User broken tangents", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Three));
	UI_COMMAND(InterpolationToggleWeighted, "Weighted Tangents", "Toggle weighted tangents for cubic interpolation modes. Only supported on some curve types", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::W));

	UI_COMMAND(FlattenTangents, "Flatten", "Flatten Tangents", EUserInterfaceActionType::Button, FInputChord(EKeys::Six));
	UI_COMMAND(StraightenTangents, "Straighten", "Straighten Tangents", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(SmartSnapKeys, "Smart Snap", "Snaps selected keys to the closest whole frame.\nThe key is placed where the curve intersects with the frame (imagine vertical line going down from the frame).", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::H));

	UI_COMMAND(MatchLastTangentToFirst, "Match Last Tangent To First", "Match last keys tangents to the first keys tangents", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MatchFirstTangentToLast, "Match First Tangent To Last", "Match first keys tangents to the last keys tangents", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BakeCurve, "Bake", "Bake curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReduceCurve, "Reduce", "Reduce curve", EUserInterfaceActionType::Button, FInputChord());

	// Pre and Post Infinity
	UI_COMMAND(SetPreInfinityExtrapCycle, "Cycle", "Cycle creates a repeating cycle from the first to last key, effectively modulating the input time. This can create jumps if the terminus values are not the same value.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPreInfinityExtrapCycleWithOffset, "Cycle with Offset", "Creates a repeating cycle where the value is added to the last value of the previous Cycle. This will avoid jumps but will cause drift over time if there's a net change in value.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPreInfinityExtrapOscillate, "Oscillate (Ping Pong)", "Creates a repeating cycle which ping pongs and will play from beginning to end, then end to beginning.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPreInfinityExtrapLinear, "Linear", "Linearly interpolates based on the in tangent of the first key.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPreInfinityExtrapConstant, "Constant", "Extrapolation will always return the value of the first key.", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(SetPostInfinityExtrapCycle, "Cycle", "Cycle creates a repeating cycle from the first to last key, effectively modulating the input time. This can create jumps if the terminus values are not the same value", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPostInfinityExtrapCycleWithOffset, "Cycle with Offset", "Creates a repeating cycle where the value is added to the last value of the previous Cycle. This will avoid jumps but will cause drift over time if there's a net change in value.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPostInfinityExtrapOscillate, "Oscillate (Ping Pong)", "Creates a repeating cycle which ping pongs and will play from beginning to end, then end to beginning.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPostInfinityExtrapLinear, "Linear", "Linearly interpolates based on the in tangent of the last key.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetPostInfinityExtrapConstant, "Constant", "Extrapolation will always return the value of the last key.", EUserInterfaceActionType::RadioButton, FInputChord());


	// Tangent Visibility
	UI_COMMAND(SetAllTangentsVisibility, "All Tangents", "Show all tangents in the curve editor.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetUserTangentsVisibility, "User Tangents", "Show tangents that are user edited, i.e. set to user or break should be visible.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetSelectedKeysTangentVisibility, "Selected Keys", "Show tangents for selected keys in the curve editor.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetNoTangentsVisibility, "No Tangents", "Show no tangents in the curve editor.", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ToggleAutoFrameCurveEditor, "Auto Frame Curves", "Auto frame curves when they are selected.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(ToggleSnapTimeToSelection, "Snap Time to Selection", "Snap the current time to the first selected key time.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(ToggleShowBufferedCurves, "Buffered Curves", "Show buffered curves for the selected curves.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(ToggleShowCurveEditorCurveToolTips, "Curve Tool Tips", "Show a tool tip with name and values when hovering over a curve.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(ToggleShowTransportControls, "Transport Controls", "Show transport controls (only used if the transport controls are not popped out)", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowBars, "Bars", "Show Bars like Constraints and Spaces", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowValueIndicatorLines, "Value Indicators", "Show horizontal, dotted lines on the selected min & max valued key handles", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AddKeyHovered, "Add Key", "Add a new key to this curve at the current position.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(PasteKeysHovered, "Paste", "Pastes the keys from the clipboard overwriting any key in destination track between the first and last pasted keys (default)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V));
	UI_COMMAND(PasteAndMerge, "Paste & Merge", "Pastes the keys from the clipboard over the keys in the graph.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V));
	UI_COMMAND(PasteRelative, "Paste Relative", "Pastes the keys from the clipboard and aligns them to the nearest key to the left of the scrubber.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::V));
	
	UI_COMMAND(AddKeyToAllCurves, "Add Key", "Add a new key to all curves at the current time.", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter) );
	UI_COMMAND(AddKeyToSelectedCurves, "Add Key to Selected Curves", "Add a new key to all curves that have selected keys at the current time.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetKeysToDefaultValues, "Reset to Default", "Reset selected keys to their default value", EUserInterfaceActionType::Button, FInputChord());

	// Curve Editor Colors
	UI_COMMAND(SetRandomCurveColorsForSelected, "Set Random Curve Colors", "Set random colors on the selected curves. Note they are stored in the Level Sequence Actor Editor Preferences.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetCurveColorsForSelected, "Set Curve Color For Selected", "Set the chosen color on the selected curves. Note they are stored in the Level Sequence Actor Editor Preferences.", EUserInterfaceActionType::Button, FInputChord());

	// Graph Viewing Modes
	UI_COMMAND(SetViewModeAbsolute, "Absolute View Mode", "Absolute view displays all curves overlapping with the Y axis proportionally scaled.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetViewModeStacked, "Stacked View Mode", "Stacked view displays each curve in its own graph with the Y axis normalized [-1, 1].", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetViewModeNormalized, "Normalized View Mode", "Normalized view displays all curves overlapping with the Y axis normalized [-1, 1].", EUserInterfaceActionType::RadioButton, FInputChord());

	// Deactivate the currently active tool
	UI_COMMAND(DeactivateCurrentTool, "Selection", "Deactivates the current tool and returns to just supporting selection.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Q));

	// User Implementable Filter window
	UI_COMMAND(OpenUserImplementableFilterWindow, "Filter...", "Opens a window which lets you choose from user implementable filter classes with advanced settings.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(SelectAllKeys, "Select All Keys", "Select all keys.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	// Deselect any keys that the user has selected.
	UI_COMMAND(DeselectAllKeys, "Deselect All Keys", "Clears your current key selection.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D));

	UI_COMMAND(SelectForward, "Select All Keys Forward", "Select all keys forward from the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket));
	UI_COMMAND(SelectBackward, "Select All Keys Backward", "Select all keys backward from the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket));

	UI_COMMAND(SelectNone, "Select None", "Select none", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(InvertSelection, "Invert Selection", "Invert the selection of keys on curves with selected keys.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::I));

	// Buffer and Apply Curves. Like copy and paste, but directly onto the curves they were stored from.
	// These names are overwritten in CurveEditorContextMenu to show the number of stashed curves.
	UI_COMMAND(BufferVisibleCurves, "Buffer Curves", "Stores a copy of the selected curves which can be applied back onto themselves.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwapBufferedCurves, "Swap Buffered Curves", "Applies the buffered curves to the curves they were stored from and stores the current curves to the buffer.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyBufferedCurves, "Apply Buffered Curves", "Applies the buffered curves to the curves they were stored from.", EUserInterfaceActionType::Button, FInputChord());

	// Axis Snapping
	UI_COMMAND(SetAxisSnappingNone, "Both", "Disable axis snapping and allow movement on both the X and Y directions.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetAxisSnappingHorizontal, "X Only", "Snap transform tool axis movement to X direction.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetAxisSnappingVertical, "Y Only", "Snap transform tool axis movement to Y direction.", EUserInterfaceActionType::Button, FInputChord());	
	
	//Key Movement
	UI_COMMAND(TranslateSelectedKeysLeft, "Translate Selected Keys Left", "Translate selected keys one frame to the left", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Left));
	UI_COMMAND(TranslateSelectedKeysRight, "Translate Selected Keys Right", "Translate selected keys one frame to the right", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Right));

	// Time Management
	UI_COMMAND(ScrubTime, "Scrub Time", "Scrub mouse left and right to change time", EUserInterfaceActionType::Button, FInputChord(EKeys::B));
}

#undef LOCTEXT_NAMESPACE
