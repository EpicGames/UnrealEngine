// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorContextMenu.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Delegates/Delegate.h"
#include "Filters/PromotedFilterContainer.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/UniquePtr.h"
#include "UObject/UnrealNames.h"

class IBufferedCurveModel;

#define LOCTEXT_NAMESPACE "CurveEditorContextMenu"

namespace UE::CurveEditor::ContextDetail
{
/** Holds shared args to make calling maker functions easier. */
class FContextMenuBuilder
{
public:
	
	const TSharedRef<FCurveEditor> CurveEditor;
	const TOptional<FCurvePointHandle> ClickedPoint;
	const TOptional<FCurveModelID> HoveredCurveID;
	
	const int32 NumSelectedKeys = CurveEditor->GetSelection().Count();
	const int32 NumActiveCurves = CurveEditor->GetSelectionFromTreeAndKeys().Num();
	int32 NumBufferedCurves = 0;
	
	// We change the name to reflect the current number of curves selected.
	TAttribute<FText> BufferedCurvesText;
	TAttribute<FText> SwapBufferedCurvesText;
	TAttribute<FText> ApplyBufferedCurvesText;

	const FCurveModel* HoveredCurve = HoveredCurveID.IsSet() ? CurveEditor->FindCurve(HoveredCurveID.GetValue()) : nullptr;
	
	FContextMenuBuilder(TSharedRef<FCurveEditor> CurveEditor, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurveID)
		: CurveEditor(CurveEditor)
		, ClickedPoint(ClickedPoint)
		, HoveredCurveID(HoveredCurveID)
	{
		for (const TUniquePtr<IBufferedCurveModel>& BufferedCurve : CurveEditor->GetBufferedCurves())
		{
			if (CurveEditor->IsActiveBufferedCurve(BufferedCurve))
			{
				++NumBufferedCurves;
			}
		}
		
		BufferedCurvesText = FText::Format(LOCTEXT("BufferCurvesContextMenu", "Buffer {0} Curves"), NumActiveCurves);
		SwapBufferedCurvesText = FText::Format(
			LOCTEXT("SwapBufferedCurvesContextMenu", "Swap {0} Buffered Curves onto {1} Selected Curves"), NumBufferedCurves, NumActiveCurves
			);
		ApplyBufferedCurvesText = FText::Format(
			LOCTEXT("ApplyBufferedCurvesContextMenu", "Apply {0} Buffered Curves onto {1} Selected Curves"), NumBufferedCurves, NumActiveCurves
			);
	}

	void Build(FMenuBuilder& MenuBuilder)
	{
		// We prioritize key selections over curve selections to reduce the pixel-perfectness needed
		// to edit the keys (which is more common than curves). Right clicking on a key or an empty space
		// should show the key menu, otherwise we show the curve menu (ie: right clicking on a curve, not 
		// directly over a key).
		const bool bBuildKeyMenu = NumSelectedKeys > 0 && (!HoveredCurveID.IsSet() || ClickedPoint.IsSet());
		if (bBuildKeyMenu)
		{
			BuildKeyMenu(MenuBuilder);
		}
		else
		{
			// Test if at least one curve is editable
			bool bIsReadOnly = true;
			TSet<FCurveModelID> CurvesToAddTo;
			for(const FCurveModelID& CurveModelID : CurveEditor->GetEditedCurves())
			{
				if (const FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID))
				{
					if (!CurveModel->IsReadOnly())
					{
						bIsReadOnly = false;
						break;
					}
				}
			}

			if (HoveredCurve)
			{
				BuildHoveredCurve(MenuBuilder, bIsReadOnly);
			}
			else
			{
				BuildBackgroundMenu(MenuBuilder, bIsReadOnly);
			}
		}
	}
	
private:

	/** Content when clicking a key */
	void BuildKeyMenu(FMenuBuilder& MenuBuilder);
	/** Adds commands for changing key types */
	void AppendTangentTypes(FMenuBuilder& MenuBuilder);

	/** Content when clicking a curve */
	void BuildHoveredCurve(FMenuBuilder& MenuBuilder, bool bIsReadOnly);
	/** Content when clicking into the background */
	void BuildBackgroundMenu(FMenuBuilder& MenuBuilder, bool bIsReadOnly);

	/** Appends items that are shared in all menu types (key, curve, and background). */
	void AppendSharedItems(FMenuBuilder& MenuBuilder);
};
	
void FContextMenuBuilder::BuildKeyMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CurveEditorKeySection", FText::Format(LOCTEXT("CurveEditorKeySection", "{0} Selected {0}|plural(one=Key,other=Keys)"), NumSelectedKeys));
	{
		bool bIsReadOnly = false;
		if (HoveredCurve)
		{
			bIsReadOnly = HoveredCurve->IsReadOnly();
		}
		else
		{
			if (ClickedPoint.IsSet())
			{
				if (FCurveModel* ClickedPointCurve = CurveEditor->FindCurve(ClickedPoint->CurveID))
				{
					bIsReadOnly = ClickedPointCurve->IsReadOnly();
				}
			}
		}

		if (!bIsReadOnly)
		{
			// Modify Data
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().PasteAndMerge);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().PasteRelative);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().FlattenTangents);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().StraightenTangents);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SmartSnapKeys);
			
			MenuBuilder.AddMenuSeparator();
			AppendTangentTypes(MenuBuilder);
			MenuBuilder.AddMenuSeparator();
		}

		// Buffer Curves
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().BufferVisibleCurves, NAME_None, BufferedCurvesText);
		if (!bIsReadOnly)
		{
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SwapBufferedCurves, NAME_None, SwapBufferedCurvesText);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ApplyBufferedCurves, NAME_None, ApplyBufferedCurvesText);
		}
		MenuBuilder.AddSeparator();
		
		AppendSharedItems(MenuBuilder);
	}
	MenuBuilder.EndSection();
}

void FContextMenuBuilder::AppendTangentTypes(FMenuBuilder& MenuBuilder)
{
	int32 SupportedTangentTypes = CurveEditor->GetSupportedTangentTypes();
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicSmartAuto)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicSmartAuto);
	};
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicAuto)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicAuto);
	};
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicUser)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicUser);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicBreak)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicBreak);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationLinear)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationLinear);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationConstant)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationConstant);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicWeighted)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationToggleWeighted);
	}
}

void FContextMenuBuilder::BuildHoveredCurve(FMenuBuilder& MenuBuilder, bool bIsReadOnly)
{
	MenuBuilder.BeginSection("CurveEditorCurveSection", FText::Format(LOCTEXT("CurveNameFormat", "Curve '{0}'"), HoveredCurve->GetLongDisplayName()));
	{
		// Modify Curve
		if (!HoveredCurve->IsReadOnly())
		{
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().AddKeyHovered);
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().PasteKeysHovered);

			MenuBuilder.AddMenuSeparator();
			
			MenuBuilder.AddSubMenu(LOCTEXT("PreInfinityText", "Pre-Infinity"), FText(), FNewMenuDelegate::CreateLambda(
				[](FMenuBuilder& SubMenu)
				{
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapCycle);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapLinear);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapConstant);
				})
			);

			MenuBuilder.AddSubMenu(LOCTEXT("PostInfinityText", "Post-Infinity"), FText(), FNewMenuDelegate::CreateLambda(
				[](FMenuBuilder& SubMenu)
				{
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapCycle);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapLinear);
					SubMenu.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapConstant);
				})
			);

			MenuBuilder.AddMenuSeparator();

			// Buffer Curves
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().BufferVisibleCurves, NAME_None, BufferedCurvesText);
			if (!bIsReadOnly)
			{
				MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SwapBufferedCurves, NAME_None, SwapBufferedCurvesText);
				MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ApplyBufferedCurves, NAME_None, ApplyBufferedCurvesText);
			}
			MenuBuilder.AddSeparator();
		}
	
		AppendSharedItems(MenuBuilder);
	}
	MenuBuilder.EndSection();
}

void FContextMenuBuilder::BuildBackgroundMenu(FMenuBuilder& MenuBuilder, bool bIsReadOnly)
{
	MenuBuilder.BeginSection("CurveEditorAllCurveSections", LOCTEXT("CurveEditorAllCurveSections", "All Curves"));
	{
		if (!bIsReadOnly)
		{
			// Modify Curves
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().AddKeyToAllCurves);

			MenuBuilder.AddSeparator();

			// Buffer Curves
			MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().BufferVisibleCurves, NAME_None, BufferedCurvesText);
			if (!bIsReadOnly)
			{
				MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SwapBufferedCurves, NAME_None, SwapBufferedCurvesText);
				MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ApplyBufferedCurves, NAME_None, ApplyBufferedCurvesText);
			}
			MenuBuilder.AddSeparator();
		}

		AppendSharedItems(MenuBuilder);
	}
	MenuBuilder.EndSection();
}

void FContextMenuBuilder::AppendSharedItems(FMenuBuilder& MenuBuilder)
{
	// Select
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SelectAllKeys);

	// Filters
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().OpenUserImplementableFilterWindow);
	CurveEditor->GetToolbarPromotedFilters()->AppendToBuilder(MenuBuilder);
			
	// View
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ZoomToFit);
}
}

void FCurveEditorContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedRef<FCurveEditor> CurveEditor, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurveID)
{
	UE::CurveEditor::ContextDetail::FContextMenuBuilder Builder(CurveEditor, ClickedPoint, HoveredCurveID);
	Builder.Build(MenuBuilder);
}

#undef LOCTEXT_NAMESPACE