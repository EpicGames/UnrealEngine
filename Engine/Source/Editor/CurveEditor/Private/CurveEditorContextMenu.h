// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class FMenuBuilder;
struct FCurveModelID;
struct FCurvePointHandle;

struct FCurveEditorContextMenu
{
	static void BuildMenu(FMenuBuilder& MenuBuilder, TSharedRef<FCurveEditor> CurveEditor, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurve);
};