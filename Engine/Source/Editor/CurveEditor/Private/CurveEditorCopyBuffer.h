// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorCopyBuffer.generated.h"

UCLASS()
class UCurveEditorCopyableCurveKeys : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY()
	TArray<FKeyPosition> KeyPositions;

	UPROPERTY()
	TArray<FKeyAttributes> KeyAttributes;

	/** This curve's short display name. Used in situations where other mechanisms provide enough context about what the curve is (such as "X") */
	UPROPERTY()
	FString ShortDisplayName;

	/** This curve's long display name. Used in situations where the UI doesn't provide enough context about what the curve is otherwise (such as "Floor.Transform.X") */
	UPROPERTY()
	FString LongDisplayName;

	/** This curve's short intention (such as Transform.X or Scale.X). Used internally to match up curves when saving/restoring curves between different objects. */
	UPROPERTY()
	FString IntentionName;

	/** 
	* This curve's long intention (such as foot_fk_l.Transform.X or foot_fk_r.Scale.X). Used internally to match up curves when saving/restoring curves between different objects.
	* Long intention names have priority in copy/paste over short intention names, but we fall back to short intention if it's unclear what the user is trying to do.
	*/
	UPROPERTY()
	FString LongIntentionName;
};

UCLASS()
class UCurveEditorCopyBuffer : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UCurveEditorCopyableCurveKeys>> Curves;

	UPROPERTY()
	double TimeOffset;
	
	UPROPERTY()
	bool bAbsolutePosition;
};

