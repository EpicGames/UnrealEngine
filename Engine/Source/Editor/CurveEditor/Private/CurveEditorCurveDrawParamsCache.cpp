// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorCurveDrawParamsCache.h"

#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorCurveDrawParamsHandle.h"
#include "CurveEditorHelpers.h"
#include "CurveModel.h"
#include "ICurveEditorCurveCachePool.h"
#include "SCurveEditorView.h"

namespace UE::CurveEditor
{
	int32 GCurveEditorUseCurveCachePool = 1;

	static FAutoConsoleVariableRef CCurveEditorUseCurveCachePool(
		TEXT("CurveEditor.UseCurveCachePool"),
		GCurveEditorUseCurveCachePool,
		TEXT("Enables improved curve editor performance by using a curve cache pool"));

	void FCurveDrawParamsCache::Invalidate(const TSharedRef<SCurveEditorView>& CurveEditorView, TConstArrayView<FCurveModelID> ModelIDs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCurveDrawParamsCache::Invalidate);

		WeakCurveEditor = CurveEditorView->GetCurveEditor();
		
		UpdateCurveCacheFlags(CurveEditorView);
		DrawCurves(CurveEditorView, ModelIDs);
	}

	void FCurveDrawParamsCache::UpdateAllCurveDrawParamSynchonous(const TSharedRef<SCurveEditorView>& CurveEditorView, const TArray<FCurveModelID>& CurveModelIDs, TArray<FCurveDrawParams>& OutParams)
	{
		if (!WeakCurveEditor.IsValid())
		{
			return;
		}
		const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();

		CachedDrawParams.Reset(CurveModelIDs.Num());
		for (const FCurveModelID& CurveModelID : CurveModelIDs)
		{
			if (FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID))
			{
				const FCurveEditorScreenSpace ScreenSpace = CurveEditorView->GetCurveSpace(CurveModelID);

				FCurveDrawParams NewDrawParams(CurveModelID);
				UpdateCurveDrawParamsSynchonous(ScreenSpace, CurveModel, CurveModelID, NewDrawParams);
				OutParams.Add(NewDrawParams);
			}
		}
	}
	
	void FCurveDrawParamsCache::UpdateCurveDrawParamsSynchonous(const FCurveEditorScreenSpace& CurveSpace, FCurveModel* CurveModel, const FCurveModelID& ModelID, FCurveDrawParams& OutParams)
	{
		if (!WeakCurveEditor.IsValid())
		{
			return;
		}
		const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();

		const double InputMin = CurveSpace.GetInputMin();
		const double InputMax = CurveSpace.GetInputMax();

		const double DisplayRatio = (CurveSpace.PixelsPerOutput() / CurveSpace.PixelsPerInput());

		const FKeyHandleSet* SelectedKeys = CurveEditor->GetSelection().GetAll().Find(ModelID);

		// Create a new set of Curve Drawing Parameters to represent this particular Curve
		OutParams.Color = CurveModel->GetColor();
		OutParams.Thickness = CurveModel->GetThickness();
		OutParams.DashLengthPx = CurveModel->GetDashLength();
		OutParams.bKeyDrawEnabled = CurveModel->IsKeyDrawEnabled();

		// Gather the display metrics to use for each key type. This allows a Curve Model to override
		// whether or not the curve supports Keys, Arrive/Leave Tangents, etc. If the Curve Model doesn't
		// support a particular capability we can skip drawing them.
		CurveModel->GetKeyDrawInfo(ECurvePointType::ArriveTangent, FKeyHandle::Invalid(), OutParams.ArriveTangentDrawInfo);
		CurveModel->GetKeyDrawInfo(ECurvePointType::LeaveTangent, FKeyHandle::Invalid(), OutParams.LeaveTangentDrawInfo);

		// Gather the interpolating points in input/output space
		TArray<TTuple<double, double>> InterpolatingPoints;

		CurveModel->DrawCurve(*CurveEditor, CurveSpace, InterpolatingPoints);
		OutParams.InterpolatingPoints.Reset(InterpolatingPoints.Num());

		// An Input Offset allows for a fixed offset to all keys, such as displaying them in the middle of a frame instead of at the start.
		double InputOffset = CurveModel->GetInputDisplayOffset();

		// Convert the interpolating points to screen space
		for (const TTuple<double, double>& Point : InterpolatingPoints)
		{
			OutParams.InterpolatingPoints.Add(
				FVector2D(
					CurveSpace.SecondsToScreen(Point.Get<0>() + InputOffset),
					CurveSpace.ValueToScreen(Point.Get<1>())
				)
			);
		}

		TArray<FKeyHandle> VisibleKeys;
		CurveModel->GetKeys(InputMin, InputMax, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), VisibleKeys);

		// Always reset the points to cover case going from 1 to 0 keys
		OutParams.Points.Reset(VisibleKeys.Num());

		if (VisibleKeys.Num())
		{
			const ECurveEditorTangentVisibility TangentVisibility = CurveEditor->GetSettings()->GetTangentVisibility();

			TArray<FKeyPosition> AllKeyPositions;
			TArray<FKeyAttributes> AllKeyAttributes;

			AllKeyPositions.SetNum(VisibleKeys.Num());
			AllKeyAttributes.SetNum(VisibleKeys.Num());

			CurveModel->GetKeyPositions(VisibleKeys, AllKeyPositions);
			CurveModel->GetKeyAttributes(VisibleKeys, AllKeyAttributes);
			for (int32 Index = 0; Index < VisibleKeys.Num(); ++Index)
			{
				const FKeyHandle      KeyHandle = VisibleKeys[Index];
				const FKeyPosition& KeyPosition = AllKeyPositions[Index];
				const FKeyAttributes& Attributes = AllKeyAttributes[Index];

				static_assert(static_cast<int32>(ECurveEditorTangentVisibility::Num) == 4, "Update drawing algorithm");
				const bool bShowTangents = TangentVisibility == ECurveEditorTangentVisibility::AllTangents
					|| (TangentVisibility == ECurveEditorTangentVisibility::SelectedKeys && SelectedKeys &&
						(SelectedKeys->Contains(VisibleKeys[Index], ECurvePointType::Any)))
					|| (TangentVisibility == ECurveEditorTangentVisibility::UserTangents && Attributes.HasTangentMode()
						&& (Attributes.GetTangentMode() == RCTM_User || Attributes.GetTangentMode() == RCTM_Break))
				;

				const double TimeScreenPos = CurveSpace.SecondsToScreen(KeyPosition.InputValue + InputOffset);
				const double ValueScreenPos = CurveSpace.ValueToScreen(KeyPosition.OutputValue);

				// Add this key
				FCurvePointInfo Key(KeyHandle);
				Key.ScreenPosition = FVector2D(TimeScreenPos, ValueScreenPos);
				Key.LayerBias = 2;

				// Add draw info for the specific key
				CurveModel->GetKeyDrawInfo(ECurvePointType::Key, KeyHandle, /*Out*/ Key.DrawInfo);
				OutParams.Points.Add(Key);

				if (bShowTangents && Attributes.HasArriveTangent())
				{
					float ArriveTangent = Attributes.GetArriveTangent();

					FCurvePointInfo ArriveTangentPoint(KeyHandle);
					ArriveTangentPoint.Type = ECurvePointType::ArriveTangent;


					if (Attributes.HasTangentWeightMode() && Attributes.HasArriveTangentWeight() &&
						(Attributes.GetTangentWeightMode() == RCTWM_WeightedBoth || Attributes.GetTangentWeightMode() == RCTWM_WeightedArrive))
					{
						FVector2D TangentOffset = ::CurveEditor::ComputeScreenSpaceTangentOffset(CurveSpace, ArriveTangent, -Attributes.GetArriveTangentWeight());
						ArriveTangentPoint.ScreenPosition = Key.ScreenPosition + TangentOffset;
					}
					else
					{
						float PixelLength = 60.0f;
						ArriveTangentPoint.ScreenPosition = Key.ScreenPosition + ::CurveEditor::GetVectorFromSlopeAndLength(ArriveTangent * -DisplayRatio, -PixelLength);
					}
					ArriveTangentPoint.LineDelta = Key.ScreenPosition - ArriveTangentPoint.ScreenPosition;
					ArriveTangentPoint.LayerBias = 1;

					// Add draw info for the specific tangent
					FKeyDrawInfo TangentDrawInfo;
					CurveModel->GetKeyDrawInfo(ECurvePointType::ArriveTangent, KeyHandle, /*Out*/ ArriveTangentPoint.DrawInfo);

					OutParams.Points.Add(ArriveTangentPoint);
				}

				if (bShowTangents && Attributes.HasLeaveTangent())
				{
					float LeaveTangent = Attributes.GetLeaveTangent();

					FCurvePointInfo LeaveTangentPoint(KeyHandle);
					LeaveTangentPoint.Type = ECurvePointType::LeaveTangent;

					if (Attributes.HasTangentWeightMode() && Attributes.HasLeaveTangentWeight() &&
						(Attributes.GetTangentWeightMode() == RCTWM_WeightedBoth || Attributes.GetTangentWeightMode() == RCTWM_WeightedLeave))
					{
						FVector2D TangentOffset = ::CurveEditor::ComputeScreenSpaceTangentOffset(CurveSpace, LeaveTangent, Attributes.GetLeaveTangentWeight());

						LeaveTangentPoint.ScreenPosition = Key.ScreenPosition + TangentOffset;
					}
					else
					{
						float PixelLength = 60.0f;
						LeaveTangentPoint.ScreenPosition = Key.ScreenPosition + ::CurveEditor::GetVectorFromSlopeAndLength(LeaveTangent * -DisplayRatio, PixelLength);
					}

					LeaveTangentPoint.LineDelta = Key.ScreenPosition - LeaveTangentPoint.ScreenPosition;
					LeaveTangentPoint.LayerBias = 1;

					// Add draw info for the specific tangent
					FKeyDrawInfo TangentDrawInfo;
					CurveModel->GetKeyDrawInfo(ECurvePointType::LeaveTangent, KeyHandle, /*Out*/ LeaveTangentPoint.DrawInfo);

					OutParams.Points.Add(LeaveTangentPoint);
				}
			}
		}
	}

	FCurveDrawParamsCache::FCurveDrawParamsCache()
	{
		CurveCacheFlags = ECurveCacheFlags::UpdateAll;
		CachedCurveEditorData.ActiveCurvesSerialNumber = 0xFFFFFFFF;
		CachedCurveEditorData.SelectionSerialNumber = 0xFFFFFFFF;
		CachedCurveEditorData.CachedGeometrySize.X = -1;
		CachedCurveEditorData.CachedGeometrySize.Y = -1;
	}

	void FCurveDrawParamsCache::UpdateCurveDrawParams(const TSharedRef<SCurveEditorView>& CurveEditorView, const TArray<FCurveDrawParamsHandle>& CurveDrawParamsHandles)
	{
		if (!WeakCurveEditor.IsValid() || IsEngineExitRequested())
		{
			return;
		}
		const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();

		const bool bUseCurveCachePool = GCurveEditorUseCurveCachePool > 0;

		TSet<ICurveEditorCurveCachePool*> CachePools;
		for (const FCurveDrawParamsHandle& CurveDrawParamsHandle : CurveDrawParamsHandles)
		{
			if (FCurveModel* CurveModel = CurveEditor->FindCurve(CurveDrawParamsHandle.GetID()))
			{
				const FCurveEditorScreenSpace ScreenSpace = CurveEditorView->GetCurveSpace(CurveDrawParamsHandle.GetID());

				ICurveEditorCurveCachePool* CachePool = bUseCurveCachePool ? CurveModel->DrawCurveToCachePool(CurveEditor, CurveDrawParamsHandle, ScreenSpace) : nullptr;
				if (CachePool)
				{
					CachePools.Add(CachePool);
				}
				else
				{
					if (FCurveDrawParams* CurveDrawParams = CurveDrawParamsHandle.Get())
					{
						UpdateCurveDrawParamsSynchonous(ScreenSpace, CurveModel, CurveDrawParamsHandle.GetID(), *CurveDrawParams);
					}
				}
			}
		}

		for (ICurveEditorCurveCachePool* CachePool : CachePools)
		{
			CachePool->DrawCachedCurves(WeakCurveEditor);
		}
	}

	void FCurveDrawParamsCache::UpdateCurveCacheFlags(const TSharedRef<SCurveEditorView>& CurveEditorView)
	{
		if (!WeakCurveEditor.IsValid())
		{
			return;
		}
		const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();

		if (CurveEditor->GetActiveCurvesSerialNumber() != CachedCurveEditorData.ActiveCurvesSerialNumber)
		{
			CachedCurveEditorData.ActiveCurvesSerialNumber = CurveEditor->GetActiveCurvesSerialNumber();
			CurveCacheFlags = ECurveCacheFlags::UpdateAll;
		}

		if (CachedCurveEditorData.TangentVisibility != CurveEditor->GetSettings()->GetTangentVisibility())
		{
			CachedCurveEditorData.TangentVisibility = CurveEditor->GetSettings()->GetTangentVisibility();
			CurveCacheFlags = ECurveCacheFlags::UpdateAll;
		}

		if (CachedCurveEditorData.SelectionSerialNumber != CurveEditor->GetSelection().GetSerialNumber())
		{
			CachedCurveEditorData.SelectionSerialNumber = CurveEditor->GetSelection().GetSerialNumber();
			CurveCacheFlags = ECurveCacheFlags::UpdateAll;
		}

		// Only get view values if we need to since we will reset them every time we get all
		if (CurveCacheFlags != ECurveCacheFlags::UpdateAll)
		{
			const double OutputMin = CurveEditorView->GetOutputMin();
			const double OutputMax = CurveEditorView->GetOutputMax();

			if (OutputMin != CachedCurveEditorData.OutputMin || OutputMax != CachedCurveEditorData.OutputMax)
			{
				CachedCurveEditorData.OutputMin = OutputMin;
				CachedCurveEditorData.OutputMax = OutputMax;
				CurveCacheFlags = ECurveCacheFlags::UpdateAll;
			}
			else if (CachedCurveEditorData.CachedGeometrySize != CurveEditorView->GetCachedGeometry().GetLocalSize())
			{
				CurveCacheFlags = ECurveCacheFlags::UpdateAll;
			}
			else
			{
				double InputMin = 0, InputMax = 1;
				CurveEditorView->GetInputBounds(InputMin, InputMax);
				if (InputMin != CachedCurveEditorData.InputMin || InputMax != CachedCurveEditorData.InputMax)
				{
					CurveCacheFlags = ECurveCacheFlags::UpdateAll;
				}
			}
		}
	}

	void FCurveDrawParamsCache::DrawCurves(const TSharedRef<SCurveEditorView>& CurveEditorView, TConstArrayView<FCurveModelID> ModelIDs)
	{
		if (!WeakCurveEditor.IsValid())
		{
			return;
		}
		const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();

		if (CurveCacheFlags == ECurveCacheFlags::UpdateAll)
		{
			CurveEditorView->GetInputBounds(CachedCurveEditorData.InputMin, CachedCurveEditorData.InputMax);
			CachedCurveEditorData.CachedGeometrySize = CurveEditorView->GetCachedGeometry().GetLocalSize();

			CachedDrawParams.Reset(ModelIDs.Num());

			TArray<FCurveDrawParamsHandle> DrawParamHandles;
			for (const FCurveModelID& ModelID : ModelIDs)
			{
				const int32 DrawParamsIndex = CachedDrawParams.Emplace(ModelID);
				DrawParamHandles.Emplace(AsShared(), DrawParamsIndex);
			}
			UpdateCurveDrawParams(CurveEditorView, DrawParamHandles);

			CurveCacheFlags = ECurveCacheFlags::CheckCurves;

			CurveEditorView->RefreshRetainer();
		}
		else if (CurveCacheFlags == ECurveCacheFlags::CheckCurves)
		{
			TArray<FCurveDrawParamsHandle> ChangedDrawParamHandles;
			for (int32 DrawParamsIndex = 0; DrawParamsIndex < CachedDrawParams.Num(); DrawParamsIndex++)
			{
				const FCurveModelID& CurveModelID = CachedDrawParams[DrawParamsIndex].GetID();

				FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID);
				if (CurveModel && CurveModel->HasChangedAndResetTest())
				{
					ChangedDrawParamHandles.Emplace(AsShared(), DrawParamsIndex);
					if (CurveEditorView->bAllowModelViewTransforms)
					{
						CurveEditorView->bUpdateModelViewTransforms = true;
						CurveEditorView->UpdateCustomAxes();
					}
					
					OnCurveHasChangedExternallyDelegate.Broadcast(CurveModelID);
				}
			}

			if (!ChangedDrawParamHandles.IsEmpty())
			{
				UpdateCurveDrawParams(CurveEditorView, ChangedDrawParamHandles);

				CurveEditorView->RefreshRetainer();
			}
		}
	}
}
