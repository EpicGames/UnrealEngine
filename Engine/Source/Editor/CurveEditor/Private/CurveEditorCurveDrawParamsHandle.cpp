// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorCurveDrawParamsHandle.h"

#include "CurveDrawInfo.h"
#include "CurveEditorCurveDrawParamsCache.h"

namespace UE::CurveEditor
{
	/** Constructs the handle from the index in curve draw params. */
	FCurveDrawParamsHandle::FCurveDrawParamsHandle(const TSharedRef<FCurveDrawParamsCache>& InDrawParamsCache, const int32 InIndex)
		: Index(InIndex)
		, WeakDrawParamsCache(InDrawParamsCache)
	{
		CurveModelID = InDrawParamsCache->CachedDrawParams[InIndex].GetID();
	}

	FCurveDrawParams* FCurveDrawParamsHandle::Get() const
	{
		if (!WeakDrawParamsCache.IsValid())
		{
			return nullptr;
		}
		FCurveDrawParamsCache& Cache = *WeakDrawParamsCache.Pin();

		if (Cache.CachedDrawParams.IsValidIndex(Index) &&
			Cache.CachedDrawParams[Index].GetID() == CurveModelID)
		{
			return &Cache.CachedDrawParams[Index];
		}
		else 
		{
			const TArray<FCurveDrawParams>& CurveDrawParams = Cache.CachedDrawParams;
			Index = WeakDrawParamsCache.Pin()->CachedDrawParams.IndexOfByPredicate(
				[this](const FCurveDrawParams& CurveDrawParams)
				{
					return CurveDrawParams.GetID() == CurveModelID;
				});

			return Index == INDEX_NONE ? nullptr : &Cache.CachedDrawParams[Index];
		}
	}
}
