// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "UObject/GCObject.h"
#include "CurveEditorKeyProxy.h"

struct FCurveEditorEditObjectContainer : FGCObject
{
	FCurveEditorEditObjectContainer() = default;

	FCurveEditorEditObjectContainer(const FCurveEditorEditObjectContainer&) = delete;
	FCurveEditorEditObjectContainer& operator=(const FCurveEditorEditObjectContainer&) = delete;

	FCurveEditorEditObjectContainer(FCurveEditorEditObjectContainer&&) = delete;
	FCurveEditorEditObjectContainer& operator=(FCurveEditorEditObjectContainer&&) = delete;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		for (auto& Pair : CurveIDToKeyProxies)
		{
			Collector.AddReferencedObjects(Pair.Value);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCurveEditorEditObjectContainer");
	}

	/**  */
	TMap<FCurveModelID, TMap<FKeyHandle, TObjectPtr<UObject>> > CurveIDToKeyProxies;
};
