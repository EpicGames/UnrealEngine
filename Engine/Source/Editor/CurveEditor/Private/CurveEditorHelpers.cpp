// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorHelpers.h"

#include "CurveEditorScreenSpace.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathSSE.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"


namespace CurveEditor
{

FVector2D ComputeScreenSpaceTangentOffset(const FCurveEditorScreenSpace& CurveSpace, float Tangent, float Weight)
{
	const float Angle = FMath::Atan(-Tangent);
	FVector2D Offset;
	FMath::SinCos(&Offset.Y, &Offset.X, Angle);
	Offset *= Weight;

	Offset.X *= CurveSpace.PixelsPerInput();
	Offset.Y *= CurveSpace.PixelsPerOutput();
	return Offset;
}

void TangentAndWeightFromOffset(const FCurveEditorScreenSpace& CurveSpace, const FVector2D& TangentOffset, float& OutTangent, float& OutWeight)
{
	double X = CurveSpace.ScreenToSeconds(TangentOffset.X) - CurveSpace.ScreenToSeconds(0);
	double Y = CurveSpace.ScreenToValue(TangentOffset.Y) - CurveSpace.ScreenToValue(0);

	OutTangent = Y / X;
	OutWeight = FMath::Sqrt(X*X + Y*Y);
}

FVector2D GetVectorFromSlopeAndLength(float Slope, float Length)
{
	float x = Length / FMath::Sqrt(Slope*Slope + 1.f);
	float y = Slope * x;
	return FVector2D(x, y);
}

void PopulateGridLineValues(float PhysicalSize, double ViewMin, double ViewMax, uint8 InMinorDivisions, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines)
{
	const double PixelsPerValue = PhysicalSize / FMath::Max(ViewMax - ViewMin, 1e-10);
	const double GridPixelSpacing = PhysicalSize / 5.0;

	const double Order = FMath::Pow(10.0, FMath::FloorToInt(FMath::LogX(10.0, GridPixelSpacing / PixelsPerValue)));

	static const int32 DesirableBases[]  = { 2, 5 };
	static const int32 NumDesirableBases = UE_ARRAY_COUNT(DesirableBases);

	const int32 Scale = FMath::RoundToInt(GridPixelSpacing / PixelsPerValue / Order);
	int32 Base = DesirableBases[0];
	for (int32 BaseIndex = 1; BaseIndex < NumDesirableBases; ++BaseIndex)
	{
		if (FMath::Abs(Scale - DesirableBases[BaseIndex]) < FMath::Abs(Scale - Base))
		{
			Base = DesirableBases[BaseIndex];
		}
	}

	double MajorGridStep = FMath::Pow(static_cast<float>(Base), FMath::FloorToFloat(FMath::LogX(static_cast<float>(Base), static_cast<float>(Scale)))) * Order;

	const double FirstMajorLine = FMath::FloorToDouble(ViewMin / MajorGridStep) * MajorGridStep;
	const double LastMajorLine = FMath::CeilToDouble(ViewMax / MajorGridStep) * MajorGridStep;

	for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine <= LastMajorLine; CurrentMajorLine += MajorGridStep)
	{
		OutMajorGridLines.Add(CurrentMajorLine);
		for (int32 Step = 1; Step < InMinorDivisions; ++Step)
		{
			OutMinorGridLines.Add(CurrentMajorLine + Step * MajorGridStep / InMinorDivisions);
		}
	}
}

void ConstructYGridLines(const FCurveEditorScreenSpace& ViewSpace, uint8 InMinorDivisions, TArray<float>& OutMajorGridLines, TArray<float>& OutMinorGridLines, FText GridLineLabelFormatY, TArray<FText>* OutMajorGridLabels)
{
	const double GridPixelSpacing = ViewSpace.GetPhysicalHeight() / 5.0;

	const double Order = FMath::Pow(10.0, FMath::FloorToInt(FMath::LogX(10.0, GridPixelSpacing / ViewSpace.PixelsPerOutput())));

	static const int32 DesirableBases[]  = { 2, 5 };
	static const int32 NumDesirableBases = UE_ARRAY_COUNT(DesirableBases);

	const int32 Scale = FMath::RoundToInt(GridPixelSpacing / ViewSpace.PixelsPerOutput() / Order);
	int32 Base = DesirableBases[0];
	for (int32 BaseIndex = 1; BaseIndex < NumDesirableBases; ++BaseIndex)
	{
		if (FMath::Abs(Scale - DesirableBases[BaseIndex]) < FMath::Abs(Scale - Base))
		{
			Base = DesirableBases[BaseIndex];
		}
	}

	double MajorGridStep = FMath::Pow(static_cast<float>(Base), FMath::FloorToFloat(FMath::LogX(static_cast<float>(Base), static_cast<float>(Scale)))) * Order;

	const double FirstMajorLine = FMath::FloorToDouble(ViewSpace.GetOutputMin() / MajorGridStep) * MajorGridStep;
	const double LastMajorLine = FMath::CeilToDouble(ViewSpace.GetOutputMax() / MajorGridStep) * MajorGridStep;

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.SetMaximumFractionalDigits(6);

	for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine <= LastMajorLine; CurrentMajorLine += MajorGridStep)
	{
		OutMajorGridLines.Add(ViewSpace.ValueToScreen(CurrentMajorLine));
		if (OutMajorGridLabels)
		{
			OutMajorGridLabels->Add(FText::Format(GridLineLabelFormatY, FText::AsNumber(CurrentMajorLine, &FormattingOptions)));
		}

		for (int32 Step = 1; Step < InMinorDivisions; ++Step)
		{
			OutMinorGridLines.Add(ViewSpace.ValueToScreen(CurrentMajorLine + Step * MajorGridStep / InMinorDivisions));
		}
	}
}

void ConstructFixedYGridLines(const FCurveEditorScreenSpace& ViewSpace, uint8 InMinorDivisions, double InMinorGridStep, TArray<float>& OutMajorGridLines, TArray<float>& OutMinorGridLines, FText GridLineLabelFormatY, 
	TArray<FText>* OutMajorGridLabels, TOptional<double> InOutputMin, TOptional<double> InOutputMax)
{
	const double MajorGridStep = InMinorGridStep * InMinorDivisions;
	const double FirstMinorLine = InOutputMin ? FMath::CeilToDouble(InOutputMin.GetValue() / InMinorGridStep) * InMinorGridStep 
		                                      : FMath::FloorToDouble(ViewSpace.GetOutputMin() / InMinorGridStep) * InMinorGridStep;
	const double LastMinorLine = InOutputMax ? FMath::FloorToDouble(InOutputMax.GetValue() / InMinorGridStep) * InMinorGridStep 
		                                     : FMath::CeilToDouble(ViewSpace.GetOutputMax() / InMinorGridStep) * InMinorGridStep;

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.SetMaximumFractionalDigits(6);

	// calculate min. distance between labels
	const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	uint16 FontHeight = FontMeasureService->GetMaxCharacterHeight(FontInfo);

	const double LabelDist = (1 / ViewSpace.PixelsPerOutput()) * (FontHeight + 3.0); // 3.0 for margin
	double LineSkip = FMath::CeilToDouble(LabelDist / MajorGridStep) * MajorGridStep;
	LineSkip = FMath::IsNearlyZero(LineSkip) ? KINDA_SMALL_NUMBER : LineSkip; // prevent mod by zero errors

	for (double CurrentMinorLine = FirstMinorLine; CurrentMinorLine <= LastMinorLine; CurrentMinorLine += InMinorGridStep)
	{
		// check if is major grid line
		if (FMath::IsNearlyZero(FMath::Fmod(FMath::Abs(CurrentMinorLine), MajorGridStep)))
		{
			OutMajorGridLines.Add(ViewSpace.ValueToScreen(CurrentMinorLine));
			if (OutMajorGridLabels)
			{
				OutMajorGridLabels->Add(FMath::IsNearlyZero(FMath::Fmod(FMath::Abs(CurrentMinorLine), LineSkip))
					? FText::Format(GridLineLabelFormatY, FText::AsNumber(CurrentMinorLine, &FormattingOptions)) 
					: FText());
			}
		}
		else
		{
			OutMinorGridLines.Add(ViewSpace.ValueToScreen(CurrentMinorLine));
		}
	}
}

} // namespace CurveEditor
