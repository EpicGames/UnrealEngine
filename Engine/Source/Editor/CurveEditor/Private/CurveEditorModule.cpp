// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "CurveEditorCommands.h"
#include "CurveEditorTypes.h"
#include "CurveEditorViewRegistry.h"
#include "Delegates/Delegate.h"
#include "Filters/CurveEditorBakeFilter.h"
#include "Filters/CurveEditorBakeFilterCustomization.h"
#include "Filters/CurveEditorEulerFilter.h"
#include "Filters/PromotedFilterContainer.h"
#include "HAL/PlatformCrt.h"
#include "ICurveEditorModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/InputState.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"

class FCurveEditorModule : public ICurveEditorModule
{
public:
	
	virtual void StartupModule() override
	{
		if (GIsEditor)
		{
			if (UToolMenus::TryGet())
			{
				RegisterCommands();
			}
			else
			{
				FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCurveEditorModule::RegisterCommands);
			}
		}

		RegisterCustomizations();
	}

	virtual void ShutdownModule() override
	{
		UE::CurveEditor::InputState::Shutdown();

		FCurveEditorCommands::Unregister();

		UnregisterCustomizations();
		ToolbarPromotedFilters.Reset();
	}

	virtual FDelegateHandle RegisterEditorExtension(FOnCreateCurveEditorExtension InOnCreateCurveEditorExtension) override
	{
		EditorExtensionDelegates.Add(InOnCreateCurveEditorExtension);
		FDelegateHandle Handle = EditorExtensionDelegates.Last().GetHandle();
		
		return Handle;
	}

	virtual void UnregisterEditorExtension(FDelegateHandle InHandle) override
	{
		EditorExtensionDelegates.RemoveAll([=](const FOnCreateCurveEditorExtension& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual FDelegateHandle RegisterToolExtension(FOnCreateCurveEditorToolExtension InOnCreateCurveEditorToolExtension) override
	{
		ToolExtensionDelegates.Add(InOnCreateCurveEditorToolExtension);
		FDelegateHandle Handle = ToolExtensionDelegates.Last().GetHandle();

		return Handle;
	}

	virtual void UnregisterToolExtension(FDelegateHandle InHandle) override
	{
		ToolExtensionDelegates.RemoveAll([=](const FOnCreateCurveEditorToolExtension& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual ECurveEditorViewID RegisterView(FOnCreateCurveEditorView InCreateViewDelegate) override
	{
		return FCurveEditorViewRegistry::Get().RegisterCustomView(InCreateViewDelegate);
	}

	virtual void UnregisterView(ECurveEditorViewID InViewID) override
	{
		return FCurveEditorViewRegistry::Get().UnregisterCustomView(InViewID);
	}

	virtual TArray<FCurveEditorMenuExtender>& GetAllToolBarMenuExtenders() override
	{
		return ToolBarMenuExtenders;
	}

	virtual TArrayView<const FOnCreateCurveEditorExtension> GetEditorExtensions() const override
	{
		return EditorExtensionDelegates;
	}

	virtual TArrayView<const FOnCreateCurveEditorToolExtension> GetToolExtensions() const override
	{
		return ToolExtensionDelegates;
	}

	virtual TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> GetGlobalToolbarPromotedFilters() const override { return ToolbarPromotedFilters; }

	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout(UCurveEditorBakeFilter::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FCurveEditorBakeFilterCustomization::MakeInstance));
		PropertyEditorModule.NotifyCustomizationModuleChanged();
	}

	void UnregisterCustomizations()
	{
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
			{
				PropertyEditorModule->UnregisterCustomClassLayout(UCurveEditorBakeFilter::StaticClass()->GetFName());
				PropertyEditorModule->NotifyCustomizationModuleChanged();
			}
		}
	}

private:
	
	/** List of editor extension handler delegates Curve Editors will execute when they are created. */
	TArray<FOnCreateCurveEditorExtension> EditorExtensionDelegates;

	/** List of tool extension handler delegates Curve Editors will execute when they are created. */
	TArray<FOnCreateCurveEditorToolExtension> ToolExtensionDelegates;

	/** List of Extenders that that should be called when building the Curve Editor toolbar. */
	TArray<FCurveEditorMenuExtender> ToolBarMenuExtenders;

	/**
	 * Keeps track of the filters that are promoted to the toolbar.
	 * 
	 * This object dynamically creates FUICommandInfos based on the filters surfaced so it must have a globally unique context name for its
	 * FBindingContext. FCurveEditor instances will reference this by default (but can theoretically create their own).
	 *
	 * Until UE-230269 is implemented, the only filter that is surfaced to this object is the Euler filter. 
	 */
	TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> ToolbarPromotedFilters;

	void RegisterCommands()
	{
		FCurveEditorCommands::Register();

		// This needs to be created after registering the commands because FCurveEditorCommands is used a parent context
		ToolbarPromotedFilters = MakeShared<UE::CurveEditor::FPromotedFilterContainer>(TEXT("ToolbarPromotedCurveEditorFilters"));
		// By default, we surface the Euler filter to the toolbar because it is a common action.
		ToolbarPromotedFilters->AddInstance(*GetMutableDefault<UCurveEditorEulerFilter>());

		UE::CurveEditor::InputState::Initialize();
	}
};

IMPLEMENT_MODULE(FCurveEditorModule, CurveEditor)