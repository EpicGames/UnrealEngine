// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorSelection.h"

#include "Algo/BinarySearch.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Algo/AllOf.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"

namespace UE::CurveEditorSelection
{

bool AreKeyHandleSetsEqual(const FKeyHandleSet& InA, const FKeyHandleSet& InB)
{
	const TArrayView<const FKeyHandle> AHandles = InA.AsArray();
	const TArrayView<const FKeyHandle> BHandles = InB.AsArray();
	if (AHandles.Num() != BHandles.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < AHandles.Num(); ++Index)
	{
		if (AHandles[Index] != BHandles[Index] || InA.PointType(AHandles[Index]) != InB.PointType(BHandles[Index]))
		{
			return false;
		}
	}

	return true;
}

bool AreSelectionsEqual(const TMap<FCurveModelID, FKeyHandleSet>& InA, const TMap<FCurveModelID, FKeyHandleSet>& InB)
{
	if (InA.Num() != InB.Num())
	{
		return false;
	}

	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InA)
	{
		const FKeyHandleSet* const OtherKeys = InB.Find(Pair.Key);
		if (!OtherKeys || !AreKeyHandleSetsEqual(Pair.Value, *OtherKeys))
		{
			return false;
		}
	}

	return true;
}

} // namespace UE::CurveEditorSelection

FScopedSelectionChangedEventSuppressor::FScopedSelectionChangedEventSuppressor(FCurveEditorSelection& InSelection)
	: Selection(InSelection)
{
	Selection.IncrementOnSelectionChangedSuppressionCount();
}

FScopedSelectionChangedEventSuppressor::~FScopedSelectionChangedEventSuppressor()
{
	Selection.DecrementOnSelectionChangedSuppressionCount();
}

FCurveEditorSelection::FCurveEditorSelection()
{
	SerialNumber = 0;
}

FCurveEditorSelection::FCurveEditorSelection(TWeakPtr<FCurveEditor> InWeakCurveEditor)
{
	WeakCurveEditor = InWeakCurveEditor;
	SerialNumber = 0;
}

void FCurveEditorSelection::Clear()
{
	const bool bIsAlreadyEmpty = CurveToSelectedKeys.IsEmpty()
		|| Algo::AllOf(CurveToSelectedKeys, [](const TPair<FCurveModelID, FKeyHandleSet>& Pair){ return Pair.Value.Num() == 0; }); 
	if (!bIsAlreadyEmpty)
	{
		CurveToSelectedKeys.Reset();
		PostChangesMade();
	}
}

void FCurveEditorSelection::SerializeSelection(FArchive& Ar)
{
	Ar << SerialNumber;
	Ar << CurveToSelectedKeys;
}

void FCurveEditorSelection::AddInternal(
	FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys, bool bIncrementSerialNumber
	)
{
	FKeyHandleSet& SelectedKeys = CurveToSelectedKeys.FindOrAdd(CurveID);
	for (const FKeyHandle& Key : Keys)
	{
		SelectedKeys.Add(Key, PointType);
	}

	/**
	 * Honor bIncrementSerialNumber.
	 * AddInternal is used by paths that may batch many selection mutations together.
	 * This parameter already exists to let callers opt out of per-call PostChangesMade(),
	 * but this function previously ignored it and always broadcasted anyway.
	 * Guarding PostChangesMade() here restores the intended behavior: callers can defer
	 * serial-number bumps / selection-changed broadcasts until the full batch is complete.
	 */
	if (bIncrementSerialNumber)
	{
		PostChangesMade();
	}
}

void FCurveEditorSelection::PostChangesMade()
{
	++SerialNumber;
	OnSelectionChangedBroadcaster.Broadcast();
}

const FKeyHandleSet* FCurveEditorSelection::FindForCurve(FCurveModelID InCurveID) const
{
	return CurveToSelectedKeys.Find(InCurveID);
}

int32 FCurveEditorSelection::Count() const
{
	int32 Num = 0;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveToSelectedKeys)
	{
		Num += Pair.Value.Num();
	}
	return Num;
}

bool FCurveEditorSelection::IsSelected(FCurvePointHandle InHandle) const
{
	return Contains(InHandle.CurveID, InHandle.KeyHandle, InHandle.PointType);
}

bool FCurveEditorSelection::Contains(FCurveModelID CurveID, FKeyHandle KeyHandle, ECurvePointType PointType) const
{
	const FKeyHandleSet* SelectedKeys = CurveToSelectedKeys.Find(CurveID);
	return SelectedKeys && SelectedKeys->Contains(KeyHandle, PointType);
}

void FCurveEditorSelection::IncrementOnSelectionChangedSuppressionCount()
{
	OnSelectionChangedBroadcaster.IncrementSuppressionCount();
}

void FCurveEditorSelection::DecrementOnSelectionChangedSuppressionCount()
{
	OnSelectionChangedBroadcaster.DecrementSuppressionCount();
}

FSimpleMulticastDelegate& FCurveEditorSelection::OnSelectionChanged()
{
	return OnSelectionChangedBroadcaster.OnChanged();
}

void FCurveEditorSelection::Add(FCurvePointHandle InHandle)
{
	Add(InHandle.CurveID, InHandle.PointType, InHandle.KeyHandle);
}

void FCurveEditorSelection::Add(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle)
{
	Add(CurveID, PointType, TArrayView<const FKeyHandle>(&KeyHandle, 1));
}

void FCurveEditorSelection::Add(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (Keys.Num() > 0 && CurveEditor)
	{
		const TUniquePtr<FCurveModel> *CurveModel = CurveEditor->GetCurves().Find(CurveID);
		if (CurveModel && CurveModel->IsValid() && !(*CurveModel)->IsReadOnly())
		{
			constexpr bool bIncrementSerialNumber = true;
			AddInternal(CurveID, PointType, Keys, bIncrementSerialNumber);
		}
	}
}

void FCurveEditorSelection::Replace(const TMap<FCurveModelID, FKeyHandleSet>& InCurveToSelectedKeys)
{
	const bool bIsAlreadyEmpty = CurveToSelectedKeys.IsEmpty()
		|| Algo::AllOf(CurveToSelectedKeys, [](const TPair<FCurveModelID, FKeyHandleSet>& InPair)
			{
				return InPair.Value.Num() == 0;
			});

	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		if (!bIsAlreadyEmpty)
		{
			Clear();
		}
		return;
	}

	TMap<FCurveModelID, FKeyHandleSet> EffectiveCurveToSelectedKeys;
	EffectiveCurveToSelectedKeys.Reserve(InCurveToSelectedKeys.Num());

	if (InCurveToSelectedKeys.Num() > 0)
	{
		for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InCurveToSelectedKeys)
		{
			if (Pair.Value.Num() == 0)
			{
				continue;
			}

			const TUniquePtr<FCurveModel>* CurveModel = CurveEditor->GetCurves().Find(Pair.Key);
			if (!CurveModel || !CurveModel->IsValid() || (*CurveModel)->IsReadOnly())
			{
				continue;
			}

			EffectiveCurveToSelectedKeys.Add(Pair.Key, Pair.Value);
		}
	}

	if (UE::CurveEditorSelection::AreSelectionsEqual(CurveToSelectedKeys, EffectiveCurveToSelectedKeys))
	{
		return;
	}

	FScopedSelectionChangedEventSuppressor SuppressEvents(*this);
	CurveToSelectedKeys = MoveTemp(EffectiveCurveToSelectedKeys);
	PostChangesMade();
}

void FCurveEditorSelection::Replace(const TMap<FCurveModelID, TArray<FKeyHandle>>& InCurveToSelectedKeys, const ECurvePointType InPointType)
{
	TMap<FCurveModelID, FKeyHandleSet> ConvertedCurveToSelectedKeys;
	ConvertedCurveToSelectedKeys.Reserve(InCurveToSelectedKeys.Num());

	for (const TPair<FCurveModelID, TArray<FKeyHandle>>& Pair : InCurveToSelectedKeys)
	{
		if (Pair.Value.IsEmpty())
		{
			continue;
		}

		FKeyHandleSet& SelectedKeys = ConvertedCurveToSelectedKeys.Add(Pair.Key);
		SelectedKeys.Replace(Pair.Value, InPointType);
	}

	Replace(ConvertedCurveToSelectedKeys);
}

void FCurveEditorSelection::Toggle(FCurvePointHandle InHandle)
{
	Toggle(InHandle.CurveID, InHandle.PointType, InHandle.KeyHandle);
}

void FCurveEditorSelection::Toggle(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle)
{
	Toggle(CurveID, PointType, TArrayView<const FKeyHandle>(&KeyHandle, 1));
}

void FCurveEditorSelection::Toggle(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys)
{
	if (Keys.Num() > 0)
	{
		FKeyHandleSet& SelectedKeys = CurveToSelectedKeys.FindOrAdd(CurveID);
		for (FKeyHandle Key : Keys)
		{
			SelectedKeys.Toggle(Key, PointType);
		}

		if (SelectedKeys.Num() == 0)
		{
			CurveToSelectedKeys.Remove(CurveID);
		}
		
		PostChangesMade();
	}
}

void FCurveEditorSelection::Remove(FCurvePointHandle InHandle)
{
	Remove(InHandle.CurveID, InHandle.PointType, InHandle.KeyHandle);
}

void FCurveEditorSelection::Remove(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle)
{
	Remove(CurveID, PointType, TArrayView<const FKeyHandle>(&KeyHandle, 1));
}

void FCurveEditorSelection::Remove(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys)
{
	if (Keys.Num() > 0)
	{
		FKeyHandleSet& SelectedKeys = CurveToSelectedKeys.FindOrAdd(CurveID);
		for (FKeyHandle Key : Keys)
		{
			SelectedKeys.Remove(Key, PointType);
		}
		
		PostChangesMade();
	}
}

void FCurveEditorSelection::Remove(FCurveModelID InCurveID)
{
	CurveToSelectedKeys.Remove(InCurveID);
	PostChangesMade();
}

void FKeyHandleSet::Add(FKeyHandle Handle, ECurvePointType PointType)
{
	int32 ExistingIndex = Algo::LowerBound(SortedHandles, Handle);
	if (ExistingIndex >= SortedHandles.Num() || SortedHandles[ExistingIndex] != Handle)
	{
		SortedHandles.Insert(Handle, ExistingIndex);
	}

	EnumAddFlags(HandleToPointType.FindOrAdd(Handle), PointType);
}

void FKeyHandleSet::Toggle(FKeyHandle Handle, ECurvePointType PointType)
{
	int32 ExistingIndex = Algo::LowerBound(SortedHandles, Handle);
	if (ExistingIndex < SortedHandles.Num() && SortedHandles[ExistingIndex] == Handle)
	{
		if (HandleToPointType.Contains(Handle))
		{
			EnumRemoveFlags(HandleToPointType.FindChecked(Handle), PointType);

			if (HandleToPointType.FindChecked(Handle) == ECurvePointType::None)
			{
				HandleToPointType.FindAndRemoveChecked(Handle);		
				SortedHandles.RemoveAt(ExistingIndex, EAllowShrinking::No);
			}
		}
	}
	else
	{
		SortedHandles.Insert(Handle, ExistingIndex);
			
		EnumAddFlags(HandleToPointType.FindOrAdd(Handle), PointType);
	}
}

void FKeyHandleSet::Remove(FKeyHandle Handle, ECurvePointType PointType)
{
	int32 ExistingIndex = Algo::LowerBound(SortedHandles, Handle);
	if (ExistingIndex < SortedHandles.Num() && SortedHandles[ExistingIndex] == Handle)
	{
		if (HandleToPointType.Contains(Handle))
		{
			EnumRemoveFlags(HandleToPointType.FindChecked(Handle), PointType);

			if (HandleToPointType.FindChecked(Handle) == ECurvePointType::None)
			{
				HandleToPointType.FindAndRemoveChecked(Handle);		
				SortedHandles.RemoveAt(ExistingIndex, EAllowShrinking::No);
			}
		}
	}
}

void FKeyHandleSet::Replace(const TArrayView<const FKeyHandle>& InKeys, const ECurvePointType InPointType)
{
	SortedHandles.Reset(InKeys.Num());
	HandleToPointType.Empty(InKeys.Num());

	for (const FKeyHandle Key : InKeys)
	{
		HandleToPointType.FindOrAdd(Key, InPointType);
	}

	SortedHandles.Reserve(HandleToPointType.Num());
	for (const TPair<FKeyHandle, ECurvePointType>& Pair : HandleToPointType)
	{
		SortedHandles.Add(Pair.Key);
	}

	SortedHandles.Sort();
}

bool FKeyHandleSet::Contains(FKeyHandle Handle, ECurvePointType PointType) const
{
	if (Algo::BinarySearch(SortedHandles, Handle) != INDEX_NONE && HandleToPointType.Contains(Handle))
	{
		return EnumHasAnyFlags(HandleToPointType.FindChecked(Handle), PointType);
	}
	
	return false;
}
