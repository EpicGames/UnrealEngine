// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSelection.h"
#include "Modification/Selection/SelectionChangeUtils.h"

namespace UE::CurveEditor { class FScopedSelectionChangeEventSuppression; }

namespace UE::CurveEditor::KeySelection 
{
/** Implements passkey pattern to restrict access to FCurveEditorSelection::AddInternal. */
class FAddInternal
{
	friend void SelectionDiffDetail::AddKeysInternal(FCurveEditorSelection&, FCurveModelID, ECurvePointType, TArrayView<const FKeyHandle>);
	
	explicit FAddInternal(
		FCurveEditorSelection& Selection, FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys,
		bool bIncrementSerialNumber = true
		)
	{
		Selection.AddInternal(CurveID, PointType, Keys, bIncrementSerialNumber);
	}
};

/** Implements passkey pattern to restrict access to FCurveEditorSelection::SerialNumber. */
class FSetSerialNumber
{
	friend void SelectionDiffDetail::SetSerialNumberInternal(FCurveEditorSelection&, uint32);
	
	explicit FSetSerialNumber(
		FCurveEditorSelection& Selection, int32 NewSerialNumber
		)
	{
		Selection.SerialNumber = NewSerialNumber;
	}
};

/** Implements passkey pattern to restrict access to FCurveEditorSelection::IncrementOnSelectionChangedSuppressionCount. */
class FIncrementOnSelectionChangedSuppressionCount
{
	friend FScopedSelectionChangeEventSuppression;

	explicit FIncrementOnSelectionChangedSuppressionCount(
		FCurveEditorSelection& Selection
		)
	{
		Selection.IncrementOnSelectionChangedSuppressionCount();
	}
};

/** Implements passkey pattern to restrict access to FCurveEditorSelection::DecrementOnSelectionChangedSuppressionCount. */
class FDecrementOnSelectionChangedSuppressionCount
{
	friend FScopedSelectionChangeEventSuppression;

	explicit FDecrementOnSelectionChangedSuppressionCount(
		FCurveEditorSelection& Selection
		)
	{
		Selection.DecrementOnSelectionChangedSuppressionCount();
	}
};
}
