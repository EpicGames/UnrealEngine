// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorSettings)

UCurveEditorSettings::UCurveEditorSettings()
{
	PanningMouseButton = ECurveEditorPanningMouseButton::RightOrAltMiddle;
	bAutoFrameCurveEditor = true;
	bShowTransportControls = true;
	bShowBars = true;
	FrameInputPadding = 50;
	FrameOutputPadding = 50;
	bShowBufferedCurves = true;
	bShowCurveEditorCurveToolTips = true;
	TangentVisibility = ECurveEditorTangentVisibility::SelectedKeys;
	ZoomPosition = ECurveEditorZoomPosition::CurrentTime;
	SnapAxis = ECurveEditorSnapAxis::CESA_None;
	bSnapTimeToSelection = false;
	bScrubTimeStartFromCursor = true;
	bCubicAddKeyPrefersUserTangent = false;
	bMiddleMouseButtonAddsKey = true;

	SelectionColor = FLinearColor::White;
	ParentSpaceCustomColor = FLinearColor(.93f, .31f, .19f); //pastel orange
	WorldSpaceCustomColor = FLinearColor(.198f, .610f, .558f); //pastel teal

	TreeViewWidth = 0.3f;
}

bool UCurveEditorSettings::AllowMousePan(const FPointerEvent& MouseEvent) const
{
	if (PanningMouseButton == ECurveEditorPanningMouseButton::AltMiddle)
	{
		return MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && MouseEvent.IsAltDown();
	}
	else if (PanningMouseButton == ECurveEditorPanningMouseButton::Right)
	{
		return MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	}
	else if (PanningMouseButton == ECurveEditorPanningMouseButton::RightOrAltMiddle)
	{
		if ((MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && MouseEvent.IsAltDown()) ||
			(MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton)))
		{
			return true;
		}
		return false;
	}

	// Default to right mouse button
	return MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
}

bool UCurveEditorSettings::AllowMouseEdit(const FPointerEvent& MouseEvent) const
{
	// If panning with alt-middle, middle allows mouse edits
	if (PanningMouseButton == ECurveEditorPanningMouseButton::AltMiddle)
	{
		return MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && !MouseEvent.IsAltDown();
	}
	// If panning with right, middle allows mouse edits
	else if (PanningMouseButton == ECurveEditorPanningMouseButton::Right)
	{
		return MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
	}
	else if (PanningMouseButton == ECurveEditorPanningMouseButton::RightOrAltMiddle)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && !MouseEvent.IsAltDown())
		{
			return true;
		}
		return false;
	}

	// Default to middle mouse button
	return MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
}

bool UCurveEditorSettings::GetAutoFrameCurveEditor() const
{
	return bAutoFrameCurveEditor;
}

void UCurveEditorSettings::SetAutoFrameCurveEditor(bool InbAutoFrameCurveEditor)
{
	if (bAutoFrameCurveEditor != InbAutoFrameCurveEditor)
	{
		bAutoFrameCurveEditor = InbAutoFrameCurveEditor;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetShowTransportControls() const
{
	return bShowTransportControls;
}

void UCurveEditorSettings::SetShowTransportControls(const bool bInShow)
{
	if (bShowTransportControls != bInShow)
	{
		bShowTransportControls = bInShow;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetShowBars() const
{
	return bShowBars;
}

void UCurveEditorSettings::SetShowBars(bool InValue)
{
	if (bShowBars != InValue)
	{
		bShowBars = InValue;
		SaveConfig();
	}
}

int32 UCurveEditorSettings::GetFrameInputPadding() const
{
	return FrameInputPadding;
}

void UCurveEditorSettings::SetFrameInputPadding(int32 InFrameInputPadding)
{
	if (FrameInputPadding != InFrameInputPadding)
	{
		FrameInputPadding = InFrameInputPadding;
		SaveConfig();
	}
}

int32 UCurveEditorSettings::GetFrameOutputPadding() const
{
	return FrameOutputPadding;
}

void UCurveEditorSettings::SetFrameOutputPadding(int32 InFrameOutputPadding)
{
	if (FrameOutputPadding != InFrameOutputPadding)
	{
		FrameOutputPadding = InFrameOutputPadding;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetShowBufferedCurves() const
{
	return bShowBufferedCurves;
}

void UCurveEditorSettings::SetShowBufferedCurves(bool InbShowBufferedCurves)
{
	if (bShowBufferedCurves != InbShowBufferedCurves)
	{
		bShowBufferedCurves = InbShowBufferedCurves;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetCubicAddKeyPrefersUserTangent() const
{
	return bCubicAddKeyPrefersUserTangent;
}

void UCurveEditorSettings::SetCubicAddKeyPrefersUserTangent(bool bInCubicAddKeyPrefersUserTangent)
{
	if (bCubicAddKeyPrefersUserTangent != bInCubicAddKeyPrefersUserTangent)
	{
		bCubicAddKeyPrefersUserTangent = bInCubicAddKeyPrefersUserTangent;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetShowCurveEditorCurveToolTips() const
{
	return bShowCurveEditorCurveToolTips;
}

void UCurveEditorSettings::SetShowCurveEditorCurveToolTips(bool InbShowCurveEditorCurveToolTips)
{
	if (bShowCurveEditorCurveToolTips != InbShowCurveEditorCurveToolTips)
	{
		bShowCurveEditorCurveToolTips = InbShowCurveEditorCurveToolTips;
		SaveConfig();
	}
}

ECurveEditorTangentVisibility UCurveEditorSettings::GetTangentVisibility() const
{
	return TangentVisibility;
}

void UCurveEditorSettings::SetTangentVisibility(ECurveEditorTangentVisibility InTangentVisibility)
{
	if (TangentVisibility != InTangentVisibility)
	{
		TangentVisibility = InTangentVisibility;
		SaveConfig();
	}
}

ECurveEditorZoomPosition UCurveEditorSettings::GetZoomPosition() const
{
	return ZoomPosition;
}

void UCurveEditorSettings::SetZoomPosition(ECurveEditorZoomPosition InZoomPosition)
{
	if (ZoomPosition != InZoomPosition)
	{
		ZoomPosition = InZoomPosition;
		SaveConfig();
	}
}

ECurveEditorSnapAxis UCurveEditorSettings::GetSnapAxis() const
{
	return SnapAxis;
}

void UCurveEditorSettings::SetSnapAxis(ECurveEditorSnapAxis InSnapAxis)
{
	if (SnapAxis != InSnapAxis)
	{
		SnapAxis = InSnapAxis;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetSnapTimeToSelection() const
{
	return bSnapTimeToSelection;
}

void UCurveEditorSettings::SetSnapTimeToSelection(bool bInSnapTimeToSelection)
{
	if (bSnapTimeToSelection != bInSnapTimeToSelection)
	{
		bSnapTimeToSelection = bInSnapTimeToSelection;
		SaveConfig();
	}
}

FLinearColor UCurveEditorSettings::GetSelectionColor() const
{
	return SelectionColor;
}

void UCurveEditorSettings::SetSelectionColor(const FLinearColor& InColor)
{
	if (SelectionColor != InColor)
	{
		SelectionColor = InColor;
		SaveConfig();
	}
}

TOptional<FLinearColor> UCurveEditorSettings::GetCustomColor(UClass* InClass, const FString& InPropertyName) const
{
	TOptional<FLinearColor> Color;
	for (const FCustomColorForChannel& CustomColor : CustomColors)
	{
		UClass* Class = CustomColor.Object.LoadSynchronous();
		if (Class == InClass && CustomColor.PropertyName == InPropertyName)
		{
			Color = CustomColor.Color;
			break;
		}
	}
	return Color;
}
void UCurveEditorSettings::SetCustomColor(UClass* InClass, const FString& InPropertyName, FLinearColor InColor)
{
	TOptional<FLinearColor> Color;
	for (FCustomColorForChannel& CustomColor : CustomColors)
	{
		UClass* Class = CustomColor.Object.LoadSynchronous();
		if (Class == InClass && CustomColor.PropertyName == InPropertyName)
		{
			CustomColor.Color = InColor;
			SaveConfig(); 
			return;
		}
	}
	FCustomColorForChannel NewColor;
	NewColor.Object = InClass;
	NewColor.PropertyName = InPropertyName;
	NewColor.Color = InColor;
	CustomColors.Add(NewColor);
	SaveConfig();
}

void UCurveEditorSettings::DeleteCustomColor(UClass* InClass, const FString& InPropertyName)
{
	TOptional<FLinearColor> Color;
	for (int32 Index = 0;Index < CustomColors.Num(); ++Index)
	{
		FCustomColorForChannel& CustomColor = CustomColors[Index];
		UClass* Class = CustomColor.Object.LoadSynchronous();
		if (Class == InClass && CustomColor.PropertyName == InPropertyName)
		{
			CustomColors.RemoveAt(Index);
			SaveConfig();
			return;
		}
	}
}

TOptional<FLinearColor> UCurveEditorSettings::GetSpaceSwitchColor(const FString& InControlName) const
{
	TOptional<FLinearColor> Color;

	if (InControlName == FString(TEXT("Parent")))
	{
		Color =  ParentSpaceCustomColor;
	}
	else if(InControlName == FString(TEXT("World")))
	{
		Color = WorldSpaceCustomColor;
	}
	else
	{
		for (const FCustomColorForSpaceSwitch& CustomColor : ControlSpaceCustomColors)
		{
			if (CustomColor.ControlName == InControlName)
			{
				Color = CustomColor.Color;
				break;
			}
		}
	}
	return Color;
}

void UCurveEditorSettings::SetSpaceSwitchColor(const FString& InControlName, FLinearColor InColor)
{
	if (InControlName == FString(TEXT("Parent")))
	{
		ParentSpaceCustomColor = InColor;
		SaveConfig();
	}
	else if (InControlName == FString(TEXT("World")))
	{
		WorldSpaceCustomColor = InColor;
		SaveConfig();
	}
	else
	{
		TOptional<FLinearColor> Color;
		for (FCustomColorForSpaceSwitch& CustomColor : ControlSpaceCustomColors)
		{
			if (CustomColor.ControlName == InControlName)
			{
				CustomColor.Color = InColor;
				SaveConfig();
				return;
			}
		}
		FCustomColorForSpaceSwitch NewColor;
		NewColor.ControlName = InControlName;
		NewColor.Color = InColor;
		ControlSpaceCustomColors.Add(NewColor);
		SaveConfig();
	}
}

void UCurveEditorSettings::DeleteSpaceSwitchColor(const FString& InControlName)
{
	TOptional<FLinearColor> Color;
	for (int32 Index = 0; Index < ControlSpaceCustomColors.Num(); ++Index)
	{
		FCustomColorForSpaceSwitch& CustomColor = ControlSpaceCustomColors[Index];
		if (CustomColor.ControlName == InControlName)
		{
			ControlSpaceCustomColors.RemoveAt(Index);
			SaveConfig();
			return;
		}
	}
}

bool UCurveEditorSettings::GetScrubTimeStartFromCursor() const
{
	return bScrubTimeStartFromCursor;
}

void UCurveEditorSettings::SetScrubTimeStartFromCursor(bool bInValue)
{
	if (bScrubTimeStartFromCursor != bInValue)
	{
		bScrubTimeStartFromCursor = bInValue;
		SaveConfig();
	}
}

bool UCurveEditorSettings::GetMiddleMouseButtonAddsKey() const
{
	return bMiddleMouseButtonAddsKey;
}

void UCurveEditorSettings::SetMiddleMouseButtonAddsKey(bool bInValue)
{
	if (bMiddleMouseButtonAddsKey != bInValue)
	{
		bMiddleMouseButtonAddsKey = bInValue;
		SaveConfig();
	}
}

FLinearColor UCurveEditorSettings::GetNextRandomColor()
{
	static TArray<FLinearColor> IndexedColor;
	static int32 NextIndex = 0;
	if (IndexedColor.Num() == 0)
	{
		IndexedColor.Add(FLinearColor(.904f, .323f, .539f)); //pastel red
		IndexedColor.Add(FLinearColor(.552f, .737f, .328f)); //pastel green
		IndexedColor.Add(FLinearColor(.947f, .418f, .219f)); //pastel orange
		IndexedColor.Add(FLinearColor(.156f, .624f, .921f)); //pastel blue
		IndexedColor.Add(FLinearColor(.921f, .314f, .337f)); //pastel red 2
		IndexedColor.Add(FLinearColor(.361f, .651f, .332f)); //pastel green 2
		IndexedColor.Add(FLinearColor(.982f, .565f, .254f)); //pastel orange 2
		IndexedColor.Add(FLinearColor(.246f, .223f, .514f)); //pastel purple
		IndexedColor.Add(FLinearColor(.208f, .386f, .687f)); //pastel blue2
		IndexedColor.Add(FLinearColor(.223f, .590f, .337f)); //pastel green 3
		IndexedColor.Add(FLinearColor(.230f, .291f, .591f)); //pastel blue 3
	}
	FLinearColor Color = IndexedColor[NextIndex];
	++NextIndex;
	int32 NewIndex = (NextIndex % IndexedColor.Num());
	NextIndex = NewIndex;
	return Color;
}

void UCurveEditorSettings::SetTreeViewWidth(const float InTreeViewWidth)
{
	if (InTreeViewWidth != TreeViewWidth)
	{
		TreeViewWidth = InTreeViewWidth;
		SaveConfig();
	}
}

void UCurveEditorSettings::SetMarqueePointSensitivity(const float InMarqueePointSensitivity)
{
	if (InMarqueePointSensitivity != MarqueePointSensitivity)
	{
		MarqueePointSensitivity = InMarqueePointSensitivity;
		SaveConfig();
	}
}

void UCurveEditorSettings::SetShowValueIndicators(bool bValue)
{
	if (bValue != bShowValueIndicators)
	{
		bShowValueIndicators = bValue;
		OnShowValueIndicatorsChangedEvent.Broadcast();
		SaveConfig();
	}
}

void UCurveEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCurveEditorSettings, CustomColors) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UCurveEditorSettings, CustomColors))
	{
		OnCustomColorsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCurveEditorSettings, SnapAxis) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UCurveEditorSettings, SnapAxis))
	{
		OnAxisSnappingChangedEvent.Broadcast();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
