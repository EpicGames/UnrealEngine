// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorTrace.h"

#include "Trace/Detail/Channel.h"

UE_TRACE_CHANNEL_DEFINE(CurveEditorChannel, "Curve Editor-specific CPU profiling events. Tracks performance of curve editing operations including key modifications, curve snapshots, diff calculations, and selection operations.");
