// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(CurveEditorChannel);
#define SCOPED_CURVE_EDITOR_TRACE(TraceName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TraceName, CurveEditorChannel)