// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorViewRegistry.h"

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Views/SCurveEditorViewAbsolute.h"
#include "Views/SCurveEditorViewNormalized.h"
#include "Views/SCurveEditorViewStacked.h"
#include "Views/SInteractiveCurveEditorView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FCurveEditor;
class SCurveEditorView;

FCurveEditorViewRegistry::FCurveEditorViewRegistry()
{
	NextViewID = ECurveEditorViewID::CUSTOM_START;
}

FCurveEditorViewRegistry& FCurveEditorViewRegistry::Get()
{
	static FCurveEditorViewRegistry Singleton;
	return Singleton;
}

ECurveEditorViewID FCurveEditorViewRegistry::RegisterCustomView(const FOnCreateCurveEditorView& InCreateViewDelegate)
{
	ensureMsgf(NextViewID != ECurveEditorViewID::Invalid, TEXT("Maximum limit for registered curve editor views (64) reached."));
	if (NextViewID == ECurveEditorViewID::Invalid)
	{
		return NextViewID;
	}

	CustomViews.Add(NextViewID, InCreateViewDelegate);

	ECurveEditorViewID ThisViewID = NextViewID;

	// When the custom view ID reaches 0x80000000 the left shift will result in well-defined unsigned integer wraparound, resulting in 0 (Invalid)
	NextViewID = ECurveEditorViewID( ((__underlying_type(ECurveEditorViewID))NextViewID) << 1 );

	return ThisViewID;
}

void FCurveEditorViewRegistry::UnregisterCustomView(ECurveEditorViewID ViewID)
{
	CustomViews.Remove(ViewID);
}

TSharedPtr<SCurveEditorView> FCurveEditorViewRegistry::ConstructView(ECurveEditorViewID ViewID, TWeakPtr<FCurveEditor> WeakCurveEditor)
{
	checkf(ViewID != ECurveEditorViewID::Invalid, TEXT("Invalid view ID specified"));

	switch (ViewID)
	{
	case ECurveEditorViewID::Absolute:
		return SNew(SCurveEditorViewAbsolute, WeakCurveEditor)
		.AutoSize(false);

	case ECurveEditorViewID::Normalized:
		return
			SNew(SCurveEditorViewNormalized, WeakCurveEditor)
			.AutoSize(false);

	case ECurveEditorViewID::Stacked:
		return
			SNew(SCurveEditorViewStacked, WeakCurveEditor);
	}

	FOnCreateCurveEditorView* Handler = CustomViews.Find(ViewID);
	if (Handler && Handler->IsBound())
	{
		return Handler->Execute(WeakCurveEditor);
	}

	return nullptr;
}