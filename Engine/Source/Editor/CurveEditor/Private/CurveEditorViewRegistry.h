// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "ICurveEditorModule.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class SCurveEditorView;

/**
 * Central, private registry of distinct view types that can be added to any curve editor panel based on which views the selected set of curve model's supports.
 * Public interface is through ICurveEditorModule::RegisterView and ICurveEditorModule::UnregisterView
 */
class FCurveEditorViewRegistry
{
public:

	/**
	 * Singleton access to the view registry
	 */
	static FCurveEditorViewRegistry& Get();


	/**
	 * Register a new view type
	 * @note: A maximum of 64 registered view types are supported. View type IDs are not recycled.
	 *
	 * @param InCreateViewDelegate    (required) A bound delegate that creates a new instance of the view widget. Delegate signature is TSharedRef<SCurveEditorView> Function(TWeakPtr<FCurveEditor>);
	 * @return A new custom view ID that identifies the registered view type. Any curve models that wish to support this view must |= this enum to its FCurveModel::SupportedViews;
	 */
	ECurveEditorViewID RegisterCustomView(const FOnCreateCurveEditorView& InCreateViewDelegate);


	/**
	 * Unregister a previously registered view type
	 *
	 * @param ViewID                  The view ID obtained from calling RegisterCustomView. Must be >= ECurveEditorViewID::CUSTOM_START
	 */
	void UnregisterCustomView(ECurveEditorViewID ViewID);


	/**
	 * Construct an instance of a view widget from its identifier
	 *
	 * @param ViewID                  The Identifier of the view to create. Must be a single ID, not a bitwise combination of IDs.
	 * @param WeakCurveEditor         A weak pointer to the curve editor on which the view is to be added.
	 * @return A new curve editor view widget, or nullptr if the ViewID does not correspond to a registered view type.
	 */
	TSharedPtr<SCurveEditorView> ConstructView(ECurveEditorViewID ViewID, TWeakPtr<FCurveEditor> WeakCurveEditor);

private:

	FCurveEditorViewRegistry();

	/** (default: ECurveEditorViewID::CUSTOM_START) The ID that the next registered view will receive */
	ECurveEditorViewID NextViewID;

	/** Mapping of single view IDs to their factory delegates */
	TMap<ECurveEditorViewID, FOnCreateCurveEditorView> CustomViews;
};