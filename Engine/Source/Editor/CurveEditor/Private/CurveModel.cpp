// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveModel.h"

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

UObject* FCurveModel::GetOwningObjectOrOuter(UClass* Class) const
{
	UObject* Obj = GetOwningObject();
	if (Obj && Class && Obj->IsA(Class))
	{
		return Obj;
	}
	return Obj ? Obj->GetTypedOuter(Class) : nullptr;
}

void FCurveModel::GetAllKeys(TArray<FKeyHandle>& OutKeyHandles) const
{
	GetKeys(
		TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(),
		OutKeyHandles
		);
}

void FCurveModel::PutKeys(
	TConstArrayView<FKeyHandle> InKeys, TConstArrayView<FKeyPosition> InKeyPositions, TConstArrayView<FKeyAttributes> InAttributes
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCurveModel::PutKeys)
	
	const int32 Num = InKeys.Num();
	if (ensure(InKeyPositions.Num() == Num && InAttributes.Num() == Num))
	{
		RemoveKeys(InKeys, 0.0);
	
		TArray<TOptional<FKeyHandle>> OptionalAddedHandles;
		OptionalAddedHandles.SetNum(InKeys.Num());
		TArrayView<TOptional<FKeyHandle>> NewKeyHandlesView = MakeArrayView(OptionalAddedHandles);
		AddKeys(InKeyPositions, InAttributes, &NewKeyHandlesView);

		TArray<FKeyHandle> AddedKeyHandles;
		Algo::TransformIf(OptionalAddedHandles, AddedKeyHandles,
			[](const TOptional<FKeyHandle>& Handle){ return Handle.IsSet(); },
			[](const TOptional<FKeyHandle>& Handle){ return *Handle; }
			);

		if (ensure(AddedKeyHandles.Num() == Num))
		{
			ReplaceKeyHandles(AddedKeyHandles, InKeys);
		}
	}
}

void FCurveModel::GetClosestKeysTo(double InTime, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const
{
	// Default implementation does linear search. Subclasses can override this with a more efficient implementations.
	double MinTime, MaxTime;
	GetTimeRange(MinTime, MaxTime);
	double MinValue, MaxValue;
	GetValueRange(MinValue, MaxValue);

	TArray<FKeyHandle> AllHandles;
	GetKeys(MinTime, MaxTime, MinValue, MaxValue, AllHandles);
	if (AllHandles.IsEmpty())
	{
		return;
	}
	
	TArray<FKeyPosition> AllPositions;
	AllPositions.SetNumUninitialized(AllHandles.Num());
	GetKeyPositions(AllHandles, AllPositions);

	// Ideally, we'd use binary search.
	// However, GetKeyPositions documentation does not dictate that the function returns a sorted array.
	// For now, this implementation does linear because it's safe.
	double CurrentClosestLeftTime = MinTime - 1.0; // We'll assume we don't underflow double range
	double CurrentClosestRightTime = MaxTime + 1.0; // We'll assume we don't overflow double range
	for (int32 Index = 0; Index < AllPositions.Num(); ++Index)
	{
		const FKeyHandle& KeyHandle = AllHandles[Index];
		const double CurrentTime = AllPositions[Index].InputValue;

		if (CurrentTime <= InTime && CurrentClosestLeftTime < CurrentTime)
		{
			CurrentClosestLeftTime = CurrentTime;
			OutPreviousKeyHandle = KeyHandle;
		}

		if (CurrentTime >= InTime && CurrentClosestRightTime > CurrentTime)
		{
			CurrentClosestRightTime = CurrentTime;
			OutNextKeyHandle = KeyHandle;
		}
	}
}

void FCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, const FKeyAttributes& InKeyAttributes, EPropertyChangeType::Type ChangeType)
{
	TArray<FKeyAttributes> ExpandedAttributes;
	ExpandedAttributes.Reserve(InKeys.Num());

	for (FKeyHandle Handle : InKeys)
	{
		ExpandedAttributes.Add(InKeyAttributes);
	}

	SetKeyAttributes(InKeys, ExpandedAttributes);
}

TOptional<FKeyHandle> FCurveModel::AddKey(const FKeyPosition& NewKeyPosition, const FKeyAttributes& InAttributes)
{
	TOptional<FKeyHandle> Handle;
	TArrayView<TOptional<FKeyHandle>> Handles = MakeArrayView(&Handle, 1);
	AddKeys(TArrayView<const FKeyPosition>(&NewKeyPosition, 1), TArrayView<const FKeyAttributes>(&InAttributes, 1), &Handles);
	return Handle;
}

void FCurveModel::OpenChangeScope()
{
	++ChangeScopeCounter;
	if (ChangeScopeCounter == 1)
	{
		OnOpenRootChangeScope();
	}
}

void FCurveModel::CloseChangeScope()
{
	if (!ensure(ChangeScopeCounter > 0))
	{
		return;
	}

	--ChangeScopeCounter;
	if (ChangeScopeCounter == 0)
	{
		OnCloseRootChangeScope();
	}
}
