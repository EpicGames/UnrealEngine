// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragOperations/CurveEditorDragOperation_Marquee.h"

#include "Algo/AnyOf.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "Math/UnrealMathUtility.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"

namespace UE::CurveEditor::MarqueeDetail
{
static TArray<FCurvePointHandle> FindOverlappingPoints(
	const FSlateRect& Marquee, const FCurveEditor* CurveEditor, const SCurveEditorView* LockedToView
	)
{
	TArray<FCurvePointHandle> AllPoints;
	
	if (LockedToView)
	{
		LockedToView->GetPointsThenCurveWithinWidgetRange(Marquee, &AllPoints);
	}
	else
	{
		const TSharedPtr<SCurveEditorPanel> CurveEditorPanel = CurveEditor->GetPanel();

		const FGeometry ViewContainerGeometry = CurveEditorPanel->GetViewContainerGeometry();
		const FSlateLayoutTransform InverseContainerTransform = ViewContainerGeometry.GetAccumulatedLayoutTransform().Inverse();
		for (TSharedPtr<SCurveEditorView> View : CurveEditorPanel->GetViews())
		{
			const FGeometry& LocalGeometry = View->GetCachedGeometry();
			const FSlateLayoutTransform ContainerToView = InverseContainerTransform.Concatenate(
				LocalGeometry.GetAccumulatedLayoutTransform()
				).Inverse();

			const FSlateRect UnclippedLocalMarquee = FSlateRect(
				ContainerToView.TransformPoint(Marquee.GetTopLeft2f()),
				ContainerToView.TransformPoint(Marquee.GetBottomRight2f())
				);
			const FSlateRect ClippedLocalMarquee = UnclippedLocalMarquee.IntersectionWith(
				FSlateRect(FVector2D(0.f,0.f), LocalGeometry.GetLocalSize())
				);

			if (ClippedLocalMarquee.IsValid() && !ClippedLocalMarquee.IsEmpty())
			{
				View->GetPointsThenCurveWithinWidgetRange(ClippedLocalMarquee, &AllPoints);
			}
		}
	}

	return AllPoints;
}

static void RestrictSelectionToEitherPointsOrTangents(bool bPreferPointSelection, FCurveEditor* CurveEditor)
{
	TArray<FCurvePointHandle> CurvePointsToRemove;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		for (FKeyHandle Handle : Pair.Value.AsArray())
		{
			ECurvePointType PointType = Pair.Value.PointType(Handle);

			if (bPreferPointSelection)
			{
				// If selecting points, deselect tangen handles (ie. anything that's not a point/key)
				if (PointType != ECurvePointType::Key)
				{
					CurvePointsToRemove.Add(FCurvePointHandle(Pair.Key, PointType, Handle));
				}
			}
			else
			{
				// Otherwise when selecting tangent handles, deselect anything that's a key
				if (PointType == ECurvePointType::Key)
				{
					CurvePointsToRemove.Add(FCurvePointHandle(Pair.Key, PointType, Handle));
				}
			}
		}
	}

	for (const FCurvePointHandle& Point : CurvePointsToRemove)
	{
		CurveEditor->Selection.Remove(Point);
	}
}

struct FSelectionPreferences
{
	bool bPreferPointSelection;
	bool bPreferTangentSelection;
};
static FSelectionPreferences GetSelectionPreferences(
	const TArray<FCurvePointHandle>& AllPoints, const FCurveEditor* CurveEditor, const bool bAnyModifiersKeysDown
	)
{
	const bool bSelectionContainsKeys = Algo::AnyOf(CurveEditor->Selection.GetAll(), [](const TPair<FCurveModelID, FKeyHandleSet>& Pair)
	{
		const FKeyHandleSet& HandleSet = Pair.Value;
		return Algo::AnyOf(Pair.Value.AsArray(), [&HandleSet](const FKeyHandle& Handle)
			{ return HandleSet.PointType(Handle) == ECurvePointType::Key; }
		);
	});
	const bool bSelectionContainsTangents = Algo::AnyOf(CurveEditor->Selection.GetAll(), [](const TPair<FCurveModelID, FKeyHandleSet>& Pair)
	{
		const FKeyHandleSet& HandleSet = Pair.Value;
		return Algo::AnyOf(Pair.Value.AsArray(), [&HandleSet](const FKeyHandle& Handle)
		{
			const ECurvePointType PointType = HandleSet.PointType(Handle);
			return PointType == ECurvePointType::ArriveTangent || PointType == ECurvePointType::LeaveTangent;
		});
	});
	// If, for whatever reason, the selection contains both keys and tangents already, prefer neither of the other.
	if (bSelectionContainsKeys && bSelectionContainsTangents)
	{
		return { false, false };
	}
	
	const bool bMarqueeHasAnyKeys = Algo::AnyOf(AllPoints, [](const FCurvePointHandle& Point)
		{ return Point.PointType == ECurvePointType::Key; }
	);
	
	// If there are any points to be selected, prefer selecting points over tangents.
	const bool bPreferPointSelection = (bMarqueeHasAnyKeys && !bAnyModifiersKeysDown)
		// If the selection already contains keys, also prefer selecting keys, i.e. don't add tangents to the pre-existing key selection... [1]
		|| (bAnyModifiersKeysDown && bSelectionContainsKeys)
		// If modifying the selection, prefer keys only if the selection does not contain tangents.
		|| (bMarqueeHasAnyKeys && bAnyModifiersKeysDown && !bSelectionContainsTangents);

	// We'll prefer tangents when the selection contains only tangents and modifying the selection (i.e. a modifier key is pressed).
	const bool bPreferTangentSelection = !bPreferPointSelection
		// [1] ... likewise if modifying the selection, prefer selecting tangents
		&& bAnyModifiersKeysDown && bSelectionContainsTangents;

	return { bPreferPointSelection, bPreferTangentSelection };
}
}

FCurveEditorDragOperation_Marquee::FCurveEditorDragOperation_Marquee(FCurveEditor* InCurveEditor)
	: FCurveEditorDragOperation_Marquee(InCurveEditor, nullptr)
{}

FCurveEditorDragOperation_Marquee::FCurveEditorDragOperation_Marquee(FCurveEditor* InCurveEditor, SCurveEditorView* InLockedToView)
	: CurveEditor(InCurveEditor)
	, LockedToView(InLockedToView)
{}

void FCurveEditorDragOperation_Marquee::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	SelectionChange.Emplace(CurveEditor->AsShared());
	RealInitialPosition = CurrentPosition;

	Marquee = FSlateRect(
		FMath::Min(RealInitialPosition.X, CurrentPosition.X),
		FMath::Min(RealInitialPosition.Y, CurrentPosition.Y),
		FMath::Max(RealInitialPosition.X, CurrentPosition.X),
		FMath::Max(RealInitialPosition.Y, CurrentPosition.Y)
		);
}

void FCurveEditorDragOperation_Marquee::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	Marquee = FSlateRect(
		FMath::Min(RealInitialPosition.X, CurrentPosition.X),
		FMath::Min(RealInitialPosition.Y, CurrentPosition.Y),
		FMath::Max(RealInitialPosition.X, CurrentPosition.X),
		FMath::Max(RealInitialPosition.Y, CurrentPosition.Y)
		);
}

void FCurveEditorDragOperation_Marquee::OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	const TArray<FCurvePointHandle> AllPoints = UE::CurveEditor::MarqueeDetail::FindOverlappingPoints(Marquee, CurveEditor, LockedToView);

	const bool bIsShiftDown = MouseEvent.IsShiftDown();
	const bool bIsAltDown = MouseEvent.IsAltDown();
	const bool bIsControlDown = MouseEvent.IsControlDown();
	const bool bAnyModifiersKeysDown = bIsShiftDown || bIsAltDown || bIsControlDown;
	
	if (!bAnyModifiersKeysDown)
	{
		CurveEditor->Selection.Clear();
	}
	
	const auto[bPreferPointSelection, bPreferTangentSelection] = UE::CurveEditor::MarqueeDetail::GetSelectionPreferences(
		AllPoints, CurveEditor, bAnyModifiersKeysDown
		);

	// When adding to the existing selection, ensure only either points or tangents are selected
	if (bIsShiftDown)
	{
		UE::CurveEditor::MarqueeDetail::RestrictSelectionToEitherPointsOrTangents(bPreferPointSelection, CurveEditor);
	}

	// Now that we've gathered the overlapping points, perform the relevant selection
	for (const FCurvePointHandle& Point : AllPoints)
	{
		if (bIsAltDown)
		{
			CurveEditor->Selection.Remove(Point);
		}
		else if (bIsControlDown)
		{
			if (bPreferPointSelection)
			{
				if (Point.PointType == ECurvePointType::Key)
				{
					CurveEditor->Selection.Toggle(Point);
				}
			}
			else
			{
				CurveEditor->Selection.Toggle(Point);
			}
		}
		else if (bPreferPointSelection)
		{
			if (Point.PointType == ECurvePointType::Key)
			{
				CurveEditor->Selection.Add(Point);
			}
		}
		else if (bPreferTangentSelection)
		{
			if (Point.PointType == ECurvePointType::ArriveTangent || Point.PointType == ECurvePointType::LeaveTangent)
			{
				CurveEditor->Selection.Add(Point);
			}
		}
		else
		{
			CurveEditor->Selection.Add(Point);
		}
	}

	SelectionChange.Reset();
}

void FCurveEditorDragOperation_Marquee::OnPaint(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId)
{
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		PaintOnLayerId,
		AllottedGeometry.ToPaintGeometry(Marquee.GetBottomRight() - Marquee.GetTopLeft(), FSlateLayoutTransform(Marquee.GetTopLeft())),
		FAppStyle::GetBrush(TEXT("MarqueeSelection"))
		);
}
