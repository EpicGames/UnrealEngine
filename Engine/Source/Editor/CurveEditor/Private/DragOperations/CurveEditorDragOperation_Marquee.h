// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreTypes.h"
#include "ICurveEditorDragOperation.h"
#include "Layout/SlateRect.h"
#include "Math/Vector2D.h"
#include "Modification/Utils/ScopedSelectionChange.h"

class FCurveEditor;
class FSlateWindowElementList;
class SCurveEditorView;
struct FGeometry;
struct FPointerEvent;

class FCurveEditorDragOperation_Marquee : public ICurveEditorDragOperation
{
public:

	FCurveEditorDragOperation_Marquee(FCurveEditor* InCurveEditor);
	FCurveEditorDragOperation_Marquee(FCurveEditor* InCurveEditor, SCurveEditorView* InLockedToView);

	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnPaint(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId) override;

private:

	/** Makes this change undo / redo able. Set for the duration of the drag. */
	TOptional<UE::CurveEditor::FScopedSelectionChange> SelectionChange;

	/** The current marquee rectangle */
	FSlateRect Marquee;
	/** Ptr back to the curve editor */
	FCurveEditor* CurveEditor;
	/** When valid, marquee selection should only occur inside this view; all geometries are in local space */
	SCurveEditorView* LockedToView;
	/** Real Initial Position do to the delayed drag*/
	FVector2D RealInitialPosition;
};