// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragOperations/CurveEditorDragOperation_MoveKeys.h"

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveModel.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "SCurveEditorView.h"
#include "ScopedTransaction.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealType.h"

struct FPointerEvent;

void FCurveEditorDragOperation_MoveKeys::OnInitialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& InCardinalPoint)
{
	CurveEditor = InCurveEditor;
	CardinalPoint = InCardinalPoint;
}

void FCurveEditorDragOperation_MoveKeys::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	using namespace UE::CurveEditor;
	
	ScopedKeyChange.Emplace(
		FCurvesSnapshotBuilder(CurveEditor->AsShared(), ECurveChangeFlags::MoveKeysAndRemoveStackedKeys)
		.TrackSelectedCurves()
		);

	KeysByCurve.Reset();
	CurveEditor->SuppressBoundTransformUpdates(true);

	LastMousePosition = CurrentPosition;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		FCurveModelID CurveID = Pair.Key;
		FCurveModel*  Curve   = CurveEditor->FindCurve(CurveID);

		if (ensureAlways(Curve))
		{
			TArrayView<const FKeyHandle> Handles = Pair.Value.AsArray();

			FKeyData& KeyData = KeysByCurve.Emplace_GetRef(CurveID);
			KeyData.Handles = TArray<FKeyHandle>(Handles.GetData(), Handles.Num());

			KeyData.StartKeyPositions.SetNumZeroed(KeyData.Handles.Num());
			Curve->GetKeyPositions(KeyData.Handles, KeyData.StartKeyPositions);

			KeyData.InitialDragTransform = Curve->GetCurveTransform();
			KeyData.LastDraggedKeyPositions = KeyData.StartKeyPositions;
		}
	}

	SnappingState.Reset();
}

void FCurveEditorDragOperation_MoveKeys::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	// OnDrag may be called multiple times in the same frame. We'll accumulate the end position and process it in OnFinishedPointerInput.
	GetOrAddAccumulatedMouseMovement(InitialPosition).EndMousePosition = CurveEditor->GetAxisSnap().GetSnappedPosition(
		InitialPosition, LastMousePosition, CurrentPosition, MouseEvent, SnappingState
		);
	LastMousePosition = CurrentPosition;
}

void FCurveEditorDragOperation_MoveKeys::OnFinishedPointerInput()
{
	// OnFinishedPointerInput is called once the engine is done pumping messages.
	if (AccumulatedMouseMovement)
	{
		UpdateFromDrag(AccumulatedMouseMovement->InitialPosition, AccumulatedMouseMovement->EndMousePosition);
		AccumulatedMouseMovement.Reset();
	}
}

void FCurveEditorDragOperation_MoveKeys::OnCancelDrag()
{
	ICurveEditorKeyDragOperation::OnCancelDrag();

	for (FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			FTransform2d InverseInitialDragTransform = KeyData.InitialDragTransform.Inverse();
			for (FKeyPosition& Position : KeyData.StartKeyPositions)
			{
				Position = Position.Transform(InverseInitialDragTransform);
			}

			Curve->SetKeyPositions(KeyData.Handles, KeyData.StartKeyPositions, EPropertyChangeType::ValueSet);
		}
	}

	CurveEditor->SuppressBoundTransformUpdates(false);
	
	ScopedKeyChange->Cancel();
	ScopedKeyChange.Reset();
}

void FCurveEditorDragOperation_MoveKeys::OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	ICurveEditorKeyDragOperation::OnEndDrag(InitialPosition, CurrentPosition, MouseEvent);

	for (const FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			Curve->SetKeyPositions(KeyData.Handles, KeyData.LastDraggedKeyPositions, EPropertyChangeType::ValueSet);
		}
	}

	CurveEditor->SuppressBoundTransformUpdates(false);
	
	ScopedKeyChange.Reset();
}

FCurveEditorDragOperation_MoveKeys::FAccumulatedMouseMovement& FCurveEditorDragOperation_MoveKeys::GetOrAddAccumulatedMouseMovement(
	const FVector2D& InitialPosition
	)
{
	if (!AccumulatedMouseMovement)
	{
		AccumulatedMouseMovement.Emplace(InitialPosition);
	}
	return *AccumulatedMouseMovement;
}

void FCurveEditorDragOperation_MoveKeys::UpdateFromDrag(const FVector2D& InitialPosition, const FVector2D& MousePosition)
{
	TArray<FKeyPosition> NewKeyPositionScratch;
	for (FKeyData& KeyData : KeysByCurve)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(KeyData.CurveID);
		if (!View)
		{
			continue;
		}

		FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID);
		if (!ensureAlways(Curve))
		{
			continue;
		}

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(KeyData.CurveID);

		double DeltaInput = (MousePosition.X - InitialPosition.X) / CurveSpace.PixelsPerInput();
		double DeltaOutput = -(MousePosition.Y - InitialPosition.Y) / CurveSpace.PixelsPerOutput();

		NewKeyPositionScratch.Reset();
		NewKeyPositionScratch.Reserve(KeyData.StartKeyPositions.Num());

		FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(KeyData.CurveID);

		// Transform by the inverse curve transform to put the start positions in view space
		FTransform2d CurveTransform               = Curve->GetCurveTransform();
		FTransform2d InverseInitialCurveTransform = KeyData.InitialDragTransform.Inverse();

		if (CardinalPoint.IsSet())
		{
			for (int KeyIndex = 0; KeyIndex < KeyData.StartKeyPositions.Num(); ++KeyIndex)
			{
				FKeyHandle KeyHandle = KeyData.Handles[KeyIndex];
				if (CardinalPoint->KeyHandle == KeyHandle)
				{
					FKeyPosition StartPosition = KeyData.StartKeyPositions[KeyIndex].Transform(InverseInitialCurveTransform);

					if (View->IsTimeSnapEnabled())
					{
						DeltaInput = SnapMetrics.SnapInputSeconds(StartPosition.InputValue + DeltaInput) - StartPosition.InputValue;
					}

					// If view is not absolute, snap based on the key that was grabbed, not all keys individually.
					if (View->IsValueSnapEnabled() && View->ViewTypeID != ECurveEditorViewID::Absolute)
					{
						DeltaOutput = SnapMetrics.SnapOutput(StartPosition.OutputValue + DeltaOutput) - StartPosition.OutputValue;
					}

					break;
				}
			}
		}
		for (FKeyPosition StartPosition : KeyData.StartKeyPositions)
		{
			StartPosition = StartPosition.Transform(InverseInitialCurveTransform);

			StartPosition.InputValue += DeltaInput;
			StartPosition.OutputValue += DeltaOutput;

			StartPosition.InputValue = View->IsTimeSnapEnabled() ? SnapMetrics.SnapInputSeconds(StartPosition.InputValue) : StartPosition.InputValue;

			// Snap value keys individually if view mode is absolute.
			if (View->ViewTypeID == ECurveEditorViewID::Absolute)
			{
				StartPosition.OutputValue = View->IsValueSnapEnabled() ? SnapMetrics.SnapOutput(StartPosition.OutputValue) : StartPosition.OutputValue;
			}

			NewKeyPositionScratch.Add(StartPosition.Transform(CurveTransform));
		}

		Curve->SetKeyPositions(KeyData.Handles, NewKeyPositionScratch, EPropertyChangeType::Interactive);

		// Make sure the last dragged key positions are up to date
		Curve->GetKeyPositions(KeyData.Handles, KeyData.LastDraggedKeyPositions);
	}
}
