// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "ICurveEditorDragOperation.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Modification/Utils/ScopedCurveChange.h"

class FCurveEditor;
struct FPointerEvent;

class FCurveEditorDragOperation_MoveKeys : public ICurveEditorKeyDragOperation
{
public:

	//~ Begin ICurveEditorDragOperation Interface
	virtual void OnInitialize(FCurveEditor* CurveEditor, const TOptional<FCurvePointHandle>& CardinalPoint) override;
	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnFinishedPointerInput() override;
	virtual void OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnCancelDrag() override;
	//~ End ICurveEditorDragOperation Interface

private:

	/** Ptr back to the curve editor */
	FCurveEditor* CurveEditor;
	TOptional<FCurvePointHandle> CardinalPoint;

	FVector2D LastMousePosition;

	/** Transacts the moved keys. */
	TOptional<UE::CurveEditor::FScopedCurveChange> ScopedKeyChange;

	struct FKeyData
	{
		FKeyData(FCurveModelID InCurveID)
			: CurveID(InCurveID)
		{}

		/** The curve that contains the keys we're dragging */
		FCurveModelID CurveID;
		/** All the handles within a given curve that we are dragging */
		TArray<FKeyHandle> Handles;
		/** The extended key info for each of the above handles */
		TArray<FKeyPosition> StartKeyPositions;
		/** Used in OnEndDrag to send final key updates */
		TArray<FKeyPosition> LastDraggedKeyPositions;
		/** Initial curve transform for this curve */
		FTransform2d InitialDragTransform;
	};

	/** Key dragging data stored per-curve */
	TArray<FKeyData> KeysByCurve;

	FCurveEditorAxisSnap::FSnapState SnappingState;

	struct FAccumulatedMouseMovement
	{
		const FVector2D InitialPosition;
		FVector2D EndMousePosition;

		explicit FAccumulatedMouseMovement(const FVector2D& InitialPosition)
			: InitialPosition(InitialPosition)
		{}
	};
	TOptional<FAccumulatedMouseMovement> AccumulatedMouseMovement;

	FAccumulatedMouseMovement& GetOrAddAccumulatedMouseMovement(const FVector2D& InitialPosition);

	void UpdateFromDrag(const FVector2D& InitialPosition, const FVector2D& MousePosition);
};