// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorDragOperation_Pan.h"

#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorZoomScaleConfig.h"
#include "ICurveEditorBounds.h"
#include "Input/Events.h"
#include "Math/UnrealMathUtility.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"

FCurveEditorDragOperation_PanView::FCurveEditorDragOperation_PanView(FCurveEditor* InCurveEditor, TSharedPtr<SCurveEditorView> InView)
	: CurveEditor(InCurveEditor)
	, View(InView)
	, bIsDragging(false)
{}

void FCurveEditorDragOperation_PanView::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	FCurveEditorScreenSpace ViewSpace = View->GetViewSpace();

	InitialInputMin = ViewSpace.GetInputMin();
	InitialInputMax = ViewSpace.GetInputMax();
	InitialOutputMin = ViewSpace.GetOutputMin();
	InitialOutputMax = ViewSpace.GetOutputMax();

	bIsDragging = true;
}

void FCurveEditorDragOperation_PanView::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	const FVector2D PixelDelta = CurrentPosition - InitialPosition;

	FCurveEditorScreenSpace ViewSpace = View->GetViewSpace();

	double InputMin = InitialInputMin - PixelDelta.X / ViewSpace.PixelsPerInput();
	double InputMax = InitialInputMax - PixelDelta.X / ViewSpace.PixelsPerInput();

	double OutputMin = InitialOutputMin + PixelDelta.Y / ViewSpace.PixelsPerOutput();
	double OutputMax = InitialOutputMax + PixelDelta.Y / ViewSpace.PixelsPerOutput();

	View->SetInputBounds(InputMin, InputMax);
	View->SetOutputBounds(OutputMin, OutputMax);
}

FReply FCurveEditorDragOperation_PanView::OnMouseWheel(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	if (bIsDragging)
	{
		FCurveEditorScreenSpace ViewSpace = View->GetViewSpace();

		double CurrentTime = ViewSpace.ScreenToSeconds(CurrentPosition.X);
		double CurrentValue = ViewSpace.ScreenToValue(CurrentPosition.Y);


		const double WheelMultiplier = CurveEditor->GetZoomScaleConfig().GetMouseWheelZoomMultiplierClamped();
		const double ZoomDelta = 1.0 - FMath::Clamp(0.1 * WheelMultiplier * MouseEvent.GetWheelDelta(), -0.9, 0.9);
		View->ZoomAround(FVector2D(ZoomDelta, ZoomDelta), CurrentTime, CurrentValue);

		// Adjust the stored initial bounds by the zoom delta so delta calculations work properly
		InitialInputMin = CurrentTime - (CurrentTime - InitialInputMin) * ZoomDelta;
		InitialInputMax = CurrentTime + (InitialInputMax - CurrentTime) * ZoomDelta;
		InitialOutputMin = CurrentValue - (CurrentValue - InitialOutputMin) * ZoomDelta;
		InitialOutputMax = CurrentValue + (InitialOutputMax - CurrentValue) * ZoomDelta;

		return FReply::Handled();
	}
	return FReply::Unhandled();
}


FCurveEditorDragOperation_PanInput::FCurveEditorDragOperation_PanInput(FCurveEditor* InCurveEditor)
	: CurveEditor(InCurveEditor)
{}

void FCurveEditorDragOperation_PanInput::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	FCurveEditorScreenSpaceH InputSpace = CurveEditor->GetPanelInputSpace();
	InitialInputMin = InputSpace.GetInputMin();
	InitialInputMax = InputSpace.GetInputMax();

}

void FCurveEditorDragOperation_PanInput::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	const FVector2D PixelDelta = CurrentPosition - InitialPosition;

	FCurveEditorScreenSpaceH InputSpace = CurveEditor->GetPanelInputSpace();

	double InputMin = InitialInputMin - PixelDelta.X / InputSpace.PixelsPerInput();
	double InputMax = InitialInputMax - PixelDelta.X / InputSpace.PixelsPerInput();
	CurveEditor->GetBounds().SetInputBounds(InputMin, InputMax);

	CurveEditor->GetPanel()->ScrollBy(-MouseEvent.GetCursorDelta().Y);
}