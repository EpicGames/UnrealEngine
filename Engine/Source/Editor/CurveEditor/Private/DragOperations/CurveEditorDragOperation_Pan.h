// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSnapMetrics.h"
#include "ICurveEditorDragOperation.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class SCurveEditorView;
struct FPointerEvent;

class FCurveEditorDragOperation_PanView : public ICurveEditorDragOperation
{
public:
	FCurveEditorDragOperation_PanView(FCurveEditor* CurveEditor, TSharedPtr<SCurveEditorView> InView);

	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;

private:

	FCurveEditor* CurveEditor;
	TSharedPtr<SCurveEditorView> View;

	bool bIsDragging;
	
	double InitialInputMin, InitialInputMax;
	double InitialOutputMin, InitialOutputMax;
	FCurveEditorAxisSnap::FSnapState SnappingState;
	FVector2D LastMousePosition;

};

class FCurveEditorDragOperation_PanInput : public ICurveEditorDragOperation
{
public:
	FCurveEditorDragOperation_PanInput(FCurveEditor* CurveEditor);

	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;

private:

	FCurveEditor* CurveEditor;
	double InitialInputMin, InitialInputMax;
};