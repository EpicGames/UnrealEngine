// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorDragOperation_ScrubTime.h"

#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "ICurveEditorBounds.h"
#include "Input/Events.h"
#include "Math/UnrealMathUtility.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Application/ThrottleManager.h"


FCurveEditorDragOperation_ScrubTime::FCurveEditorDragOperation_ScrubTime(FCurveEditor* InCurveEditor)
	: CurveEditor(InCurveEditor) , NextPlayerStatus(ETimeSliderPlaybackStatus::Stopped)
{}

void FCurveEditorDragOperation_ScrubTime::OnMouseButtonDown(FVector2D InitialPosition, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FSlateThrottleManager::Get().DisableThrottle(true);
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (TimeSliderController)
	{
		const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
		if (Settings && Settings->GetScrubTimeStartFromCursor())
		{
			const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
			FCurveEditorScreenSpaceH InputSpace = CurveEditor->GetPanelInputSpace();
			const double InitialMouseTime = InputSpace.ScreenToSeconds(InitialPosition.X);

			if (TimeSliderController->GetPlaybackStatus() == ETimeSliderPlaybackStatus::Playing)
			{
				TimeSliderController->SetPlaybackStatus(ETimeSliderPlaybackStatus::Paused);
				TimeSliderController->SetStoppedPosition((InitialMouseTime)*TickResolution);
				TimeSliderController->SetPlaybackStatus(ETimeSliderPlaybackStatus::Playing);
			}
			else
			{
				TimeSliderController->SetScrubPosition((InitialMouseTime)*TickResolution, /*bEvaluate*/ true);
			}
		}

	}
}

void FCurveEditorDragOperation_ScrubTime::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FSlateThrottleManager::Get().DisableThrottle(false);
}

void FCurveEditorDragOperation_ScrubTime::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	FSlateThrottleManager::Get().DisableThrottle(true);
	NextPlayerStatus = ETimeSliderPlaybackStatus::Stopped;
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (TimeSliderController)
	{	
		SnappingState.Reset();
		LastMousePosition = CurrentPosition;
		const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		if (TimeSliderController->GetPlaybackStatus() == ETimeSliderPlaybackStatus::Playing)
		{
			NextPlayerStatus = ETimeSliderPlaybackStatus::Playing;
		}
		else
		{
 			NextPlayerStatus = ETimeSliderPlaybackStatus::Stopped;
		}		
		TimeSliderController->SetPlaybackStatus(ETimeSliderPlaybackStatus::Scrubbing);
		
		InitialTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());
	}
}

void FCurveEditorDragOperation_ScrubTime::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (TimeSliderController)
	{

		FVector2D PixelDelta = CurveEditor->GetAxisSnap().GetSnappedPosition(InitialPosition, LastMousePosition, CurrentPosition, MouseEvent, SnappingState, true) - InitialPosition;
		LastMousePosition = CurrentPosition;

		FCurveEditorScreenSpaceH InputSpace = CurveEditor->GetPanelInputSpace();
		const double InitialMouseTime = InputSpace.ScreenToSeconds(InitialPosition.X);
		const double CurrentMouseTime = InputSpace.ScreenToSeconds(LastMousePosition.X);
		const double DiffSeconds = CurrentMouseTime - InitialMouseTime;
		const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		double TimeToSet = (InitialTime + DiffSeconds);
		if (CurveEditor->InputSnapEnabledAttribute.Get())
		{
			FCurveSnapMetrics SnapMetrics;
			SnapMetrics.bSnapInputValues = true;
			SnapMetrics.InputSnapRate = CurveEditor->InputSnapRateAttribute.Get();
			TimeToSet = SnapMetrics.SnapInputSeconds(TimeToSet);
		}
		TimeSliderController->SetScrubPosition((TimeToSet)*TickResolution, /*bEvaluate*/ true);
	}
}

void FCurveEditorDragOperation_ScrubTime::OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (TimeSliderController)
	{
		TimeSliderController->SetPlaybackStatus(NextPlayerStatus);
	}
	FSlateThrottleManager::Get().DisableThrottle(false);
}

void FCurveEditorDragOperation_ScrubTime::OnCancelDrag()
{
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (TimeSliderController)
	{
		TimeSliderController->SetPlaybackStatus(NextPlayerStatus);
	}
}