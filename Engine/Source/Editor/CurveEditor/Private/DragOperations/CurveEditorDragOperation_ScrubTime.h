// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSnapMetrics.h"
#include "ICurveEditorDragOperation.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "ITimeSlider.h"

class FCurveEditor;
struct FPointerEvent;
class ITimeSliderController;

class FCurveEditorDragOperation_ScrubTime : public ICurveEditorDragOperation
{
public:
	FCurveEditorDragOperation_ScrubTime(FCurveEditor* CurveEditor);

	virtual void OnMouseButtonDown(FVector2D InitialPosition, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;

	virtual void OnCancelDrag() override;

private:

	FCurveEditor* CurveEditor;
	FCurveEditorAxisSnap::FSnapState SnappingState;
	FVector2D LastMousePosition;
	double InitialTime;
	ETimeSliderPlaybackStatus NextPlayerStatus;
};