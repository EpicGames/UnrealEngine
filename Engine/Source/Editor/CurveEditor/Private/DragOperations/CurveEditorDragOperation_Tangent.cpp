// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorDragOperation_Tangent.h"

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "CurveEditor.h"
#include "CurveEditorHelpers.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSettings.h"
#include "CurveModel.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "SCurveEditorView.h"
#include "ScopedTransaction.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealType.h"

namespace CurveEditorDragOperation
{
	FTangentSolver::FTangentSolver(ECurvePointType InPointType, FKeyAttributes InInitialKeyAttributes)
		: InitialKeyAttributes(InInitialKeyAttributes)
		, PointType(InPointType)
	{}

	void FTangentSolver::Process(
		FCurveEditor* CurveEditor,
		const FPointerEvent& MouseEvent, 
		const FCurveEditorScreenSpace& CurveSpace, 
		const FVector2D& PixelDelta)
	{
		const float DisplayRatio = (CurveSpace.PixelsPerOutput() / CurveSpace.PixelsPerInput());

		if (InitialKeyAttributes.HasArriveTangent() && EnumHasAnyFlags(PointType, ECurvePointType::ArriveTangent))
		{
			const bool bArriveTangent = true;
			const float ArriveTangent = InitialKeyAttributes.GetArriveTangent();

			const FSolverParams SolverParams(
				CurveEditor,
				MouseEvent,
				CurveSpace,
				PixelDelta,
				DisplayRatio,
				bArriveTangent,
				ArriveTangent);

			if (InitialKeyAttributes.HasTangentWeightMode() && InitialKeyAttributes.HasArriveTangentWeight() &&
				(InitialKeyAttributes.GetTangentWeightMode() == RCTWM_WeightedBoth || InitialKeyAttributes.GetTangentWeightMode() == RCTWM_WeightedArrive))
			{
				SolveTangentMutableWeight(SolverParams, InitialKeyAttributes.GetArriveTangentWeight());
			}
			else
			{
				SolveTangentConstantWeight(SolverParams);
			}
		}

		if (InitialKeyAttributes.HasLeaveTangent() && EnumHasAnyFlags(PointType, ECurvePointType::LeaveTangent))
		{
			const bool bArriveTangent = false;
			const float LeaveTangent = InitialKeyAttributes.GetLeaveTangent();

			const FSolverParams SolverParams(
				CurveEditor,
				MouseEvent,
				CurveSpace,
				PixelDelta,
				DisplayRatio,
				bArriveTangent,
				LeaveTangent);

			if (InitialKeyAttributes.HasTangentWeightMode() && InitialKeyAttributes.HasLeaveTangentWeight() &&
				(InitialKeyAttributes.GetTangentWeightMode() == RCTWM_WeightedBoth || InitialKeyAttributes.GetTangentWeightMode() == RCTWM_WeightedLeave))
			{
				SolveTangentMutableWeight(SolverParams, InitialKeyAttributes.GetLeaveTangentWeight());
			}
			else
			{
				SolveTangentConstantWeight(SolverParams);
			}
		}
	}

	FTangentSolver::FSolverParams::FSolverParams(
		FCurveEditor* InCurveEditor,
		const FPointerEvent& InMouseEvent,
		const FCurveEditorScreenSpace& InCurveSpace,
		const FVector2D& InPixelDelta,
		const float InDisplayRatio,
		const bool bInArriveTangent,
		const float InInitialTangent)
		: CurveEditor(InCurveEditor)
		, MouseEvent(InMouseEvent)
		, CurveSpace(InCurveSpace)
		, PixelDelta(InPixelDelta)
		, DisplayRatio(InDisplayRatio)
		, bArriveTangent(bInArriveTangent)
		, InitialTangent(InInitialTangent)
	{}

	void FTangentSolver::SolveTangentConstantWeight(const FSolverParams& Params)
	{
		if (Params.MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			Params.CurveEditor->GetSettings()->AllowMouseEdit(Params.MouseEvent))
		{				
			const float PixelLength = Params.bArriveTangent ? -60.0f : 60.f;
			const float Slope = Params.InitialTangent * -Params.DisplayRatio;

			const FVector2D InitialTangentOffset = CurveEditor::GetVectorFromSlopeAndLength(Slope, PixelLength);
			FVector2D NewTangentOffset = InitialTangentOffset + Params.PixelDelta;

			MendNearlyZeroXTangentOffset(Params.bArriveTangent, NewTangentOffset);

			const float Tangent = (-NewTangentOffset.Y / NewTangentOffset.X) / Params.DisplayRatio;

			if (Params.bArriveTangent)
			{
				KeyAttributes.SetArriveTangent(Tangent);
			}
			else
			{
				KeyAttributes.SetLeaveTangent(Tangent);
			}

			LastTangentOffset = NewTangentOffset;
		}
	}

	void FTangentSolver::SolveTangentMutableWeight(const FSolverParams& Params, float InitialWeight)
	{
		InitialWeight = Params.bArriveTangent ? -InitialWeight : InitialWeight;

		FVector2D InitialTangentOffset = CurveEditor::ComputeScreenSpaceTangentOffset(
			Params.CurveSpace,
			Params.InitialTangent,
			InitialWeight);

		FVector2D NewTangentOffset;

		if (Params.MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			Params.CurveEditor->GetSettings()->AllowMouseEdit(Params.MouseEvent))
		{
			const FVector2D PreviousTangentOffset = LastTangentOffset.IsSet() ? LastTangentOffset.GetValue() : InitialTangentOffset;
			const FVector2D PointAtOffset = InitialTangentOffset + Params.PixelDelta;

			if (Params.MouseEvent.IsShiftDown() &&
				Params.MouseEvent.IsControlDown())
			{				
				// Draging with Ctrl-Shift modifiers adjusts the tangent, but not the weight
				const double Weigth = PreviousTangentOffset.Size();
				NewTangentOffset = PointAtOffset.GetSafeNormal() * Weigth;

				if ((Params.bArriveTangent && NewTangentOffset.X >= 0.0) ||
					(!Params.bArriveTangent && NewTangentOffset.X <= 0.0))
				{
					NewTangentOffset = { 0.0, Weigth * FMath::Sign(NewTangentOffset.Y) };
				}
			}
			else if (Params.MouseEvent.IsShiftDown())
			{				
				// Draging with Shift modifier adjusts the weight, but not the tangent
				const double Scalar = FVector2D::DotProduct(PointAtOffset, PreviousTangentOffset.GetSafeNormal());

				if (Scalar >= 1.0)
				{
					NewTangentOffset = PreviousTangentOffset.GetSafeNormal() * Scalar;
				}
				else
				{
					NewTangentOffset = PreviousTangentOffset.GetSafeNormal();
				}
			}
			else
			{
				// Draging without modifiers adjusts the tangent and weight 
				NewTangentOffset = InitialTangentOffset + Params.PixelDelta;
			}

			LastTangentOffset = NewTangentOffset;

			MendNearlyZeroXTangentOffset(Params.bArriveTangent, NewTangentOffset);

			float Tangent;
			float Weight;
			CurveEditor::TangentAndWeightFromOffset(Params.CurveSpace, NewTangentOffset, Tangent, Weight);

			if (Params.bArriveTangent)
			{
				KeyAttributes.SetArriveTangent(Tangent);
				KeyAttributes.SetArriveTangentWeight(Weight);
			}
			else
			{
				KeyAttributes.SetLeaveTangent(Tangent);
				KeyAttributes.SetLeaveTangentWeight(Weight);
			}
		}
	}

	void FTangentSolver::MendNearlyZeroXTangentOffset(const bool bArriveTangent, FVector2D& InOutTangentOffset) const
	{
		constexpr float TangentCrossoverThresholdPx = 1.f;
		if (bArriveTangent)
		{
			InOutTangentOffset.X = FMath::Min(InOutTangentOffset.X, -TangentCrossoverThresholdPx);
		}
		else
		{
			InOutTangentOffset.X = FMath::Max(InOutTangentOffset.X, TangentCrossoverThresholdPx);
		}
	}
}

void FCurveEditorDragOperation_Tangent::OnInitialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& InCardinalPoint)
{
	CurveEditor = InCurveEditor;
}

void FCurveEditorDragOperation_Tangent::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	using namespace UE::CurveEditor;
	
	ScopedKeyChange.Emplace(FCurvesSnapshotBuilder(CurveEditor->AsShared(), ECurveChangeFlags::KeyAttributes).TrackSelectedCurves());
	CurveEditor->SuppressBoundTransformUpdates(true);

	KeysByCurve.Reset();
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		FCurveModelID CurveID = Pair.Key;
		FCurveModel* Curve    = CurveEditor->FindCurve(CurveID);

		if (ensureAlways(Curve))
		{
			TArrayView<const FKeyHandle> Handles = Pair.Value.AsArray();

			FKeyData& KeyData = KeysByCurve.Emplace_GetRef(CurveID);
			KeyData.Handles = TArray<FKeyHandle>(Handles.GetData(), Handles.Num());

			TArray<FKeyAttributes> KeyAttributesArray;
			KeyAttributesArray.SetNum(KeyData.Handles.Num());
			Curve->GetKeyAttributes(KeyData.Handles, KeyAttributesArray);

			KeyData.TangentSolvers.Reserve(Handles.Num());
			for (int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex)
			{
				const FKeyHandle& Handle = Handles[HandleIndex];
				const ECurvePointType PointType = Pair.Value.PointType(Handle);
				const FKeyAttributes& KeyAttributes = KeyAttributesArray[HandleIndex];

				KeyData.TangentSolvers.Add(CurveEditorDragOperation::FTangentSolver(PointType, KeyAttributes));
			}
		}
	}
}

void FCurveEditorDragOperation_Tangent::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	FVector2D PixelDelta = CurrentPosition - InitialPosition;

	TArray<FKeyAttributes> NewKeyAttributesScratch;

	for (FKeyData& KeyData : KeysByCurve)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(KeyData.CurveID);
		if (!View)
		{
			continue;
		}

		FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID);
		if (!ensureAlways(Curve))
		{
			continue;
		}

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(KeyData.CurveID);

		NewKeyAttributesScratch.Reset();
		NewKeyAttributesScratch.Reserve(KeyData.TangentSolvers.Num());

		for (CurveEditorDragOperation::FTangentSolver& Solver : KeyData.TangentSolvers)
		{
			Solver.Process(CurveEditor, MouseEvent, CurveSpace, PixelDelta);

			NewKeyAttributesScratch.Add(Solver.GetKeyAttributes());
		}

		Curve->SetKeyAttributes(KeyData.Handles, NewKeyAttributesScratch, EPropertyChangeType::Interactive);
	}
}

void FCurveEditorDragOperation_Tangent::OnCancelDrag()
{
	ICurveEditorKeyDragOperation::OnCancelDrag();

	for (const FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			TArray<FKeyAttributes> InitialAttributes;
			InitialAttributes.Reserve(KeyData.TangentSolvers.Num());
			Algo::Transform(KeyData.TangentSolvers, InitialAttributes, &CurveEditorDragOperation::FTangentSolver::GetInitialKeyAttributes);

			Curve->SetKeyAttributes(KeyData.Handles, InitialAttributes, EPropertyChangeType::ValueSet);
		}
	}

	CurveEditor->SuppressBoundTransformUpdates(false);
	
	ScopedKeyChange->Cancel();
	ScopedKeyChange.Reset();
}

void FCurveEditorDragOperation_Tangent::OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	ICurveEditorKeyDragOperation::OnEndDrag(InitialPosition, CurrentPosition, MouseEvent);

	for (const FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			TArray<FKeyAttributes> FinalAttributes;
			FinalAttributes.Reserve(KeyData.TangentSolvers.Num());
			Algo::Transform(KeyData.TangentSolvers, FinalAttributes, &CurveEditorDragOperation::FTangentSolver::GetKeyAttributes);

			Curve->SetKeyAttributes(KeyData.Handles, FinalAttributes, EPropertyChangeType::ValueSet);
		}
	}
	
	CurveEditor->SuppressBoundTransformUpdates(false);
	ScopedKeyChange.Reset();
}
