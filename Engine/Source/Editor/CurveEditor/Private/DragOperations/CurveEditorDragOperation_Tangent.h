// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "ICurveEditorDragOperation.h"
#include "Math/Vector2D.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Modification/Utils/ScopedCurveChange.h"

class FCurveEditor;
struct FCurveEditorScreenSpace;
struct FPointerEvent;

namespace CurveEditorDragOperation
{
	/** Solves the tangent offset for a single key depending on mouse button and modifier keys */
	class FTangentSolver
	{
	public:
		FTangentSolver(ECurvePointType InPointType, FKeyAttributes InInitialKeyAttributes);

		/** 
		 * Solves the tangent considering the current mouse position, the mouse buttons and modifier keys
		 * 
		 * @param MouseEvent			The mouse event to Process
		 * @param CurveSpace			The screen space of the curve editor
		 * @param PixelDelta			The mouse position delta from the initial mouse position in pixels
		 */
		void Process(FCurveEditor* CurveEditor, const FPointerEvent& MouseEvent, const FCurveEditorScreenSpace& CurveSpace, const FVector2D& PixelDelta);

		/** Returns the current key attributes */
		const FKeyAttributes& GetKeyAttributes() const { return KeyAttributes; }

		/** Returns the initial key attributes */
		const FKeyAttributes& GetInitialKeyAttributes() const { return InitialKeyAttributes; }

	private:
		/** Struct that holds relevant data to compute the new tangent */
		struct FSolverParams
		{
			FSolverParams(
				FCurveEditor* InCurveEditor,
				const FPointerEvent& InMouseEvent,
				const FCurveEditorScreenSpace& InCurveSpace,
				const FVector2D& InPixelDelta,
				const float InDisplayRatio,
				const bool bInArriveTangent,
				const float InInitialTangent);

			/** The curve editor */
			FCurveEditor* CurveEditor;

			/** The current mouse event */
			const FPointerEvent& MouseEvent;

			/** The current curve space */
			const FCurveEditorScreenSpace& CurveSpace;

			/** The delta of the mouse cursor from the initial position in pixels */
			const FVector2D& PixelDelta;

			/** The current display ratio */
			const float DisplayRatio;

			/** True if the tangent to process is an arrive tangent, else it is a leave tangent */
			const bool bArriveTangent;

			/** The initial tangent when the drag drop op started */
			const float InitialTangent;

			// Non-copyable
			FSolverParams(const FSolverParams&) = delete;
			FSolverParams& operator=(const FSolverParams&) = delete;
		};

		/** Solves tangent leaving the weight untouched. Useful when tangent weight mode is disabled. */
		void SolveTangentConstantWeight(const FSolverParams& Params);

		/** Solves tangent and weight. Useful when tangent weight mode is enabled. */
		void SolveTangentMutableWeight(const FSolverParams& Params, float InitialWeight);

		/** 
		 * Prevents the handle from crossing over the 0 point. Curve editor would handle it but it creates 
		 * an ugly pop in the curve and it lets the Arrive tangents become Leave tangents which defeats the point.
		 */
		void MendNearlyZeroXTangentOffset(const bool bArriveTangent, FVector2D& InOutTangentOffset) const;
		
		/** Holds the last tanget offset. Useful when editing the tangent or weight exclusively. */
		TOptional<FVector2D> LastTangentOffset;

		/** The current attributes */
		FKeyAttributes KeyAttributes;

		/** The initial attributes of the key */
		const FKeyAttributes InitialKeyAttributes;

		/** The point type of the key handle for which the tangent is solved */
		const ECurvePointType PointType;
	};
}

class FCurveEditorDragOperation_Tangent : public ICurveEditorKeyDragOperation
{
public:

	virtual void OnInitialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& InCardinalPoint) override;
	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnCancelDrag() override;

private:		
	/** Ptr back to the curve editor */
	FCurveEditor* CurveEditor;

private:

	struct FKeyData
	{
		FKeyData(FCurveModelID InCurveID)
			: CurveID(InCurveID)
		{}

		/** The curve that contains the keys we're dragging */
		FCurveModelID CurveID;

		/** The handles that are being dragged */
		TArray<FKeyHandle> Handles;

		/** Tangent solvers for the handles that are being dragged */
		TArray<CurveEditorDragOperation::FTangentSolver> TangentSolvers;
	};

	/** Key dragging data stored per-curve */
	TArray<FKeyData> KeysByCurve;
	
	/** Transacts the keys' tangents. */
	TOptional<UE::CurveEditor::FScopedCurveChange> ScopedKeyChange;
};