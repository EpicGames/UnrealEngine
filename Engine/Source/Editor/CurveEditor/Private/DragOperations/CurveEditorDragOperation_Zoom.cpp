// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorDragOperation_Zoom.h"

#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorZoomScaleConfig.h"
#include "ICurveEditorBounds.h"
#include "SCurveEditor.h"
#include "Input/Events.h"
#include "Math/UnrealMathUtility.h"
#include "SCurveEditorView.h"

TAutoConsoleVariable<bool> CVarLogVerticalZoomMultipliers(
	TEXT("CurveEditor.LogVerticalZoomMultipliers"),
	false,
	TEXT("Logs the zoom multipliers to make it easier for you to tweak FCurveEditorZoomScaleConfig::VerticalZoomScale")
	);
TAutoConsoleVariable<bool> CVarLogHorizontalZoomMultipliers(
	TEXT("CurveEditor.LogHorizontalZoomMultipliers"),
	false,
	TEXT("Logs the zoom multipliers to make it easier for you to tweak FCurveEditorZoomScaleConfig::HorizontalZoomScale")
	);

FCurveEditorDragOperation_Zoom::FCurveEditorDragOperation_Zoom(FCurveEditor* InCurveEditor, TSharedPtr<SCurveEditorView> InOptionalView)
	: CurveEditor(InCurveEditor)
	, OptionalView(MoveTemp(InOptionalView))
{}

void FCurveEditorDragOperation_Zoom::OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	FCurveEditorScreenSpaceH InputSpace  = OptionalView ? OptionalView->GetViewSpace() : CurveEditor->GetPanelInputSpace();

	ZoomFactor.X = InitialPosition.X / InputSpace.GetPhysicalWidth();
	OriginalInputRange = (InputSpace.GetInputMax() - InputSpace.GetInputMin());
	ZoomOriginX = InputSpace.GetInputMin() + OriginalInputRange * ZoomFactor.X;

	if (OptionalView)
	{
		FCurveEditorScreenSpaceV OutputSpace = OptionalView->GetViewSpace();

		ZoomFactor.Y = InitialPosition.Y / OutputSpace.GetPhysicalHeight();
		OriginalOutputRange = (OutputSpace.GetOutputMax() - OutputSpace.GetOutputMin());
		ZoomOriginY = OutputSpace.GetOutputMin() + OriginalOutputRange * (1.f - ZoomFactor.Y);
	}
}

void FCurveEditorDragOperation_Zoom::OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	constexpr double ClampRange = 1e9f;

	const FVector2D PixelDelta = CurrentPosition - InitialPosition;

	const FCurveEditorZoomScaleConfig& ZoomConfig = CurveEditor->GetZoomScaleConfig();

	// Zoom input range
	const FCurveEditorScreenSpaceH InputSpace  = OptionalView
		? static_cast<FCurveEditorScreenSpaceH>(OptionalView->GetViewSpace()) : CurveEditor->GetPanelInputSpace();

	const double InputZoomMultiplier = GetZoomMultiplier_InputAxis(-PixelDelta.X);
	const double DiffX = InputZoomMultiplier * PixelDelta.X / (InputSpace.GetPhysicalWidth() / OriginalInputRange);

	// This flips the horizontal zoom to match existing DCC tools
	const double NewInputRange = OriginalInputRange - DiffX;
	const double InputMin = FMath::Clamp<double>(ZoomOriginX - NewInputRange * ZoomFactor.X, -ClampRange, ClampRange);
	const double InputMax = FMath::Clamp<double>(ZoomOriginX + NewInputRange * (1.f - ZoomFactor.X), InputMin, ClampRange);
	const bool bExceedsHorizontalLimit = ZoomConfig.bLimitHorizontalZoomOut && ZoomConfig.MaxHorizontalZoomOut <= InputMax - InputMin;
	if (!bExceedsHorizontalLimit)
	{
		CurveEditor->GetBounds().SetInputBounds(InputMin, InputMax);
	}

	// Zoom output range
	if (OptionalView.IsValid())
	{
		FCurveEditorScreenSpaceV ViewSpace = OptionalView->GetViewSpace();

		const double OutputZoomMultiplier = GetZoomMultiplier_OutputAxis(PixelDelta.Y);
		const double DiffY = OutputZoomMultiplier * PixelDelta.Y / (ViewSpace.GetPhysicalHeight() / OriginalOutputRange);
		double NewOutputRange = OriginalOutputRange + DiffY;

		// If they're holding a shift they can scale on both axis at once, non-proportionally
		if (!MouseEvent.IsShiftDown())
		{
			// By default, do proportional zoom
			NewOutputRange = (NewInputRange / OriginalInputRange) * OriginalOutputRange;
		}

		const double OutputMin = FMath::Clamp<double>(ZoomOriginY - NewOutputRange * (1.f - ZoomFactor.Y), -ClampRange, ClampRange);
		const double OutputMax = FMath::Clamp<double>(ZoomOriginY + NewOutputRange * (ZoomFactor.Y), OutputMin, ClampRange);

		const bool bExceedsVerticalLimit = ZoomConfig.bLimitVerticalZoomOut && ZoomConfig.MaxVerticalZoomOut <= OutputMax - OutputMin;
		if (!bExceedsVerticalLimit)
		{
			OptionalView->SetOutputBounds(OutputMin, OutputMax);
		}
	}
}

double FCurveEditorDragOperation_Zoom::GetZoomMultiplier_InputAxis(double InMovedMouseX) const
{
	const FCurveEditorZoomScaleConfig& ZoomConfig = CurveEditor->GetZoomScaleConfig();
	const double Factor = ZoomConfig.EvalHorizontalZoom(InMovedMouseX);
	UE_CLOGF(CVarLogHorizontalZoomMultipliers.GetValueOnAnyThread(), LogCurveEditor, Log, "Horizontal Zoom: X: %f, Factor: %f", InMovedMouseX, Factor);
	return Factor;
}

double FCurveEditorDragOperation_Zoom::GetZoomMultiplier_OutputAxis(double InMovedMouseY) const
{
	const FCurveEditorZoomScaleConfig& ZoomConfig = CurveEditor->GetZoomScaleConfig();
	const double Factor = ZoomConfig.EvalVerticalZoom(InMovedMouseY);
	UE_CLOGF(CVarLogVerticalZoomMultipliers.GetValueOnAnyThread(), LogCurveEditor, Log, "Vertical Zoom: X: %f, Factor: %f", InMovedMouseY, Factor);
	return Factor;
}