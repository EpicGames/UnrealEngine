// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorDragOperation.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class SCurveEditorView;
struct FPointerEvent;

class FCurveEditorDragOperation_Zoom : public ICurveEditorDragOperation
{
public:
	
	FCurveEditorDragOperation_Zoom(FCurveEditor* InCurveEditor, TSharedPtr<SCurveEditorView> InOptionalView = nullptr);

	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent) override;

private:

	FCurveEditor* CurveEditor;
	TSharedPtr<SCurveEditorView> OptionalView;

	FVector2D ZoomFactor;
	double ZoomOriginX, ZoomOriginY;
	double OriginalInputRange, OriginalOutputRange;

	double GetZoomMultiplier_InputAxis(double InMovedMouseX) const;
	double GetZoomMultiplier_OutputAxis(double InMovedMouseY) const;
};
