// Copyright Epic Games, Inc. All Rights Reserved.

#include "ICurveEditorDragOperation.h"

class FSlateWindowElementList;
struct FCurvePointHandle;
struct FGeometry;
struct FPointerEvent;

void ICurveEditorDragOperation::MouseButtonDown(FVector2D InitialPosition, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	OnMouseButtonDown(InitialPosition, MyGeometry, MouseEvent);
}

void ICurveEditorDragOperation::MouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	OnMouseButtonUp(MyGeometry, MouseEvent);
}

void ICurveEditorDragOperation::BeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	OnBeginDrag(InitialPosition, CurrentPosition, MouseEvent);
}

void ICurveEditorDragOperation::Drag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	OnDrag(InitialPosition, CurrentPosition, MouseEvent);
}

void ICurveEditorDragOperation::FinishedPointerInput()
{
	OnFinishedPointerInput();
}

FReply ICurveEditorDragOperation::MouseWheel(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	return OnMouseWheel(InitialPosition, CurrentPosition, MouseEvent);
}

void ICurveEditorDragOperation::EndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
{
	OnEndDrag(InitialPosition, CurrentPosition, MouseEvent);
}

void ICurveEditorDragOperation::Paint(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId)
{
	OnPaint(AllottedGeometry, OutDrawElements, PaintOnLayerId);
}

void ICurveEditorDragOperation::CancelDrag()
{
	OnCancelDrag();
}

void ICurveEditorKeyDragOperation::Initialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& CardinalPoint)
{
	// TODO: maybe cache snap data for all selected curves?
	OnInitialize(InCurveEditor, CardinalPoint);
}