// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyDragOperationUtils.h"

#include "CurveEditor.h"
#include "CurveEditorDragOperation_MoveKeys.h"
#include "CurveEditorDragOperation_Tangent.h"
#include "CurveEditorTypes.h"

namespace UE::CurveEditor
{
TUniquePtr<ICurveEditorKeyDragOperation> CreateAndInitializeKeyDrag(
	FCurveEditor& InCurveEditor, ECurvePointType InKeyType, const TOptional<FCurvePointHandle>& InCardinalPoint
	)
{
	TUniquePtr<ICurveEditorKeyDragOperation> Result;
	switch (InKeyType)
	{
	case ECurvePointType::ArriveTangent:
	case ECurvePointType::LeaveTangent:
		Result = MakeUnique<FCurveEditorDragOperation_Tangent>();
		break;

	default:
		Result = MakeUnique<FCurveEditorDragOperation_MoveKeys>();
	}

	if (Result)
	{
		Result->Initialize(&InCurveEditor, InCardinalPoint);
	}
	
	return Result;
}

namespace Private
{
static FText CreateTransactionTitle(ECurvePointType InKeyType, const int32 InNumKeys)
{
	switch (InKeyType)
	{
	case ECurvePointType::ArriveTangent:
	case ECurvePointType::LeaveTangent:
		return FText::Format(NSLOCTEXT("CurveEditor", "DragTangentsFormat", "Drag {0}|plural(one=Tangent, other=Tangents)"), InNumKeys);

	default:
		return FText::Format(NSLOCTEXT("CurveEditor", "MoveKeysFormat", "Move {0}|plural(one=Key, other=Keys)"), InNumKeys);
	}
}
}

TUniquePtr<FScopedTransaction> CreateKeyOperationTransaction(ECurvePointType InKeyType, const int32 InNumKeys)
{
	// For now all operations create a transaction. In the future, you might add an operation that requires no transactions.
	// In that case, it's fine if you introduce some control flow here to return nullptr.
	return MakeUnique<FScopedTransaction>(Private::CreateTransactionTitle(InKeyType, InNumKeys));
}

TUniquePtr<FScopedTransaction> CreateKeyOperationTransaction(const FCurveEditor& InCurveEditor, ECurvePointType InKeyType)
{
	int32 NumKeys = 0;
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InCurveEditor.GetSelection().GetAll())
	{
		NumKeys += Pair.Value.Num();
	}
	return CreateKeyOperationTransaction(InKeyType, NumKeys);
}
}
