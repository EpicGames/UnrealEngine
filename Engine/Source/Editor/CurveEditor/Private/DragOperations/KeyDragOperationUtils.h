// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"

class FCurveEditor;
class FScopedTransaction;
class ICurveEditorKeyDragOperation;
enum class ECurvePointType : uint8;
struct FCurvePointHandle;

namespace UE::CurveEditor
{
/** Creates the drag operation based on InKeyType. */
TUniquePtr<ICurveEditorKeyDragOperation> CreateAndInitializeKeyDrag(
	FCurveEditor& InCurveEditor, ECurvePointType InKeyType, const TOptional<FCurvePointHandle>& InCardinalPoint
	);

/** Creates the transaction that the ICurveEditorKeyDragOperation created by CreateKeyDrag would use. */
TUniquePtr<FScopedTransaction> CreateKeyOperationTransaction(ECurvePointType InKeyType, const int32 InNumKeys);
/** Version that determines the number of selected keys. */
TUniquePtr<FScopedTransaction> CreateKeyOperationTransaction(const FCurveEditor& InCurveEditor, ECurvePointType InKeyType);
}
