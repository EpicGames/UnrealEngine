// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorBakeFilter.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorBakeFilter)


void UCurveEditorBakeFilter::InitializeFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor)
{
	if (!InCurveEditor->GetTimeSliderController())
	{
		return;
	}

	// There is a valid time slider controller, so we can use the display rate and tick resolution from it to define frames
	bUseSeconds = false;

	// Initialize Bake Interval and Custom Ranges if they haven't been initialized with these parameters
	const FFrameRate DisplayRate = InCurveEditor->GetTimeSliderController()->GetDisplayRate();
	const FFrameRate TickResolution = InCurveEditor->GetTimeSliderController()->GetTickResolution();

	if (DisplayRate == InitialDisplayRate && TickResolution == InitialTickResolution)
	{
		return;
	}

	InitialDisplayRate = DisplayRate;
	InitialTickResolution = TickResolution;

	TRange<FFrameNumber> PlayRange = InCurveEditor->GetTimeSliderController()->GetPlayRange();

	BakeInterval = FFrameRate::TransformTime(FFrameTime(1), DisplayRate, TickResolution).FrameNumber;
	CustomRange.Min = PlayRange.GetLowerBoundValue();
	CustomRange.Max = PlayRange.GetUpperBoundValue();
}

bool UCurveEditorBakeFilter::CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor)
{
	return bCustomRangeOverride || InCurveEditor->GetSelection().Count() > 0 || InCurveEditor->GetCurves().Num() > 0;
}

void UCurveEditorBakeFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;

	TArray<FKeyPosition> NewKeyPositions;
	TArray<FKeyAttributes> NewKeyAttributes;

	const FFrameRate DisplayRate = InCurveEditor->GetTimeSliderController() ? InCurveEditor->GetTimeSliderController()->GetDisplayRate() : FFrameRate();
	const FFrameRate TickResolution = InCurveEditor->GetTimeSliderController() ? InCurveEditor->GetTimeSliderController()->GetTickResolution() : FFrameRate();

	// @todo: This code is a bit convoluted because the filter API is set up to only ever operate on key selections,
	//        which needs correcting.
	TMap<FCurveModelID, FKeyHandleSet> LocalKeysToOperateOn;
	const TMap<FCurveModelID, FKeyHandleSet>* KeysToOperateOn = &InKeysToOperateOn;

	if (InKeysToOperateOn.Num() == 0 && bCustomRangeOverride)
	{
		KeysToOperateOn = &LocalKeysToOperateOn;
		for (const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : InCurveEditor->GetCurves())
		{
			LocalKeysToOperateOn.Add(CurvePair.Key);
		}
	}

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : *KeysToOperateOn)
	{
		FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);

		if (!Curve)
		{
			continue;
		}

		// Look at our Input Snap rate to determine how far apart the baked keys are.
		FFrameRate BakeRate = InCurveEditor->GetCurveSnapMetrics(Pair.Key).InputSnapRate;

		KeyHandles.Reset(Pair.Value.Num());
		KeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

		// Get all the selected keys
		SelectedKeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);

		// Find the hull of the range of the selected keys
		double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();

		// Hull of the sampling, maybe different than the hull of the keys
		double SampleMinKey = MinKey, SampleMaxKey = MaxKey;

		//we want the first key to be on whole frame if it's not we delete it
		bool bDeleteFirstKey = false;
		//similarly if the last frame is not on a whole key it may be out of range and also needs to be deleted.
		bool bDeleteLastKey = false;
		FFrameTime MinInDisplayFrames(0, 0.0);
		FFrameTime MaxInDisplayFrames(0, 0.0);

		if (bCustomRangeOverride)
		{
			//with custom range we just wipe everything out
			bDeleteFirstKey = true;
			bDeleteLastKey = true;
			if (bUseSeconds)
			{
				MinKey = CustomRangeMinInSeconds;
				MaxKey = CustomRangeMaxInSeconds;
			}
			else
			{
				MinInDisplayFrames = FFrameRate::TransformTime(CustomRange.Min, TickResolution, DisplayRate);
				MaxInDisplayFrames = FFrameRate::TransformTime(CustomRange.Max, TickResolution, DisplayRate);
				MinKey = DisplayRate.AsSeconds(MinInDisplayFrames);
				MaxKey = DisplayRate.AsSeconds(MaxInDisplayFrames);
			}

			if (MaxKey < MinKey)
			{
				const double TempMinKey = MinKey;
				MinKey = MaxKey;
				MaxKey = TempMinKey;
			}
			SampleMinKey = MinKey;
			SampleMaxKey = MaxKey;
		}
		else
		{
			for (FKeyPosition Key : SelectedKeyPositions)
			{
				SampleMinKey = MinKey = FMath::Min(Key.InputValue, MinKey);
				SampleMaxKey = MaxKey = FMath::Max(Key.InputValue, MaxKey);
			}
			MinInDisplayFrames = DisplayRate.AsFrameTime(SampleMinKey);
			if(MinInDisplayFrames.GetSubFrame() > 0.0)
			{
				bDeleteFirstKey = true;
				MinInDisplayFrames = FFrameTime(MinInDisplayFrames.RoundToFrame(), 0.0);
				SampleMinKey = DisplayRate.AsSeconds(MinInDisplayFrames);
			}
			MaxInDisplayFrames = DisplayRate.AsFrameTime(SampleMaxKey);
			if (MaxInDisplayFrames.GetSubFrame() > 0.0)
			{
				bDeleteLastKey = true;
				MaxInDisplayFrames = FFrameTime(MaxInDisplayFrames.RoundToFrame(), 0.0);
				SampleMaxKey = DisplayRate.AsSeconds(MaxInDisplayFrames);
			}
		}

		// Get all keys that exist between the time range
		KeyHandles.Reset();
		Curve->GetKeys(MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);

		// Determine new times for new keys
		
		double Interval = bUseSeconds ? BakeIntervalInSeconds : DisplayRate.AsSeconds(FFrameRate::TransformTime(BakeInterval, TickResolution, DisplayRate));

		int32 KeyIndex = 0;
		if (bDeleteFirstKey == false)
		{
			++KeyIndex;
		}

		int32 NumKeysToAdd = FMath::RoundToInt((SampleMaxKey - SampleMinKey) / Interval) + 1;
	
		NewKeyPositions.Reset(NumKeysToAdd);
		NewKeyAttributes.Reset(NumKeysToAdd);

		// Get the default interpolation type at the first frame, use this for all of the new keys.
		TPair<ERichCurveInterpMode, ERichCurveTangentMode> Modes = Curve->GetInterpolationMode(MinKey, ERichCurveInterpMode::RCIM_Cubic, ERichCurveTangentMode::RCTM_SmartAuto);
		FKeyAttributes DefaultKeyAttribute;
		DefaultKeyAttribute.SetInterpMode(Modes.Key);
		DefaultKeyAttribute.SetTangentMode(Modes.Value);
		
		for (; KeyIndex < NumKeysToAdd; ++KeyIndex)
		{
			double NewTime = SampleMinKey + KeyIndex * Interval;
			if (NewTime <= SampleMaxKey)
			{
				FKeyPosition CurrentKey(NewTime, 0.0);
				if (Curve->Evaluate(CurrentKey.InputValue, CurrentKey.OutputValue))
				{
					NewKeyPositions.Add(CurrentKey);
					NewKeyAttributes.Add(DefaultKeyAttribute);
				}
			}
			else
			{
				break;
			}
		}

		// We want to store the newly added ones so we can add them to the users selection to mimic
		// their selection pre-baking.
		TArray<TOptional<FKeyHandle>> NewKeyHandles;
		NewKeyHandles.SetNumUninitialized(NewKeyPositions.Num());

		TArrayView<TOptional<FKeyHandle>> NewKeyHandlesView = MakeArrayView(NewKeyHandles);
		FKeyHandleSet& OutHandleSet = OutKeysToSelect.Add(Pair.Key);

		// If not deleting, manually add the first and last keys to the out set before we remove them below.
		if (bDeleteFirstKey == false && KeyHandles.Num() > 0)
		{
			OutHandleSet.Add(KeyHandles[0], ECurvePointType::Key);
		}
		if (bDeleteLastKey == false && KeyHandles.Num() > 1)
		{
			OutHandleSet.Add(KeyHandles.Last(0), ECurvePointType::Key);
		}

		// Below is true except if they are on subframes, we need to delete them
		// We need to leave the first and the last key of the selection alone for two reasons;
		// 1. Undo/Redo works better as selections aren't transacted so they don't handle keys being removed/re-added.
		// by leaving the keys alone, they survive undo/redo operations.
		// 2. With high interval settings, the shape of the curve can change drastically because the next interval would fall right
		// outside the last key. If we remove the keys, then the shape changes in these cases. We avoid putting a new key on the last
		// key (if it exists), to avoid any duplication. This preserves the shape of the curves better in most test cases.
		if (bDeleteFirstKey == false && KeyHandles.Num() > 0)
		{
			KeyHandles.RemoveAt(0);
		}
		if (bDeleteLastKey == false && KeyHandles.Num() > 1)
		{
			KeyHandles.RemoveAt(KeyHandles.Num() - 1);
		}


		// Remove all the old in-between keys and add the new ones
		Curve->RemoveKeys(KeyHandles,0.0);
		Curve->AddKeys(NewKeyPositions, NewKeyAttributes, &NewKeyHandlesView);

		for (const TOptional<FKeyHandle>& Handle : NewKeyHandlesView)
		{
			if (Handle.IsSet())
			{
				OutHandleSet.Add(Handle.GetValue(), ECurvePointType::Key);
			}
		}
	}
}
