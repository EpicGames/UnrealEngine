// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorBakeFilterCustomization.h"

#include "Filters/CurveEditorBakeFilter.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

void FCurveEditorBakeFilterCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	if (DetailObjects.Num() == 0)
	{
		return;
	}

	bool bUseSeconds = true;
	for (TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		UCurveEditorBakeFilter* BakeFilter = CastChecked<UCurveEditorBakeFilter>(DetailObject.Get());
		if (BakeFilter)
		{
			bUseSeconds = BakeFilter->bUseSeconds;
		}
	}

	if (bUseSeconds)
	{
		TSharedPtr<IPropertyHandle> BakeIntervalPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCurveEditorBakeFilter, BakeInterval));
		DetailBuilder.HideProperty(BakeIntervalPropertyHandle);

		TSharedPtr<IPropertyHandle> CustomRangePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCurveEditorBakeFilter, CustomRange));
		DetailBuilder.HideProperty(CustomRangePropertyHandle);
	}
	else
	{
		TSharedPtr<IPropertyHandle> BakeIntervalInSecondsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCurveEditorBakeFilter, BakeIntervalInSeconds));
		DetailBuilder.HideProperty(BakeIntervalInSecondsPropertyHandle);

		TSharedPtr<IPropertyHandle> CustomRangeMinInSecondsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCurveEditorBakeFilter, CustomRangeMinInSeconds));
		DetailBuilder.HideProperty(CustomRangeMinInSecondsPropertyHandle);

		TSharedPtr<IPropertyHandle> CustomRangeMaxInSecondsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCurveEditorBakeFilter, CustomRangeMaxInSeconds));
		DetailBuilder.HideProperty(CustomRangeMaxInSecondsPropertyHandle);
	}
}
