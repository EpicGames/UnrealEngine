// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class UCurveEditorBakeFilter;

class IPropertyHandle;

/**
 * Customizes a UCurveEditorBakeFilter
 */
class FCurveEditorBakeFilterCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCurveEditorBakeFilterCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface
};