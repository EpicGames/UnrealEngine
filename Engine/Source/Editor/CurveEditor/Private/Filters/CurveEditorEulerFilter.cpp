// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorEulerFilter.h"

#include "Algo/AnyOf.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorEulerFilter)

void UCurveEditorEulerFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyHandle> KeyHandlesToModify;
	TArray<FKeyPosition> SelectedKeyPositions;

	TArray<FKeyPosition> NewKeyPositions;

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		// first check if curve exists
		FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);
		if (!Curve)
		{
			continue;
		}

		// then check if curve is for rotation value
		if (!Curve->GetIntentionName().Contains("Rotation"))
		{
			continue;
		}

		KeyHandles.Reset(Pair.Value.Num());
		KeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

		// Get all the selected keys
		SelectedKeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);

		// Find the hull of the range of the selected keys
		double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();
		for (FKeyPosition Key : SelectedKeyPositions)
		{
			MinKey = FMath::Min(Key.InputValue, MinKey);
			MaxKey = FMath::Max(Key.InputValue, MaxKey);
		}

		// Get all keys that exist between the time range
		KeyHandles.Reset();
		Curve->GetKeys(MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);

		if (KeyHandles.Num() >= 2)
		{
			// get the selected key positions
			SelectedKeyPositions.SetNum(KeyHandles.Num());
			Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);
			
			// reset the old calculated key positions
			NewKeyPositions.Reset();
			KeyHandlesToModify.Reset();

			for (int32 KeyIndex = 0; KeyIndex < KeyHandles.Num() - 1; ++KeyIndex)
			{
				// calculate the euler-filtered key
				float NextKeyVal = SelectedKeyPositions[KeyIndex + 1].OutputValue;
				FMath::WindRelativeAnglesDegrees(SelectedKeyPositions[KeyIndex].OutputValue, NextKeyVal);
				SelectedKeyPositions[KeyIndex + 1].OutputValue = NextKeyVal;
				
				// create the new key position and add it and its handle to their respective lists
				FKeyPosition FilteredKey(SelectedKeyPositions[KeyIndex + 1].InputValue, NextKeyVal);
				NewKeyPositions.Add(FilteredKey);
				KeyHandlesToModify.Add(KeyHandles[KeyIndex + 1]);
			}

			Curve->SetKeyPositions(KeyHandlesToModify, NewKeyPositions);
		}
	}
	OutKeysToSelect = InKeysToOperateOn;
}

bool UCurveEditorEulerFilter::CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor)
{
	const TMap<FCurveModelID, FKeyHandleSet>& Selection = InCurveEditor->GetSelection().GetAll();
	if (!Selection.IsEmpty())
	{
		return Algo::AnyOf(Selection,
			[&InCurveEditor](const TPair<FCurveModelID, FKeyHandleSet>& Pair)
			{
				const FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);
				return Curve && Curve->GetIntentionName().Contains("Rotation");
			});
	}
	// No selection — allow if any visible curve is a rotation curve
	return Algo::AnyOf(InCurveEditor->GetCurves(),
		[](const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair)
		{
			return CurvePair.Value && CurvePair.Value->GetIntentionName().Contains("Rotation");
		});
}
