// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorFilterBase.h"

#include "CurveEditor.h"
#include "Internationalization/Text.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorFilterBase)

#define LOCTEXT_NAMESPACE "CurveEditorFilterBase"

FText UCurveEditorFilterBase::GetLabel(const TSubclassOf<UCurveEditorFilterBase>& InClass)
{
	const FString* Label = InClass ? InClass->FindMetaData(TEXT("CurveEditorLabel")) : nullptr;
	if (!Label || !ensure(!InClass->GetName().IsEmpty()))
	{
		return LOCTEXT("Filter.LabelEmpty", "Filter");
	}
	
	return FText::AsLocalizable_Advanced(
		TEXT("CurveEditorFilterBase"),
		*FString::Printf(TEXT("Filter.%s.Label"), *InClass->GetName()),
		*Label
		);
}

FText UCurveEditorFilterBase::GetDescription(const TSubclassOf<UCurveEditorFilterBase>& InClass)
{
	const FString* Description = InClass ? InClass->FindMetaData(TEXT("CurveEditorDescription")) : nullptr;
	if (!Description || !ensure(!InClass->GetName().IsEmpty()))
	{
		return FText::GetEmpty();
	}
	
	return FText::AsLocalizable_Advanced(
		TEXT("CurveEditorFilterBase"),
		*FString::Printf(TEXT("Filter.%s.Description"), *InClass->GetName()),
		*Description
		);
}

FSlateIcon UCurveEditorFilterBase::GetIcon(const TSubclassOf<UCurveEditorFilterBase>& InClass)
{
	return FSlateIconFinder::FindIconForClass(InClass.Get());
}

bool UCurveEditorFilterBase::CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor)
{
	return InCurveEditor->GetSelection().Count() > 0 || InCurveEditor->GetCurves().Num() > 0;
}

#undef LOCTEXT_NAMESPACE
