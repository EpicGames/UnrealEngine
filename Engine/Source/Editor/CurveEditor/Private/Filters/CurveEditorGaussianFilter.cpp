// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorGaussianFilter.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/CurveEvaluation.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorGaussianFilter)


TArray<double> UCurveEditorGaussianFilter::CalculateKernel(int32 InKernelWidth)
{
	const int32 KernelWidth = ((InKernelWidth / 2) * 2) + 1; //make sure it's odd
	double Sigma = ((double)(KernelWidth) - 1.0) / 6.0; //99% is 6sigma
	TArray<double> Kernel;
	Kernel.SetNum(KernelWidth);
	int32 Mid = (KernelWidth / 2);
	int32 Index = 0;
	for (int32 KernelIndex = -Mid; KernelIndex < Mid + 1; ++KernelIndex)
	{
		Kernel[Index] = (1.0 / (Sigma * FMath::Sqrt(2.0 * PI))) * (1.0 / (FMath::Exp((KernelIndex * KernelIndex) / (2.0 * Sigma * Sigma))));
		++Index;
	}
	return Kernel;
}

double UCurveEditorGaussianFilter::ApplyKernel(int32 PositionIndex,const TArray<double> Kernel, const TArray<FKeyPosition>& KeyPositions)
{
	const int32 StartKernel = - Kernel.Num() / 2;
	const int32 EndKernel = StartKernel + Kernel.Num();
	double TotalWeight = 0.0;
	double Sum = 0.0;
	int32 KernelIndex = 0;
	for (int32 KernelLocation = StartKernel; KernelLocation < EndKernel; ++KernelLocation)
	{
		int32 Index = KernelLocation + PositionIndex;
		if (Index < 0)
		{
			Index = 0;
		}
		else if (Index >= KeyPositions.Num())
		{
			Index = KeyPositions.Num() - 1;
		}
		TotalWeight += Kernel[KernelIndex];
		Sum += (Kernel[KernelIndex] * KeyPositions[Index].OutputValue);

		++KernelIndex;
	}
	if (TotalWeight < 0.9999 && TotalWeight > 0.0)
	{
		Sum *= (1.0 / TotalWeight);
	}
	return Sum;
}

void UCurveEditorGaussianFilter::Gaussian(FCurveModel* Curve, const FGaussianParams& InParams, const TOptional<FKeyHandleSet>& KeyHandleSet,
	FKeyHandleSet& OutHandleSet)
{
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;
	double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();

	//if key's are set use that to find range to reduce otherwise use full range.
	if (KeyHandleSet.IsSet())
	{
		KeyHandles.Reset(KeyHandleSet.GetValue().Num());
		KeyHandles.Append(KeyHandleSet.GetValue().AsArray().GetData(), KeyHandleSet.GetValue().Num());
		//assume in order todo check
		SelectedKeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);
		// Find the hull of the range of the selected keys
		for (FKeyPosition Key : SelectedKeyPositions)
		{
			MinKey = FMath::Min(Key.InputValue, MinKey);
			MaxKey = FMath::Max(Key.InputValue, MaxKey);
		}
	}
	else //get all keys
	{
		Curve->GetTimeRange(MinKey, MaxKey);
	}

	// Get all keys that exist between the time range
	KeyHandles.Reset();
	Curve->GetKeys(MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);
	if (KeyHandles.Num() <= 2)
	{
		return;
	}
	
	TArray<FKeyPosition> KeyPositions, NewKeyPositions;
	KeyPositions.SetNum(KeyHandles.Num());
	Curve->GetKeyPositions(KeyHandles, KeyPositions);
	NewKeyPositions = KeyPositions;
	TArray<double> Kernel = CalculateKernel(InParams.KernelWidth);
	for (int32 Index = 0; Index < KeyPositions.Num(); ++Index)
	{
		NewKeyPositions[Index].OutputValue = ApplyKernel(Index, Kernel, KeyPositions);
	}
	Curve->SetKeyPositions(KeyHandles, NewKeyPositions);
	for (const FKeyHandle& NewHandle : KeyHandles)
	{
		OutHandleSet.Add(NewHandle, ECurvePointType::Key);
	}
}

void UCurveEditorGaussianFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	OutKeysToSelect.Reset();
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);
		if (!Curve)
		{
			continue;
		}
		FKeyHandleSet& OutHandleSet = OutKeysToSelect.Add(Pair.Key);

		TOptional<FKeyHandleSet> KeyHandleSet = Pair.Value;
		Gaussian(Curve, GaussianParams, KeyHandleSet, OutHandleSet);
	}
}
