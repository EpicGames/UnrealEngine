// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorReduceFilter.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/CurveEvaluation.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorReduceFilter)

static float EvalForTwoKeys(const FKeyPosition& Key1Pos, const FKeyAttributes& Key1Attrib,
	const FKeyPosition& Key2Pos, const FKeyAttributes& Key2Attrib,
	const float InTime)
{
	const float Diff = Key2Pos.InputValue - Key1Pos.InputValue;

	if (Diff > 0.f && Key1Attrib.HasInterpMode() && Key1Attrib.GetInterpMode() != RCIM_Constant)
	{
		const float Alpha = (InTime - Key1Pos.InputValue) / Diff;
		const float P0 = Key1Pos.OutputValue;
		const float P3 = Key2Pos.OutputValue;

		if (Key1Attrib.GetInterpMode() == RCIM_Linear)
		{
			return FMath::Lerp(P0, P3, Alpha);
		}
		else
		{
			const float OneThird = 1.0f / 3.0f;
			const float P1 = Key1Attrib.HasLeaveTangent() ? P0 + (Key1Attrib.GetLeaveTangent() * Diff*OneThird) : P0;
			const float P2 = Key2Attrib.HasArriveTangent() ? P3 - (Key2Attrib.GetArriveTangent() * Diff*OneThird) : P3;

			return UE::Curves::BezierInterp(P0, P1, P2, P3, Alpha);
		}
	}
	else
	{
		return Key1Pos.OutputValue;
	}
}

void UCurveEditorReduceFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;
	TArray<FKeyAttributes> SelectedKeyAttributes;

	FFrameRate BakeRate;

	// Since we only remove keys we'll just copy the whole set and then remove the handles as well.
	OutKeysToSelect = InKeysToOperateOn;

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);
		if (!Curve)
		{
			continue;
		}

		BakeRate = InCurveEditor->GetCurveSnapMetrics(Pair.Key).InputSnapRate;

		KeyHandles.Reset(Pair.Value.Num());
		KeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

		// Get all the selected keys
		SelectedKeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);

		// Find the hull of the range of the selected keys
		double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();
		for (FKeyPosition Key : SelectedKeyPositions)
		{
			MinKey = FMath::Min(Key.InputValue, MinKey);
			MaxKey = FMath::Max(Key.InputValue, MaxKey);
		}

		// Get all keys that exist between the time range
		KeyHandles.Reset();
		Curve->GetKeys( MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);
		if (KeyHandles.Num() > 2) //need at least 3 keys to reduce
		{
			SelectedKeyPositions.SetNum(KeyHandles.Num());
			Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);
			SelectedKeyAttributes.SetNum(KeyHandles.Num());
			Curve->GetKeyAttributes(KeyHandles, SelectedKeyAttributes);

			FKeyHandleSet& OutHandleSet = OutKeysToSelect.FindOrAdd(Pair.Key);
			int32 MostRecentKeepKeyIndex = 0;
			TArray<FKeyHandle> KeysToRemove;
			for (int32 TestIndex = 1; TestIndex < KeyHandles.Num() - 1; ++TestIndex)
			{
				bool bKeepKey = false;				
				if (bTryRemoveUserSetTangentKeys && SampleRate.IsValid())
				{
					const FFrameTime KeyOneFrameTime = SampleRate.AsFrameTime(SelectedKeyPositions[MostRecentKeepKeyIndex].InputValue);
					const FFrameTime KeyTwoFrameTime = SampleRate.AsFrameTime(SelectedKeyPositions[TestIndex + 1].InputValue);

					FFrameTime SampleFrameTime = KeyOneFrameTime;
					const double ToRemoveKeyTime = SelectedKeyPositions[TestIndex].InputValue;

					// Sample curve at intervals, dictated by provided sample-rate, between the keys surrounding the to-be-removed key
					// this makes sure that any impact on non-lerp keys is taken into account
					while (SampleFrameTime <= KeyTwoFrameTime)
					{
						const double SampleSeconds = SampleRate.AsSeconds(SampleFrameTime);
						const float ValueWithoutKey = EvalForTwoKeys(SelectedKeyPositions[MostRecentKeepKeyIndex],	SelectedKeyAttributes[MostRecentKeepKeyIndex],
							SelectedKeyPositions[TestIndex + 1], SelectedKeyAttributes[TestIndex + 1],
							SampleSeconds);

						const FKeyPosition& KeyOne = SampleSeconds <= ToRemoveKeyTime ? SelectedKeyPositions[MostRecentKeepKeyIndex] : SelectedKeyPositions[TestIndex];
						const FKeyPosition& KeyTwo = SampleSeconds <= ToRemoveKeyTime ? SelectedKeyPositions[TestIndex] : SelectedKeyPositions[TestIndex + 1];
						
						const FKeyAttributes& AttributesOne = SampleSeconds <= ToRemoveKeyTime ? SelectedKeyAttributes[MostRecentKeepKeyIndex] : SelectedKeyAttributes[TestIndex];
						const FKeyAttributes& AttributesTwo = SampleSeconds <= ToRemoveKeyTime ? SelectedKeyAttributes[TestIndex] : SelectedKeyAttributes[TestIndex + 1];
						
						const float ValueWithKey = EvalForTwoKeys(KeyOne, AttributesOne,KeyTwo, AttributesTwo, SampleSeconds);
				
						if (FMath::Abs(ValueWithoutKey - ValueWithKey) > Tolerance)
						{
							bKeepKey = true;
							break;
						}

						// Next frame
						SampleFrameTime.FrameNumber += 1;	
					}	
					
				}
				else
				{
					auto IsNonUserTangentKey = [](const FKeyAttributes& KeyAttributes)
					{
						return !KeyAttributes.HasTangentMode() ||
							(KeyAttributes.GetTangentMode() != RCTM_User && KeyAttributes.GetTangentMode() != RCTM_Break);
					};
					
					bKeepKey = true;

					// Only try to remove keys which have automatic tangent data
					if (IsNonUserTangentKey(SelectedKeyAttributes[MostRecentKeepKeyIndex]) &&
						IsNonUserTangentKey(SelectedKeyAttributes[TestIndex]) &&
						IsNonUserTangentKey(SelectedKeyAttributes[TestIndex + 1]))
					{
						const float KeyValue = SelectedKeyPositions[TestIndex].OutputValue;
						const float ValueWithoutKey = EvalForTwoKeys(SelectedKeyPositions[MostRecentKeepKeyIndex], SelectedKeyAttributes[MostRecentKeepKeyIndex],
							SelectedKeyPositions[TestIndex + 1], SelectedKeyAttributes[TestIndex + 1],
							SelectedKeyPositions[TestIndex].InputValue);

						// Check if there is a great enough change in value to consider this key needed.
						bKeepKey = FMath::Abs(ValueWithoutKey - KeyValue) > Tolerance;
					}
				}

				if (bKeepKey)
				{
					MostRecentKeepKeyIndex = TestIndex;
				}
				else
				{
					KeysToRemove.Add(KeyHandles[TestIndex]);
					OutHandleSet.Remove(KeyHandles[TestIndex], ECurvePointType::Key);
				}
			}
			Curve->RemoveKeys(KeysToRemove,0.0);
		}
	}
}
