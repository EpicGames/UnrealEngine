// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorSmartReduceFilter.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/CurveEvaluation.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"


// cache of times and positions based on the frame rate.

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorSmartReduceFilter)
struct FCurveCache
{
	double Step = 0.1;
	double Min = TNumericLimits<double>::Max();
	double Max = TNumericLimits<double>::Lowest();

	TArray<double> Times;
	TArray<double> Positions;

	double Evaluate(double Time)
	{	
		if (Positions.Num() <= 0)
		{
			return 0.0;
		}
		int32 Index = GetIndex(Time);
		return Positions[Index];
	}
	int32 GetIndex(double Time)
	{
		int32 Index = (int32)(((Time - Times[0]) / Step) + 0.5);
		if (Index < 0)
		{
			Index = 0;
		}
		else if (Index >= Positions.Num())
		{
			Index = Positions.Num();
		}
		return Index;
	}
};

// Add a key at the bisection of the two key times(indices). We use cache indices to keep things on frames.
// Doing a bisection will result in more accurate results with less keys then doing forward tolerance checks.
static void Bisect(FCurveModel* Curve, FCurveCache& Cache, double Tolerance, const FKeyAttributes& NewDefaultAttribute,
	int32 StartCacheIndex, int32 EndCacheIndex, TArray<FKeyHandle>& HandlesAdded)
{
	if (EndCacheIndex > (StartCacheIndex +1))
	{
		double CurveValue = 0.0, CacheValue = 0.0;
		const int32 EvalIndex = (int32)((double(EndCacheIndex + StartCacheIndex)) * 0.5);
		const double EvalTime = Cache.Times[EvalIndex];
		if (Curve->Evaluate(EvalTime, CurveValue))
		{
			CacheValue = Cache.Evaluate(EvalTime);
			const double Diff = FMath::Abs(CurveValue - CacheValue);
			if (Diff > Tolerance)
			{
				FKeyPosition KeyPosition(EvalTime, CacheValue);
				TOptional<FKeyHandle> NewHandle = Curve->AddKey(KeyPosition, NewDefaultAttribute);
				if (NewHandle.IsSet())
				{
					HandlesAdded.Add(NewHandle.GetValue());
				}
			}
			Bisect(Curve, Cache, Tolerance, NewDefaultAttribute, StartCacheIndex, EvalIndex, HandlesAdded);
			Bisect(Curve, Cache, Tolerance, NewDefaultAttribute, EvalIndex, EndCacheIndex, HandlesAdded);
		}
	}
};

static bool IsUserSpecifiedTangentKey(const FKeyAttributes& KeyAttributes)
{
	return KeyAttributes.HasTangentMode() &&
		(KeyAttributes.GetInterpMode() == RCIM_Cubic &&
			(KeyAttributes.GetTangentMode() != RCTM_Auto && KeyAttributes.GetTangentMode() != RCTM_SmartAuto));
};

static void Difference(const TArray<double>& Input, TArray<double>& Output)
{
	if (Input.Num() > 1)
	{
		Output.SetNum(Input.Num());
		for (int32 Index = 0; Index < Input.Num() - 1; ++Index)
		{
			Output[Index] = Input[Index + 1] - Input[Index];
		}
		Output[Output.Num() - 1] = Output[Output.Num() - 2];
	}

}
//Will key reduce over the RANGE of the key handles if they are set
//Will first find all keys that are cubic and non-auto over that set and save them as keys to keep that way we keep custom tangents,
//unless bNeedToTestExisting is set to false since we know there will be no custom tangents, like if we just did a bake.
//We then find keys that change velocities(e.g. they are peaks or valleys),
//Then finally we do bisections over those intervals to finiush up the key reduction
void UCurveEditorSmartReduceFilter::SmartReduce(FCurveModel* Curve, const FSmartReduceParams& InParams, const TOptional<FKeyHandleSet>& KeyHandleSet,
	const bool bNeedToTestExisting, FKeyHandleSet& OutHandleSet)
{

	// bOnlyReduceStaticChannels: skip curves whose value range is not flat
	if (InParams.bOnlyReduceStaticChannels)
	{
		double MinValue, MaxValue;
		Curve->GetValueRange(MinValue, MaxValue);
		if (!FMath::IsNearlyEqual(MinValue, MaxValue, UE_KINDA_SMALL_NUMBER))
		{
			return;
		}
	}

	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;
	double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();

	//if key's are set use that to find range to reduce otherwise use full range.
	if (KeyHandleSet.IsSet())
	{
		KeyHandles.Reset(KeyHandleSet.GetValue().Num());
		KeyHandles.Append(KeyHandleSet.GetValue().AsArray().GetData(), KeyHandleSet.GetValue().Num());
		//assume in order todo check
		SelectedKeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, SelectedKeyPositions);
		// Find the hull of the range of the selected keys
		for (FKeyPosition Key : SelectedKeyPositions)
		{
			MinKey = FMath::Min(Key.InputValue, MinKey);
			MaxKey = FMath::Max(Key.InputValue, MaxKey);
		}
	}
	else //get all keys
	{
		Curve->GetTimeRange(MinKey, MaxKey);
	}

	// Get all keys that exist between the time range
	KeyHandles.Reset();
	Curve->GetKeys(MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), KeyHandles);
	if (KeyHandles.Num() <= 2)
	{
		return;
	}
	
	TMap<FFrameNumber, TPair<FKeyPosition, FKeyAttributes>> KeysToSet;
	TArray<FKeyPosition> NewPositions;
	TArray<FKeyAttributes> NewAttributes;
	FKeyPosition KeyPosition;
	bool bKeptFirstKey = false, bKeptLastKey = false;
	TArray<double> KeyTimes;
	TArray<TPair<FKeyPosition, FKeyAttributes>> KeyPosAttrs;
	if (bNeedToTestExisting)
	{
		TSortedMap<double, TPair<FKeyPosition, FKeyAttributes>> KeysToKeep;
		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNum(KeyHandles.Num());
		Curve->GetKeyPositions(KeyHandles, KeyPositions);
		TArray<FKeyAttributes> KeyAttributes;
		KeyAttributes.SetNum(KeyHandles.Num());
		Curve->GetKeyAttributes(KeyHandles, KeyAttributes);

		for (int32 Index = 0; Index < KeyAttributes.Num(); ++Index)
		{
			if (IsUserSpecifiedTangentKey(KeyAttributes[Index]))
			{
				if (Index == 0)
				{
					bKeptFirstKey = true;
				}
				else if (Index == KeyAttributes.Num() - 1)
				{
					bKeptLastKey = true;
				}
				TPair<FKeyPosition, FKeyAttributes> PosAttribute(KeyPositions[Index], KeyAttributes[Index]);
				KeysToKeep.Add(KeyPositions[Index].InputValue, PosAttribute);
			}
		}
		KeysToKeep.GenerateKeyArray(KeyTimes);
		KeysToKeep.GenerateValueArray(KeyPosAttrs);
	}

	FCurveCache Cache;
	Cache.Step = InParams.SampleRate.AsInterval();
	Cache.Min = TNumericLimits<double>::Max();
	Cache.Max = TNumericLimits<double>::Lowest();
	double Value = 0.0;

	for (double Time = MinKey; Time <= MaxKey; Time += Cache.Step)
	{
		Cache.Times.Add(Time);
		Curve->Evaluate(Time, Value);
		Cache.Positions.Add(Value);
		if (Value < Cache.Min)
		{
			Cache.Min = Value;
		}
		if (Value > Cache.Max)
		{
			Cache.Max = Value;
		}
	}

	TArray<double> Velocities;
	Difference(Cache.Positions, Velocities);
	Curve->RemoveKeys(KeyHandles, 0.0);
	FKeyAttributes NewDefaultAttribute;
	NewDefaultAttribute.SetInterpMode(ERichCurveInterpMode::RCIM_Cubic);
	NewDefaultAttribute.SetTangentMode(ERichCurveTangentMode::RCTM_SmartAuto);

	if (bKeptFirstKey == false)
	{
		KeyPosition.InputValue = Cache.Times[0];
		KeyPosition.OutputValue = Cache.Positions[0];
		NewPositions.Add(KeyPosition);
		NewAttributes.Add(NewDefaultAttribute);
	}
	int32 KeyIndex = 0;
	double PrevVelocitySign = FMath::Sign(Velocities[0]);
	double PrevPosition = Cache.Positions[0];
	// go through all of the cached times, if we have keys that we need to keep(non auto's) we add them 
	//otherwise we add a default auto based upon if it changes velocity sign or if it fails the difference threshold via bisection
	for (int32 Index = 1; Index < Cache.Positions.Num()-1; ++Index)
	{
		bool bKeyWasAdded = false;
		if (KeyIndex < KeyPosAttrs.Num())
		{
			while (KeyIndex < KeyTimes.Num() && (KeyTimes[KeyIndex] <= (Cache.Times[Index] + UE_DOUBLE_SMALL_NUMBER)))
			{
				KeyPosition.InputValue = KeyPosAttrs[KeyIndex].Key.InputValue;
				KeyPosition.OutputValue = KeyPosAttrs[KeyIndex].Key.OutputValue;
				NewPositions.Add(KeyPosition);
				NewAttributes.Add(KeyPosAttrs[KeyIndex].Value);
				if (FMath::IsNearlyEqual(KeyTimes[KeyIndex], Cache.Times[Index]))
				{
					bKeyWasAdded = true;
				}
				++KeyIndex;
			}
		}
		
		const double VelocitySign = FMath::Sign(Velocities[Index]);
		if (bKeyWasAdded == false && (VelocitySign != PrevVelocitySign && FMath::IsNearlyEqual(Cache.Positions[Index],PrevPosition) == false))
		{ 
			KeyPosition.InputValue = Cache.Times[Index];
			KeyPosition.OutputValue = Cache.Positions[Index];
			NewPositions.Add(KeyPosition);
			NewAttributes.Add(NewDefaultAttribute);
		}
		PrevVelocitySign = VelocitySign;
		PrevPosition = Cache.Positions[Index];
	}

	if (bKeptLastKey == false)
	{
		KeyPosition.InputValue = Cache.Times[Cache.Positions.Num() - 1];
		KeyPosition.OutputValue = Cache.Positions[Cache.Positions.Num() - 1];
		NewPositions.Add(KeyPosition);
		NewAttributes.Add(NewDefaultAttribute);
	}
	else
	{
		KeyIndex = KeyPosAttrs.Num() - 1;
		KeyPosition.InputValue = KeyPosAttrs[KeyIndex].Key.InputValue;
		KeyPosition.OutputValue = KeyPosAttrs[KeyIndex].Key.OutputValue;
		NewPositions.Add(KeyPosition);
		NewAttributes.Add(KeyPosAttrs[KeyIndex].Value);
	}
	
	TArray<TOptional<FKeyHandle>> NewKeyHandles;
	NewKeyHandles.SetNumUninitialized(NewPositions.Num());

	TArrayView<TOptional<FKeyHandle>> NewKeyHandlesView = MakeArrayView(NewKeyHandles);

	Curve->AddKeys(NewPositions, NewAttributes, &NewKeyHandlesView);

	for (const TOptional<FKeyHandle>& Handle : NewKeyHandlesView)
	{
		if (Handle.IsSet())
		{
			OutHandleSet.Add(Handle.GetValue(), ECurvePointType::Key);
		}
	}
	const float Tolerance = (InParams.TolerancePercentage / 100.0) * (Cache.Max - Cache.Min);
	TArray<FKeyHandle> HandlesAdded;
	for (int32 Index = 0; Index < (NewPositions.Num() -1); ++Index)
	{
		const int32 StartIndex = Cache.GetIndex(NewPositions[Index].InputValue);
		const int32 EndIndex = Cache.GetIndex(NewPositions[Index + 1].InputValue);
		Bisect(Curve, Cache, Tolerance, NewDefaultAttribute, StartIndex, EndIndex,HandlesAdded);
	}
	for (const FKeyHandle& NewHandle : HandlesAdded)
	{
		OutHandleSet.Add(NewHandle, ECurvePointType::Key);
	}


	// bDeleteStaticChannels: if exactly 2 keys remain with the same value, remove them
	// and let RemoveKeys set the channel default via InCurrentTime
	if (InParams.bDeleteStaticChannels)
	{
		TArray<FKeyHandle> AllRemainingKeys;
		Curve->GetAllKeys(AllRemainingKeys);

		if (AllRemainingKeys.Num() == 2)
		{
			TArray<FKeyPosition> KeyPositions;
			KeyPositions.SetNum(2);
			Curve->GetKeyPositions(AllRemainingKeys, KeyPositions);

			if (FMath::IsNearlyEqual(
				KeyPositions[0].OutputValue,
				KeyPositions[1].OutputValue,
				UE_KINDA_SMALL_NUMBER))
			{
				// Remove both keys; InCurrentTime = first key time so the default
				// is evaluated to the constant value and stored on the channel
				Curve->RemoveKeys(AllRemainingKeys, KeyPositions[0].InputValue);
				for (const FKeyHandle& RemainingHandle : AllRemainingKeys)
				{
					OutHandleSet.Remove(RemainingHandle, ECurvePointType::Key);
				}
			}
		}
	}
}

void UCurveEditorSmartReduceFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	OutKeysToSelect.Reset();
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		FCurveModel* Curve = InCurveEditor->FindCurve(Pair.Key);
		if (!Curve)
		{
			continue;
		}

		const bool bNeedToTestExisting = true;
		FKeyHandleSet& OutHandleSet = OutKeysToSelect.Add(Pair.Key);

		TOptional<FKeyHandleSet> KeyHandleSet = Pair.Value;
		SmartReduce(Curve, SmartReduceParams, KeyHandleSet, bNeedToTestExisting, OutHandleSet);

	}
}
