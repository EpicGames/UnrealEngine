// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/CurveEditorSmartSnapFilter.h"

#include "Misc/SmartSnap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorSmartSnapFilter)

void UCurveEditorSmartSnapFilter::ApplyFilter_Impl(
	TSharedRef<FCurveEditor> InCurveEditor,
	const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect
	)
{
	UE::CurveEditor::EnumerateSmartSnappableKeys(*InCurveEditor, InKeysToOperateOn, OutKeysToSelect,
		[&OutKeysToSelect](const FCurveModelID& CurveModelId, FCurveModel& CurveModel, const UE::CurveEditor::FSmartSnapResult& SnapResult)
		{
			ApplySmartSnap(CurveModel, SnapResult);
		});
}

bool UCurveEditorSmartSnapFilter::CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor)
{
	return UE::CurveEditor::CanSmartSnapSelection(InCurveEditor->GetSelection()) || InCurveEditor->GetCurves().Num() > 0;
}
