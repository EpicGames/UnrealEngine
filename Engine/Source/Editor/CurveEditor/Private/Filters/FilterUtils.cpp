// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterUtils.h"

#include "Filters/CurveEditorFilterBase.h"
#include "Modification/Utils/ScopedSelectionChange.h"

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveModel.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Templates/SharedPointer.h"

namespace UE::CurveEditor::FilterUtils
{
	void ApplyFilter(const TSharedRef<FCurveEditor>& InCurveEditor, UCurveEditorFilterBase& InFilter)
	{
		const TMap<FCurveModelID, FKeyHandleSet>& SelectedKeys = InCurveEditor->Selection.GetAll();
		auto HasSelectedKeys = [&SelectedKeys]()
			{
				if (SelectedKeys.IsEmpty())
				{
					return false;
				}
				else
				{
					for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : SelectedKeys)
					{
						if (Pair.Value.Num() > 0)
						{
							return true;
						}
					}
				}
				return false;

			};
		const bool bHasSelectedKeys = HasSelectedKeys();
		if (bHasSelectedKeys == false)
		{
			// No keys selected — operate on all visible curves
			TMap<FCurveModelID, FKeyHandleSet> AllCurveKeys;
			for (const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : InCurveEditor->GetCurves())
			{
				FKeyHandleSet& HandleSet = AllCurveKeys.Add(CurvePair.Key);
				for (const FKeyHandle& Handle : CurvePair.Value->GetAllKeys())
				{
					HandleSet.Add(Handle, ECurvePointType::Key);
				}
			}
			ApplyFilter(InCurveEditor, InFilter, AllCurveKeys, bHasSelectedKeys);
		}
		else
		{
			ApplyFilter(InCurveEditor, InFilter, SelectedKeys, bHasSelectedKeys);
		}
	}

	void ApplyFilter(
		const TSharedRef<FCurveEditor>& InCurveEditor,
		UCurveEditorFilterBase& InFilter,
		const TMap<FCurveModelID, FKeyHandleSet>& InKeysToFilter,
		bool bResetSelection
		)
	{
		using namespace UE::CurveEditor;
		
		TMap<FCurveModelID, FKeyHandleSet> OutKeysToSelect;
		{
			const FScopedCurveChange KeyChange(FCurvesSnapshotBuilder(InCurveEditor, InKeysToFilter));
			InFilter.ApplyFilter(InCurveEditor, InKeysToFilter, OutKeysToSelect);
		}
		if (bResetSelection)
		{
			const FScopedSelectionChange Transaction(InCurveEditor);
			// Clear their selection and then set it to the keys the filter thinks you should have selected.
			InCurveEditor->GetSelection().Clear();

			for (const TTuple<FCurveModelID, FKeyHandleSet>& OutSet : OutKeysToSelect)
			{
				InCurveEditor->GetSelection().Add(OutSet.Key, ECurvePointType::Key, OutSet.Value.AsArray());
			}
		}
	}
}