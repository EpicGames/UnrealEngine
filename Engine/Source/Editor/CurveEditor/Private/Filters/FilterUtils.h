// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class FCurveEditor;
class UCurveEditorFilterBase;

struct FCurveModelID;
struct FKeyHandleSet;

namespace UE::CurveEditor::FilterUtils
{
	/** Applies the filter to the entire user selection. */
	void ApplyFilter(const TSharedRef<FCurveEditor>& InCurveEditor, UCurveEditorFilterBase& InFilter);
	
	/** Applies the filter to the supplied keys. */
	void ApplyFilter(
		const TSharedRef<FCurveEditor>& InCurveEditor,
		UCurveEditorFilterBase& InFilter,
		const TMap<FCurveModelID, FKeyHandleSet>& InSelectedKeys,
		bool bResetSelection
		);
}
