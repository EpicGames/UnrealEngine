// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/PromotedFilterCommandBinder.h"

#include "Filters/FilterUtils.h"
#include "Filters/PromotedFilterContainer.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FPromotedFilterCommandBinder"

namespace UE::CurveEditor
{
	FPromotedFilterCommandBinder::FPromotedFilterCommandBinder(
		const TSharedRef<FPromotedFilterContainer>& InContainer,
		const TSharedRef<FUICommandList>& InCommandList,
		const TSharedRef<FCurveEditor>& InCurveEditor
		)
		: Container(InContainer)
		, CommandList(InCommandList)
		, CurveEditor(InCurveEditor)
	{
		InContainer->OnFilterAdded().AddRaw(this, &FPromotedFilterCommandBinder::OnFilterAdded);
		InContainer->OnFilterRemoved().AddRaw(this, &FPromotedFilterCommandBinder::OnFilterRemoved);

		InContainer->ForEachFilter([this, &InCommandList](UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& Command)
		{
			MapAction(InFilter, Command, *InCommandList);
		});
	}

	FPromotedFilterCommandBinder::~FPromotedFilterCommandBinder()
	{
		const TSharedPtr<FPromotedFilterContainer> ContainerPin = Container.Pin();
		if (ContainerPin)
		{
			ContainerPin->OnFilterAdded().RemoveAll(this);
			ContainerPin->OnFilterRemoved().RemoveAll(this);
		}
		
		const TSharedPtr<FUICommandList> CommandListPin = CommandList.Pin();
		if (ContainerPin && CommandListPin)
		{
			ContainerPin->ForEachFilter([&CommandListPin](UCurveEditorFilterBase&, const TSharedRef<FUICommandInfo>& Command)
			{
				CommandListPin->UnmapAction(Command);
			});
		}
	}
	
	void FPromotedFilterCommandBinder::OnFilterAdded(UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand) const
	{
		const TSharedPtr<FUICommandList> CommandListPin = CommandList.Pin();
		if (ensure(CommandListPin))
		{
			MapAction(InFilter, InCommand, *CommandListPin);
		}
	}

	void FPromotedFilterCommandBinder::OnFilterRemoved(UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand) const
	{
		const TSharedPtr<FUICommandList> CommandListPin = CommandList.Pin();
		if (ensure(CommandListPin))
		{
			CommandListPin->UnmapAction(InCommand);
		}
	}

	void FPromotedFilterCommandBinder::MapAction(
		UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand, FUICommandList& CommandListPin
		) const
	{
		CommandListPin.MapAction(InCommand, FUIAction(
			FExecuteAction::CreateRaw(this, &FPromotedFilterCommandBinder::ApplyFilter, &InFilter, InCommand.ToWeakPtr()),
			FCanExecuteAction::CreateRaw(this, &FPromotedFilterCommandBinder::CanApplyFilter, &InFilter, InCommand.ToWeakPtr())
			));
	}

	void FPromotedFilterCommandBinder::ApplyFilter(UCurveEditorFilterBase* InFilter, TWeakPtr<FUICommandInfo> Command) const
	{
		check(InFilter);
	
		// Unlikely but: if CommandPin is null, that means somebody called RemoveInstance while a menu was open.
		const TSharedPtr<FUICommandInfo> CommandPin = Command.Pin();
		const TSharedPtr<FCurveEditor> Editor = CurveEditor.Pin();
		if (CommandPin && Editor && ensure(InFilter))
		{
			const FScopedTransaction Transaction(FText::Format(LOCTEXT("ApplyFmt", "Apply {0}"), CommandPin->GetLabel()));
			FilterUtils::ApplyFilter(Editor.ToSharedRef(), *InFilter);	
		}
	}

	bool FPromotedFilterCommandBinder::CanApplyFilter(UCurveEditorFilterBase* InFilter, TWeakPtr<FUICommandInfo> Command) const
	{
		check(InFilter);
	
		// Unlikely but: if CommandPin is null, that means somebody called RemoveInstance while a menu was open.
		if (const TSharedPtr<FUICommandInfo> CommandPin = Command.Pin())
		{
			TSharedPtr<FCurveEditor> Editor = CurveEditor.Pin();
			return ensure(Editor && InFilter) && InFilter->CanApplyFilter(Editor.ToSharedRef());
		}
		return false;
	}
}

#undef LOCTEXT_NAMESPACE