// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/PromotedFilterContainer.h"

#include "Algo/NoneOf.h"
#include "CurveEditorCommands.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"

#define LOCTEXT_NAMESPACE "FPromotedFilterContainer"

namespace UE::CurveEditor
{
FPromotedFilterContainer::FPromotedFilterContainer(const FName InContextName)
	: CommandContext(MakeShared<FBindingContext>(
		InContextName,
		// This is what it will show up in Editor Preferences
		LOCTEXT("CurveEditorFilters.Description", "Curve Editor Filters"),
		// Bindings are not allowed to be the same between parent and child contexts
		FCurveEditorCommands::Get().GetContextName(),
		FAppStyle::GetAppStyleSetName())
		)
{}

FPromotedFilterContainer::~FPromotedFilterContainer()
{
	FInputBindingManager::Get().RemoveContextByName(CommandContext->GetContextName());
}

void FPromotedFilterContainer::AppendToBuilder(FToolBarBuilder& InToolBarBuilder, const FMenuEntryResizeParams& InResizeParams) const
{
	for (const FFilterData& FilterData : PromotedFilters)
	{
		InToolBarBuilder.AddToolBarButton(FilterData.Command, NAME_None, {}, {}, {}, NAME_None, {}, {}, {}, InResizeParams);
	}
}

void FPromotedFilterContainer::AppendToBuilder(FMenuBuilder& InMenuBuilder) const
{
	for (const FFilterData& FilterData : PromotedFilters)
	{
		InMenuBuilder.AddMenuEntry(FilterData.Command);
	}
}

void FPromotedFilterContainer::AddInstance(UCurveEditorFilterBase& InFilter)
{
	const int32 Index = IndexOf(InFilter.GetClass());
	const bool bIsAlreadyContained = PromotedFilters.IsValidIndex(Index);
	if (bIsAlreadyContained)
	{
		return;
	}

	// Assuming all passed in InFilters are in the same package (e.g. GetTransientPackage()), GetFName should be unique.
	const FName CommandName = InFilter.GetFName();
	const bool bIsUniqueName = Algo::NoneOf(PromotedFilters, [CommandName](const FFilterData& FilterData)
	{
		return FilterData.FilterInstance.GetFName() == CommandName;
	});
	if (!ensure(bIsUniqueName))
	{
		return;
	}

	UClass* FilterClass = InFilter.GetClass();
	TSharedPtr<FUICommandInfo> Command;
	FUICommandInfo::MakeCommandInfo(CommandContext, Command,
		CommandName, 
		UCurveEditorFilterBase::GetLabel(FilterClass),
		UCurveEditorFilterBase::GetDescription(FilterClass),
		UCurveEditorFilterBase::GetIcon(FilterClass),
		EUserInterfaceActionType::Button,
		FInputChord()
		);
	PromotedFilters.Emplace(InFilter, Command.ToSharedRef());
	
	OnFilterAddedDelegate.Broadcast(InFilter, Command.ToSharedRef());
}
	
void FPromotedFilterContainer::RemoveInstance(UCurveEditorFilterBase& InFilter)
{
	const int32 Index = IndexOf(InFilter.GetClass());
	if (PromotedFilters.IsValidIndex(Index))
	{
		RemoveAtInternal(Index);
	}
}

void FPromotedFilterContainer::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (int32 Index = 0; Index < PromotedFilters.Num(); ++Index)
	{
		TObjectPtr<UCurveEditorFilterBase> Filter = &PromotedFilters[Index].FilterInstance;
		Collector.AddReferencedObject(Filter);

		// GC deleted the object?
		if (!Filter)
		{
			// Preserver order, no swap.
			RemoveAtInternal(Index);
			--Index;
		}
	}
}

int32 FPromotedFilterContainer::IndexOf(const UClass* InFilterClass) const
{
	return InFilterClass
		? PromotedFilters.IndexOfByPredicate([InFilterClass](const FFilterData& FilterData){ return FilterData.FilterInstance.GetClass() == InFilterClass; })
		: INDEX_NONE;
}

void FPromotedFilterContainer::RemoveAtInternal(const int32 InIndex)
{
	checkSlow(PromotedFilters.IsValidIndex(InIndex));
	
	UCurveEditorFilterBase* Filter = &PromotedFilters[InIndex].FilterInstance;
	const TSharedRef<FUICommandInfo> Command = PromotedFilters[InIndex].Command;
	FUICommandInfo::UnregisterCommandInfo(CommandContext, Command);
	PromotedFilters.RemoveAt(InIndex);
		
	OnFilterRemovedDelegate.Broadcast(*Filter, Command);
}
}

#undef LOCTEXT_NAMESPACE