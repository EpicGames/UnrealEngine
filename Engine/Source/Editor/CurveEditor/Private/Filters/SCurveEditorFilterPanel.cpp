// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SCurveEditorFilterPanel.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Filters/FilterUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Math/Vector2D.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

TWeakPtr<SWindow> SCurveEditorFilterPanel::ExistingFilterWindow;

/** ClassViewerFilter for Animation Modifier classes */
class FCurveFilterClassFilter : public IClassViewerFilter
{
public:
	bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return InClass->IsChildOf(UCurveEditorFilterBase::StaticClass()) && InClass != UCurveEditorFilterBase::StaticClass();
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return InClass->IsChildOf(UCurveEditorFilterBase::StaticClass());
	}
};

void SCurveEditorFilterPanel::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor, UClass* DefaultFilterClass)
{
	WeakCurveEditor = InCurveEditor;

	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.bShowNoneOption = false;
	// Options.Mode = EClassViewerMode::ClassBrowsing;
	Options.bAllowViewOptions = false;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.InitiallySelectedClass = DefaultFilterClass;

	TSharedPtr<FCurveFilterClassFilter> ClassFilter = MakeShared<FCurveFilterClassFilter>();
	Options.ClassFilters.Add(ClassFilter.ToSharedRef());

	FOnClassPicked OnPicked(FOnClassPicked::CreateSP(this, &SCurveEditorFilterPanel::SetFilterClass));

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	
	// Configure the Details View
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.ViewIdentifier = "CurveEditorFilterPanel";
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowOptions = false;

	// Generate it and store a reference so we can update it with the right object later.
	DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	TSharedRef<SWidget> ClassFilterWidget = ClassViewerModule.CreateClassViewer(Options, OnPicked);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Class Picker to choose which class shows up in the Details Panel
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ClassFilterWidget
		]
		// Details Panel
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			DetailsView.ToSharedRef()
		]
		
		// Footer Row
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[			
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4, 0, 0, 0)
			.FillWidth(1.0f)
			[
				// Unfortunately the Class Filter can't have a class selected by default so 
				// when the panel is opened you don't know what filter has been pre-selected.
				// We're solving this for now by putting a textbox that shows your current filter
				// by name.
				SNew(STextBlock)
				.Text(this, &SCurveEditorFilterPanel::GetCurrentFilterText)
				.TextStyle(FAppStyle::Get(), "LargeText")
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				// Apply Button
				SNew(SButton)
				.Text(NSLOCTEXT("CurveEditorFilterPanel", "ApplyFilter", "Apply"))
				.OnClicked(this, &SCurveEditorFilterPanel::OnApplyClicked)
				.IsEnabled(this, &SCurveEditorFilterPanel::CanApplyFilter)
			]
		]
	];
}

void SCurveEditorFilterPanel::SetFilterClass(UClass* InClass)
{
	UObject* ClassCDO = InClass->GetDefaultObject<UObject>();

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		if (UCurveEditorFilterBase* Filter = Cast<UCurveEditorFilterBase>(ClassCDO))
		{
			Filter->InitializeFilter(CurveEditor.ToSharedRef());
		}
	}

	DetailsView->SetObject(ClassCDO);

	OnFilterClassChanged.ExecuteIfBound();
}

FReply SCurveEditorFilterPanel::OnApplyClicked()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor && DetailsView->GetSelectedObjects().Num() > 0)
	{
		const int32 SelectionCount = CurveEditor->GetSelection().Count();
		FText TransactionText;
		if (SelectionCount > 0)
		{
			TransactionText = FText::Format(
				NSLOCTEXT("CurveEditorFilterApply", "Filter Curves", "Filtered {0}|plural(one=Curve, other=Curves)"),
				SelectionCount);
		}
		else
		{
			TransactionText = NSLOCTEXT("CurveEditorFilterApply", "Filter All Visible Curves", "Filtered All Visible Curves");
		}
		const FScopedTransaction Transaction(TransactionText);

		UCurveEditorFilterBase* Filter = Cast<UCurveEditorFilterBase>(DetailsView->GetSelectedObjects()[0]);
		UE::CurveEditor::FilterUtils::ApplyFilter(CurveEditor.ToSharedRef(), *Filter);
	}

	return FReply::Handled();
}

bool SCurveEditorFilterPanel::CanApplyFilter() const
{
	if (DetailsView->GetSelectedObjects().Num() == 0)
	{
		return false;
	}

	UCurveEditorFilterBase* CurrentFilter = Cast<UCurveEditorFilterBase>(DetailsView->GetSelectedObjects()[0].Get());

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid() && CurrentFilter)
	{
		return CurrentFilter->CanApplyFilter(CurveEditor.ToSharedRef());
	}

	return false;
}


TSharedPtr<SCurveEditorFilterPanel> SCurveEditorFilterPanel::OpenDialog(TSharedPtr<SWindow> RootWindow, TSharedRef<FCurveEditor> InHostCurveEditor, TSubclassOf<UCurveEditorFilterBase> DefaultFilterClass)
{
	TSharedPtr<SWindow> ExistingWindow = ExistingFilterWindow.Pin();
	if (ExistingWindow.IsValid())
	{
		ExistingWindow->BringToFront();
	}
	else
	{
		ExistingWindow = SNew(SWindow)
			.Title(NSLOCTEXT("CurveEditorFilterPanel", "WindowTitle", "Curve Editor Filters"))
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.ClientSize(FVector2D(480, 360));

		if (RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		}
	}

	TSharedPtr<SCurveEditorFilterPanel> FilterPanel;
	ExistingWindow->SetContent(
		SAssignNew(FilterPanel, SCurveEditorFilterPanel, InHostCurveEditor, DefaultFilterClass)
	);

	ExistingFilterWindow = ExistingWindow;

	return FilterPanel;
}

void SCurveEditorFilterPanel::CloseDialog()
{
	TSharedPtr<SWindow> ExistingWindow = ExistingFilterWindow.Pin();
	if (ExistingWindow.IsValid())
	{
		ExistingWindow->RequestDestroyWindow();
		ExistingWindow = nullptr;
	}
}

FText SCurveEditorFilterPanel::GetCurrentFilterText() const
{
	FText CurrentFilterName = NSLOCTEXT("SCurveEditorFilterPanel", "NoFilterSelectedName", "None");
	if (DetailsView->GetSelectedObjects().Num() > 0)
	{
		UCurveEditorFilterBase* CurrentFilter = Cast<UCurveEditorFilterBase>(DetailsView->GetSelectedObjects()[0].Get());
		if (CurrentFilter)
		{
			CurrentFilterName = CurrentFilter->GetClass()->GetDisplayNameText();
		}
	}

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		const int32 SelectedKeysCount = CurveEditor->GetSelection().Count();
		if (SelectedKeysCount > 0)
		{
			const FText FormatString = NSLOCTEXT("SCurveEditorFilterPanel", "CurrentFilterWithKeysFormat", "Current Filter: {0} ({1} Keys)");
			return FText::Format(FormatString, CurrentFilterName, SelectedKeysCount);
		}
		else if (CurveEditor->GetCurves().Num() > 0)
		{
			const FText FormatString = NSLOCTEXT("SCurveEditorFilterPanel", "CurrentFilterAllCurvesFormat", "Current Filter: {0} (All Visible Curves)");
			return FText::Format(FormatString, CurrentFilterName);
		}
		else
		{
			const FText FormatString = NSLOCTEXT("SCurveEditorFilterPanel", "CurrentFilterNoKeysFormat", "Current Filter: {0} (No Keys Selected)");
			return FText::Format(FormatString, CurrentFilterName);
		}
	}

	return FText();
}
