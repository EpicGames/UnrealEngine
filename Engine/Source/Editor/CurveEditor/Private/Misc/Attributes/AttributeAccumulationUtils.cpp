// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeAccumulationUtils.h"

#include "CurveEditor.h"
#include "CurveEditorSelectionPrivate.h"
#include "CurveEditorTrace.h"
#include "CurveModel.h"

namespace UE::CurveEditor
{
void UpdateCommonCurveInfo(
	const FCurveEditor& InCurveEditor, bool& bOutSelectionSupportsWeightedTangents,
	FCurveAttributes& OutCachedCommonCurveAttributes, FKeyAttributes& OutCachedCommonKeyAttributes)
{
	SCOPED_CURVE_EDITOR_TRACE(UpdateCommonCurveInfo);
	
	FAccumulatedCurveInfo AccumulatedCurveInfo = InCurveEditor.Selection.IsEmpty() ?
		AccumulateAllCurvesInfo(InCurveEditor) :
		AccumulateSelectedCurvesInfo(InCurveEditor);

	// Reset the common curve and key info
	bOutSelectionSupportsWeightedTangents = AccumulatedCurveInfo.bSelectionSupportsWeightedTangents;
	OutCachedCommonCurveAttributes = AccumulatedCurveInfo.CurveAttributes.Get(FCurveAttributes());
	OutCachedCommonKeyAttributes = AccumulatedCurveInfo.KeyAttributes.Get(FKeyAttributes());
}

FAccumulatedCurveInfo AccumulateSelectedCurvesInfo(const FCurveEditor& InCurveEditor)
{
	FAccumulatedCurveInfo AccumulatedAttributes;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InCurveEditor.Selection.GetAll())
	{
		if (const FCurveModel* Curve = InCurveEditor.FindCurve(Pair.Key))
		{
			AccumulateCurveInfo(Curve, Pair.Value.AsArray(), AccumulatedAttributes);
		}
	}
	return AccumulatedAttributes;
}

FAccumulatedCurveInfo AccumulateAllCurvesInfo(const FCurveEditor& InCurveEditor)
{	
	FAccumulatedCurveInfo AccumulatedAttributes;
	for (const FCurveModelID ID : InCurveEditor.GetEditedCurves())
	{
		if (FCurveModel* Curve = InCurveEditor.FindCurve(ID))
		{
			const TArray<FKeyHandle> KeyHandles = Curve->GetAllKeys();
			AccumulateCurveInfo(Curve, KeyHandles, AccumulatedAttributes);
		}
	}	
	return AccumulatedAttributes;
}

void AccumulateCurveInfo(
	const FCurveModel* InCurve, TConstArrayView<const FKeyHandle> InKeys, FAccumulatedCurveInfo& InOutAccumulatedAttributes)
{
	FCurveAttributes CurveAttributes;
	InCurve->GetCurveAttributes(CurveAttributes);

	// Some curves don't support extrapolation. We don't count them for determining the accumulated state.
	const bool bCurveHasNoExtrapolation = CurveAttributes.HasPreExtrapolation() && CurveAttributes.GetPreExtrapolation() == RCCE_None
		&& CurveAttributes.HasPostExtrapolation() && CurveAttributes.GetPostExtrapolation() == RCCE_None;
	if (!bCurveHasNoExtrapolation && !InOutAccumulatedAttributes.CurveAttributes.IsSet())
	{
		InOutAccumulatedAttributes.CurveAttributes = CurveAttributes;
	}
	else if (!bCurveHasNoExtrapolation)
	{
		InOutAccumulatedAttributes.CurveAttributes = FCurveAttributes::MaskCommon(
			InOutAccumulatedAttributes.CurveAttributes.GetValue(), CurveAttributes
			);
	}
		
	TArray<FKeyAttributes> AllKeyAttributes;	
	AllKeyAttributes.SetNum(InKeys.Num());

	InCurve->GetKeyAttributes(InKeys, AllKeyAttributes);
	for (const FKeyAttributes& Attributes : AllKeyAttributes)
	{
		if (Attributes.HasTangentWeightMode())
		{
			InOutAccumulatedAttributes.bSelectionSupportsWeightedTangents = true;
		}

		if (!InOutAccumulatedAttributes.KeyAttributes.IsSet())
		{
			InOutAccumulatedAttributes.KeyAttributes = Attributes;
		}
		else
		{
			InOutAccumulatedAttributes.KeyAttributes = FKeyAttributes::MaskCommon(
				InOutAccumulatedAttributes.KeyAttributes.GetValue(), Attributes
				);
		}
	}
}
}