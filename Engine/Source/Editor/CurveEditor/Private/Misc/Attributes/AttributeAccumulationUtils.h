// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RealCurve.h"
#include "CurveDataAbstraction.h"

class FCurveEditor;
class FCurveModel;

namespace  UE::CurveEditor
{
	/** Used to hold curve and key attributes to update the Common Curve Info with. */
	struct FAccumulatedCurveInfo
	{
		bool bSelectionSupportsWeightedTangents = false;
		TOptional<FCurveAttributes> CurveAttributes;
		TOptional<FKeyAttributes> KeyAttributes;
	};

	/** Update current curve selection attributes, if no keys are selected, info of all curves currently contained in the Curve Editor is used. */
	void UpdateCommonCurveInfo(const FCurveEditor& InCurveEditor, bool& bOutSelectionSupportsWeightedTangents, FCurveAttributes& OutCachedCommonCurveAttributes, FKeyAttributes& OutCachedCommonKeyAttributes);

	/** Gather curve and key attributes from all currently selected curves. */
	FAccumulatedCurveInfo AccumulateSelectedCurvesInfo(const FCurveEditor& InCurveEditor);
	/** Gather curve and key attributes from all curves currently contained in the Curve Editor regardless of visibility. .*/
	FAccumulatedCurveInfo AccumulateAllCurvesInfo(const FCurveEditor& InCurveEditor);

	/** Add curve and key attributes from the given curve and the keys on it to the accumulated info. */
	void AccumulateCurveInfo(const FCurveModel* InCurve, TConstArrayView<const FKeyHandle> InKeys, FAccumulatedCurveInfo& InOutAccumulatedAttributes);
}

