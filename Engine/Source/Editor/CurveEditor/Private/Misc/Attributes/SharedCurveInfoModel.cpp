// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedCurveInfoModel.h"

#include "AttributeAccumulationUtils.h"
#include "CurveEditor.h"
#include "SCurveEditorViewContainer.h"
#include "Misc/CoreDelegates.h"

namespace  UE::CurveEditor
{
FSharedCurveInfoModel::FSharedCurveInfoModel(const TSharedRef<FCurveEditor>& InCurveEditor, const TSharedRef<SCurveEditorViewContainer>& InViewContainer)
	: WeakCurveEditor(InCurveEditor.ToWeakPtr())
	, ViewContainer(InViewContainer.ToWeakPtr())
	, CurveModifiedListener(FCurveChangeListener::MakeForAllCurves(InCurveEditor))
{
	InCurveEditor->Selection.OnSelectionChanged().AddRaw(this, &FSharedCurveInfoModel::DeferToRefreshToEndOfFrame);
	InCurveEditor->OnCurveArrayChanged.AddRaw(this, &FSharedCurveInfoModel::OnCurveArrayChanged);
	CurveModifiedListener.OnCurveModified().AddRaw(this, &FSharedCurveInfoModel::DeferToRefreshToEndOfFrame);
	InViewContainer->OnCurveHasChangedExternally().AddRaw(this, &FSharedCurveInfoModel::OnCurveHasChanged);

	DeferToRefreshToEndOfFrame();
}

FSharedCurveInfoModel::~FSharedCurveInfoModel()
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		CurveEditorPin->Selection.OnSelectionChanged().RemoveAll(this);
		CurveEditorPin->OnCurveArrayChanged.RemoveAll(this);
	}
	
	if (const TSharedPtr<SCurveEditorViewContainer> ViewContainerPin = ViewContainer.Pin())
	{
		ViewContainerPin->OnCurveHasChangedExternally().RemoveAll(this);
	}
		
	ClearPendingRefresh();
}

void FSharedCurveInfoModel::Tick(float DeltaTime)
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (CurveEditorPin && CurveEditorPin->GetActiveCurvesSerialNumber() != ActiveCurvesSerialNumber)
	{
		DeferToRefreshToEndOfFrame();
	}
}

TStatId FSharedCurveInfoModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FSharedCurveInfoModel, STATGROUP_Tickables); 
}

void FSharedCurveInfoModel::DeferToRefreshToEndOfFrame()
{
	if (!HasDeferredRefresh())
	{
		FCoreDelegates::OnEndFrame.AddRaw(this, &FSharedCurveInfoModel::Refresh);
	}
}

void FSharedCurveInfoModel::Refresh()
{
	ClearPendingRefresh();

	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}

	TArray<FCurveModelID> Curves;
	CurveEditorPin->GetCurves().GenerateKeyArray(Curves);
	CurveModifiedListener.ResubscribeTo(MoveTemp(Curves));

	ActiveCurvesSerialNumber = CurveEditorPin->GetActiveCurvesSerialNumber();
	
	UpdateCommonCurveInfo(*CurveEditorPin, bSelectionSupportsWeightedTangents, CachedCommonCurveAttributes, CachedCommonKeyAttributes);
}

bool FSharedCurveInfoModel::HasDeferredRefresh() const
{
	return FCoreDelegates::OnEndFrame.IsBoundToObject(this);
}

void FSharedCurveInfoModel::ClearPendingRefresh() const
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}
}
