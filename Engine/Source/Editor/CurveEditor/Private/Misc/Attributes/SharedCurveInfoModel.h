// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "CurveModel.h"
#include "Misc/CurveChangeListener.h"
#include "Templates/UnrealTemplate.h"
#include "TickableEditorObject.h"

class FCurveEditor;
class SCurveEditorViewContainer;

namespace  UE::CurveEditor
{
class FCurveDrawParamsCache;

/**
 * Keeps track of information that is shared for all keys: either for all selected keys, or if no keys are selected, all keys in view.
 * Detects when changes are made to the curves and refreshes the informationa at the end of tick.
 */
class FSharedCurveInfoModel : public FNoncopyable, public FTickableEditorObject
{
public:
	
	explicit FSharedCurveInfoModel(const TSharedRef<FCurveEditor>& InCurveEditor, const TSharedRef<SCurveEditorViewContainer>& InViewContainer);
	~FSharedCurveInfoModel();

	bool DoesSelectionSupportWeightedTangents() const { return bSelectionSupportsWeightedTangents; }
	const FCurveAttributes& GetCachedCommonCurveAttributes() const { return CachedCommonCurveAttributes; }
	const FKeyAttributes& GetCachedCommonKeyAttributes() const { return CachedCommonKeyAttributes; }

	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

private:

	/** Owning curve editor. Needed to (un)subscribe from relevant change events. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;

	/**
	 * This is used to detect any external changes made to curves.
	 * Specifically, this UI calls FCurveModel::HasChangedAndResetTest and broadcasts a delegate when a change is detected.
	 *
	 * Ideally, we would not reference UI here but another model. 
	 */
	const TWeakPtr<SCurveEditorViewContainer> ViewContainer;

	/** Subscribed to FCurveModel::OnCurveModified of all curves. */
	FCurveChangeListener CurveModifiedListener;
	/** Last FCurveEditor::ActiveCurvesSerialNumber. Used to detect changes. */
	uint32 ActiveCurvesSerialNumber;
	
	/** True if the current selection supports weighted tangents, false otherwise */
	bool bSelectionSupportsWeightedTangents = false;
	/** Cached curve attributes that are common to all visible curves */
	FCurveAttributes CachedCommonCurveAttributes;
	/** Cached key attributes that are common to all selected keys */
	FKeyAttributes CachedCommonKeyAttributes;

	/** Enqueues a call to Refresh that will take place at the end of the current tick. */
	void DeferToRefreshToEndOfFrame();
	/** Updates the shared curve info */
	void Refresh();
	/** @return Whether a call to Refresh is enqueued */
	bool HasDeferredRefresh() const;
	/** Clears the subscription to FCoreDelegates. */
	void ClearPendingRefresh() const;

	/** On next Refresh, we'll resubscribe to all FCurveModel::OnCurveModified delegates. */
	void OnCurveArrayChanged(FCurveModel*, bool, const FCurveEditor*) { DeferToRefreshToEndOfFrame(); }
	/** Invoked when some FCurveModel::HasChangedAndRestTest returned true. */
	void OnCurveHasChanged(const FCurveModelID&) { DeferToRefreshToEndOfFrame(); }
};
}

