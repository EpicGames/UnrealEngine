// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CurveChangeListener.h"

#include "CurveEditor.h"
#include "CurveModel.h"

namespace UE::CurveEditor
{
FCurveChangeListener::FCurveChangeListener(const TSharedRef<FCurveEditor>& InCurveEditor, TArray<FCurveModelID> InCurvesToListenTo)
	: WeakCurveEditor(InCurveEditor)
	, SubscribedToCurves(MoveTemp(InCurvesToListenTo))
{
	SubscribeToCurves();
}

FCurveChangeListener::~FCurveChangeListener()
{
	UnsubscribeFromCurves();
}

void FCurveChangeListener::ResubscribeTo(TArray<FCurveModelID> InNewCurvesToSubscribeTo)
{
	UnsubscribeFromCurves();
	SubscribedToCurves = MoveTemp(InNewCurvesToSubscribeTo);
	SubscribeToCurves();
}

FCurveChangeListener FCurveChangeListener::MakeForAllCurves(const TSharedRef<FCurveEditor>& InCurveEditor)
{
	TArray<FCurveModelID> Curves;
	InCurveEditor->GetCurves().GenerateKeyArray(Curves);
	return FCurveChangeListener(InCurveEditor, MoveTemp(Curves));
}

FCurveChangeListener FCurveChangeListener::MakeForSelectedCurves(const TSharedRef<FCurveEditor>& InCurveEditor)
{
	TArray<FCurveModelID> Curves;
	InCurveEditor->Selection.GetAll().GetKeys(Curves);
	return FCurveChangeListener(InCurveEditor, MoveTemp(Curves));
}

void FCurveChangeListener::SubscribeToCurves()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}
	
	for (const FCurveModelID& CurveId : SubscribedToCurves)
	{
		if (FCurveModel* Model = CurveEditorPin->FindCurve(CurveId))
		{
			Model->OnCurveModified().AddRaw(this, &FCurveChangeListener::HandleCurveModified);
		}
	}
}

void FCurveChangeListener::UnsubscribeFromCurves()
{
	if (TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		for (const FCurveModelID& CurveId : SubscribedToCurves)
		{
			if (FCurveModel* Model = CurveEditorPin->FindCurve(CurveId))
			{
				Model->OnCurveModified().RemoveAll(this);
			}
		}
	}
}
}
