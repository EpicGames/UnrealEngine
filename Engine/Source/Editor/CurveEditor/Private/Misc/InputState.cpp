// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/InputState.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Templates/Function.h"
#include "UnrealClient.h"

namespace UE::CurveEditor::InputState
{

bool ForEachActiveChord(const TSharedPtr<FUICommandInfo>& InCommandInfo, const TFunctionRef<bool(const FInputChord&)>& InVisitor)
{
	if (!InCommandInfo.IsValid())
	{
		return false;
	}

	for (uint32 Index = 0; Index < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++Index)
	{
		const EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(Index);
		const FInputChord& Chord = *InCommandInfo->GetActiveChord(ChordIndex);
		if (InVisitor(Chord))
		{
			return true;
		}
	}

	return false;
}

/**
 * This class tracks non-modifier key state at the slate application level so scrub chords can still be
 * recognized when a click gives focus to a widget that did not receive the original key-down event.
 * For example, without this class, you could still "B" + LMB scrub in Curve Editor or Sequencer when
 * it doesn't have focus and move keys if the mouse is over keys.
 */
class FInputChordStateProcessor : public IInputProcessor
{
public:
	static void Initialize();
	static void Shutdown();

	static FInputChordStateProcessor* Get();

	static bool DoModifierStatesMatchChord(const FInputChord& InChord, const FModifierKeysState& InModifierKeys);

	bool IsChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys) const;
	bool IsScrubChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys) const;
	bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo) const;
	bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
		, const FModifierKeysState& InModifierKeys, const FKey InKey = FKey()) const;
	void SetChordPressed(const FInputChord& InChord, const bool bInPressed);
	void SetKeyPressed(const FKey& InKey, const bool bInPressed);
	bool IsScrubTimeActive() const;
	void SetScrubTimeKeyPressed(const FKey& InKey, const bool bInPressed);

	//~ Begin IInputProcessor
	virtual void Tick(const float InDeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> InCursor) override;
	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	virtual bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	//~ End IInputProcessor

private:
	void HandleApplicationActivationStateChanged(const bool bInActive);

	void ClearPressedKeys();

	static TSharedPtr<FInputChordStateProcessor> Instance;

	FDelegateHandle ApplicationActivationStateChangedHandle;

	TSet<FKey> PressedKeys;
	TSet<FKey> ForcedPressedKeys;
	TSet<FKey> ScrubTimeKeys;
};

TSharedPtr<FInputChordStateProcessor> FInputChordStateProcessor::Instance = nullptr;

void FInputChordStateProcessor::Initialize()
{
	const FInputChordStateProcessor* const CurrentInstance = Get();

	if (CurrentInstance && !Instance->ApplicationActivationStateChangedHandle.IsValid())
	{
		Instance->ApplicationActivationStateChangedHandle = FSlateApplication::Get().OnApplicationActivationStateChanged()
			.AddSP(Instance.ToSharedRef(), &FInputChordStateProcessor::HandleApplicationActivationStateChanged);
	}
}

void FInputChordStateProcessor::Shutdown()
{
	if (!FSlateApplication::IsInitialized() || !Instance.IsValid())
	{
		return;
	}

	FSlateApplication& SlateApp = FSlateApplication::Get();

	if (Instance->ApplicationActivationStateChangedHandle.IsValid())
	{
		SlateApp.OnApplicationActivationStateChanged().Remove(Instance->ApplicationActivationStateChangedHandle);
		Instance->ApplicationActivationStateChangedHandle.Reset();
	}

	SlateApp.UnregisterInputPreProcessor(Instance);
	Instance.Reset();
}

FInputChordStateProcessor* FInputChordStateProcessor::Get()
{
	if (!FSlateApplication::IsInitialized())
	{
		return nullptr;
	}

	if (!Instance.IsValid())
	{
		Instance = MakeShared<FInputChordStateProcessor>();
		FSlateApplication::Get().RegisterInputPreProcessor(Instance);
	}

	return Instance.Get();
}

void FInputChordStateProcessor::Tick(const float InDeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> InCursor)
{
	if (!SlateApp.IsActive() || SlateApp.GetActiveModalWindow().IsValid())
	{
		ClearPressedKeys();
	}
}

void FInputChordStateProcessor::ClearPressedKeys()
{
	PressedKeys.Reset();
	ForcedPressedKeys.Reset();
	ScrubTimeKeys.Reset();
}

void FInputChordStateProcessor::HandleApplicationActivationStateChanged(const bool bInActive)
{
	if (!bInActive)
	{
		ClearPressedKeys();
	}
}

bool FInputChordStateProcessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	PressedKeys.Add(InKeyEvent.GetKey());

	return false;
}

bool FInputChordStateProcessor::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	PressedKeys.Remove(InKeyEvent.GetKey());
	ForcedPressedKeys.Remove(InKeyEvent.GetKey());
	ScrubTimeKeys.Remove(InKeyEvent.GetKey());

	return false;
}

void FInputChordStateProcessor::SetChordPressed(const FInputChord& InChord, const bool bInPressed)
{
	if (InChord.IsValidChord())
	{
		SetKeyPressed(InChord.Key, bInPressed);
		SetScrubTimeKeyPressed(InChord.Key, bInPressed);
	}
}

void FInputChordStateProcessor::SetKeyPressed(const FKey& InKey, const bool bInPressed)
{
	if (!InKey.IsValid())
	{
		return;
	}

	if (bInPressed)
	{
		ForcedPressedKeys.Add(InKey);
	}
	else
	{
		ForcedPressedKeys.Remove(InKey);
	}
}

bool FInputChordStateProcessor::IsScrubTimeActive() const
{
	return !ScrubTimeKeys.IsEmpty();
}

void FInputChordStateProcessor::SetScrubTimeKeyPressed(const FKey& InKey, const bool bInPressed)
{
	if (!InKey.IsValid())
	{
		return;
	}

	if (bInPressed)
	{
		ScrubTimeKeys.Add(InKey);
	}
	else
	{
		ScrubTimeKeys.Remove(InKey);
	}
}

bool FInputChordStateProcessor::DoModifierStatesMatchChord(const FInputChord& InChord, const FModifierKeysState& InModifierKeys)
{
	return (InChord.NeedsAlt() ? InModifierKeys.IsAltDown() : !InModifierKeys.IsAltDown())
		&& (InChord.NeedsShift() ? InModifierKeys.IsShiftDown() : !InModifierKeys.IsShiftDown())
		&& (InChord.NeedsControl() ? InModifierKeys.IsControlDown() : !InModifierKeys.IsControlDown())
		&& (InChord.NeedsCommand() ? InModifierKeys.IsCommandDown() : !InModifierKeys.IsCommandDown());
}

bool FInputChordStateProcessor::IsChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys) const
{
	return InChord.IsValidChord()
		&& (PressedKeys.Contains(InChord.Key) || ForcedPressedKeys.Contains(InChord.Key))
		&& DoModifierStatesMatchChord(InChord, InModifierKeys);
}

bool FInputChordStateProcessor::IsScrubChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys) const
{
	return InChord.IsValidChord()
		&& (PressedKeys.Contains(InChord.Key) || ForcedPressedKeys.Contains(InChord.Key) || ScrubTimeKeys.Contains(InChord.Key))
		&& DoModifierStatesMatchChord(InChord, InModifierKeys);
}

bool FInputChordStateProcessor::IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo) const
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	return ForEachActiveChord(InCommandInfo, [this, &ModifierKeys](const FInputChord& InChord)
		{
			return IsChordPressed(InChord, ModifierKeys);
		});
}

bool FInputChordStateProcessor::IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FModifierKeysState& InModifierKeys, const FKey InKey) const
{
	return ForEachActiveChord(InCommandInfo, [this, &InModifierKeys, InKey](const FInputChord& Chord)
		{
			return Chord.IsValidChord()
				&& Chord.Key == InKey
				&& DoModifierStatesMatchChord(Chord, InModifierKeys);
		});
}

void Initialize()
{
	FInputChordStateProcessor::Initialize();
}

void Shutdown()
{
	FInputChordStateProcessor::Shutdown();
}

bool IsChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	return InputProcessor && InputProcessor->IsChordPressed(InChord, InModifierKeys);
}

bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	return InputProcessor && InputProcessor->IsCommandPressed(InCommandInfo);
}

bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKeyEvent& InKeyEvent)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	return InputProcessor && InputProcessor->IsCommandPressed(InCommandInfo, InKeyEvent.GetModifierKeys(), InKeyEvent.GetKey());
}

bool IsScrubTimeCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	if (!InputProcessor)
	{
		return false;
	}

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bHandled = ForEachActiveChord(InCommandInfo, [InputProcessor, &ModifierKeys](const FInputChord& Chord)
		{
			return InputProcessor->IsScrubChordPressed(Chord, ModifierKeys);
		});
	if (bHandled)
	{
		return true;
	}

	return InputProcessor->IsCommandPressed(InCommandInfo);
}

void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FKey& InKey, const bool bInPressed)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	if (!InputProcessor)
	{
		return;
	}

	ForEachActiveChord(InCommandInfo, [InputProcessor, InKey, bInPressed](const FInputChord& Chord)
		{
			if (Chord.IsValidChord() && Chord.Key == InKey)
			{
				InputProcessor->SetChordPressed(Chord, bInPressed);
			}
			return false;
		});
}

void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FModifierKeysState& InModifierKeys, const bool bInPressed)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	if (!InputProcessor)
	{
		return;
	}

	ForEachActiveChord(InCommandInfo, [InputProcessor, &InModifierKeys, bInPressed](const FInputChord& Chord)
		{
			if (InputProcessor->IsScrubChordPressed(Chord, InModifierKeys))
			{
				InputProcessor->SetChordPressed(Chord, bInPressed);
			}
			return false;
		});
}

void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FViewport* InViewport, const bool bInPressed)
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	if (!InputProcessor || !InViewport)
	{
		return;
	}

	ForEachActiveChord(InCommandInfo, [InputProcessor, InViewport, bInPressed](const FInputChord& Chord)
		{
			if (Chord.IsValidChord() && InViewport->KeyState(Chord.Key))
			{
				InputProcessor->SetChordPressed(Chord, bInPressed);
			}
			return false;
		});
}

void SetKeyPressed(const FKey& InKey, const bool bInPressed)
{
	if (FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get())
	{
		InputProcessor->SetKeyPressed(InKey, bInPressed);
	}
}

void SetScrubTimeKeyPressed(const FKey& InKey, const bool bInPressed)
{
	if (FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get())
	{
		InputProcessor->SetScrubTimeKeyPressed(InKey, bInPressed);
	}
}

bool IsScrubTimeActive()
{
	FInputChordStateProcessor* const InputProcessor = FInputChordStateProcessor::Get();
	return InputProcessor && InputProcessor->IsScrubTimeActive();
}

} // namespace UE::CurveEditor::InputState
