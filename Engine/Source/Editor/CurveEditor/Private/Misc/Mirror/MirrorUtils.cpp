// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Mirror/MirrorUtils.h"

#include "Containers/Array.h"
#include "CurveEditor.h"
#include "Misc/Mirror/PositionMirrorSolver.h"
#include "Misc/Mirror/TangentMirrorSolver.h"

namespace UE::CurveEditor
{
FCurveBounds FindMinMaxHeight(TConstArrayView<FKeyHandle> InKeys, const FCurveModel& InCurveModel)
{
	if (!ensure(!InKeys.IsEmpty()))
	{
		return { 0, 0 };
	}

	TArray<FKeyPosition> Positions;
	Positions.SetNumUninitialized(InKeys.Num());
	InCurveModel.GetKeyPositions(InKeys, Positions);

	double Min = Positions[0].OutputValue;
	double Max = Positions[0].OutputValue;
	for (int32 Index = 1; Index < InKeys.Num(); ++Index)
	{
		Min = FMath::Min(Positions[Index].OutputValue, Min);
		Max = FMath::Max(Positions[Index].OutputValue, Max);
	}
	
	return FCurveBounds{ Min, Max };
}

void MirrorKeyPositions(
	TConstArrayView<FKeyHandle> InKeysToMirror, const FCurveModelID& InCurveId,
	double InBottomHeight, double InTopHeight, double InMirrorHeight, FCurveEditor& InCurveEditor)
{
	const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!ensure(CurveModel))
	{
		return;
	}
	
	// Subdivide the keys into 2 groups: those above and those below the midpoint.
	TArray<FKeyHandle> OldAboveMirrorEdge, OldBelowMirrorEdge;
	for (const FKeyHandle& KeyHandle : InKeysToMirror)
	{
		FKeyPosition KeyPosition;
		CurveModel->GetKeyPositions(TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyPosition>(&KeyPosition, 1));
		if (KeyPosition.OutputValue > InMirrorHeight)
		{
			OldAboveMirrorEdge.Add(KeyHandle);
		}
		else
		{
			OldBelowMirrorEdge.Add(KeyHandle);
		}
	}
	
	// Then, mirror the bottom to the top, and the top to the bottom.
	// Move imaginary edge going through top control point opposite the mirror edge...
	FPositionMirrorSolver TopToBottom(InCurveEditor, InTopHeight, InMirrorHeight);
	TopToBottom.AddKeyPositions(InCurveId, OldAboveMirrorEdge);
	TopToBottom.OnMoveEdge(InBottomHeight);
	
	// ... and do the same for the bottom
	FPositionMirrorSolver BottomToTop(InCurveEditor, InBottomHeight, InMirrorHeight);
	BottomToTop.AddKeyPositions(InCurveId , OldBelowMirrorEdge);
	BottomToTop.OnMoveEdge(InTopHeight);
}

void MirrorTangents(
	TConstArrayView<FKeyHandle> InKeysToMirror, const FCurveModelID& InCurveId,
	double InBottomHeight, double InTopHeight, double InMirrorHeight, FCurveEditor& InCurveEditor
	)
{
	const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!ensure(CurveModel))
	{
		return;
	}
	
	// Subdivide the keys into 2 groups: those above and those below the midpoint.
	TArray<FKeyHandle> OldAboveMirrorEdge, OldBelowMirrorEdge;
	for (const FKeyHandle& KeyHandle : InKeysToMirror)
	{
		FKeyPosition KeyPosition;
		CurveModel->GetKeyPositions(TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyPosition>(&KeyPosition, 1));
		if (KeyPosition.OutputValue > InMirrorHeight)
		{
			OldAboveMirrorEdge.Add(KeyHandle);
		}
		else
		{
			OldBelowMirrorEdge.Add(KeyHandle);
		}
	}
	
	// Then, mirror the bottom to the top, and the top to the bottom.
	// Move imaginary edge going through top control point opposite the mirror edge...
	FTangentMirrorSolver TopToBottom(InTopHeight, InMirrorHeight);
	TopToBottom.AddTangents(InCurveEditor, InCurveId, OldAboveMirrorEdge);
	TopToBottom.OnMoveEdge(InCurveEditor, InBottomHeight);
	
	// ... and do the same for the bottom
	FTangentMirrorSolver BottomToTop(InBottomHeight, InMirrorHeight);
	BottomToTop.AddTangents(InCurveEditor, InCurveId , OldBelowMirrorEdge);
	BottomToTop.OnMoveEdge(InCurveEditor, InTopHeight);
}
}