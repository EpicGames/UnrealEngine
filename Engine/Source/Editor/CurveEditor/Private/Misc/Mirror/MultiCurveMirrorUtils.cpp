// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Mirror/MultiCurveMirrorUtils.h"

#include "CurveEditor.h"
#include "CurveModel.h"

namespace UE::CurveEditor
{
FMirrorableTangentInfo FilterMirrorableTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys)
{
	FMirrorableTangentInfo Result;
	TArray<FKeyHandle>& OutMirroredKeys = Result.MirrorableKeys;
	TArray<FKeyAttributes>& OutInitialAttributes = Result.InitialAttributes;
	TArray<FVector2D>& OutTangents = Result.Tangents;
	TArray<double>& OutHeights = Result.KeyHeights;
	
	const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!ensure(CurveModel))
	{
		return Result;
	}
	
	const int32 NumKeys = InKeys.Num();
	OutMirroredKeys.Reserve(NumKeys);
	OutTangents.Reserve(NumKeys);
	OutHeights.Reserve(NumKeys);
	
	for (int32 Index = 0; Index < NumKeys; ++Index)
	{
		const FKeyHandle& KeyHandle = InKeys[Index];
		if (!CurveModel)
		{
			continue;
		}

		FKeyAttributes KeyAttribute;
		CurveModel->GetKeyAttributesExcludingAutoComputed(
			TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyAttributes>(&KeyAttribute, 1)
			);

		const bool bHasValidMode = KeyAttribute.HasTangentMode()
			&& (KeyAttribute.GetTangentMode() == RCTM_User || KeyAttribute.GetTangentMode() == RCTM_Break);
		if (!bHasValidMode)
		{
			continue;
		}

		OutMirroredKeys.Add(KeyHandle);
		if (KeyAttribute.HasArriveTangent() && KeyAttribute.HasLeaveTangent())
		{
			OutTangents.Add({ KeyAttribute.GetArriveTangent(), KeyAttribute.GetLeaveTangent() });
		}
		else if (KeyAttribute.HasArriveTangent())
		{
			OutTangents.Add({ KeyAttribute.GetArriveTangent(), 0.0 });
		}

		FKeyPosition KeyPosition;
		CurveModel->GetKeyPositions(TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyPosition>(&KeyPosition, 1));
		OutHeights.Add(KeyPosition.OutputValue);
	}
	
	OutInitialAttributes.SetNumUninitialized(OutMirroredKeys.Num());
	CurveModel->GetKeyAttributes(OutMirroredKeys, OutInitialAttributes);

	return Result;
}
}
