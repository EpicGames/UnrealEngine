// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Mirror/PositionMirrorSolver.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CurveEditor.h"

namespace UE::CurveEditor
{
FPositionMirrorSolver::FPositionMirrorSolver(FCurveEditor& InCurveEditor, double InStartY, double InMiddlePointY)
	: CurveEditor(InCurveEditor)
	, StartY(InStartY)
	, MiddlePointY(InMiddlePointY)
{}

bool FPositionMirrorSolver::AddKeyPositions(
	const FCurveModelID& InCurveId, TArray<FKeyHandle> InKeys, TArray<FKeyPosition> InPositions
	)
{
	const bool bIsValid = !InKeys.IsEmpty() && (InKeys.Num() == InPositions.Num() || InPositions.IsEmpty());  
	if (!bIsValid)
	{
		return false;
	}
	
	if (InPositions.IsEmpty())
	{
		const FCurveModel* CurveModel = CurveEditor.FindCurve(InCurveId);
		if (!CurveModel)
		{
			return false;
		}
		InPositions.SetNumUninitialized(InKeys.Num());
		CurveModel->GetKeyPositions(InKeys, InPositions);
	}

	TArray<double> Heights;
	Algo::Transform(InPositions, Heights, [](const FKeyPosition& InKeyPosition){ return InKeyPosition.OutputValue; });

	TArray<double> HeightsRelativeToMiddlePoint = Heights;
	for (double& Height : HeightsRelativeToMiddlePoint)
	{
		// TMirrorSolver compute MiddlePointY + MirrorAlpha * Height.
		// MirrorAlpha = 1 at start edge position, MirrorAlpha = -1 on opposite side (perfectly mirrored).
		Height -= MiddlePointY;
	}
	
	AllCurveData.Add(
		InCurveId,
		FCachedCurveData(
			TUniformMirrorSolver<double>(StartY, MiddlePointY, MoveTemp(HeightsRelativeToMiddlePoint), MoveTemp(Heights), MiddlePointY),
			MoveTemp(InKeys), MoveTemp(InPositions)
			)
	);
	return true;
}

void FPositionMirrorSolver::OnMoveEdge(double InDraggedEdgeHeight)
{
	for (TPair<FCurveModelID, FCachedCurveData>& TangentDataPair : AllCurveData)
	{
		FCurveModel* CurveModel = CurveEditor.FindCurve(TangentDataPair.Key);
		if (!CurveModel)
		{
			continue;
		}

		FCachedCurveData& CurveData = TangentDataPair.Value;
		CurveData.TangentSolver.ComputeMirroringParallel(InDraggedEdgeHeight, [this, &CurveData](int32 KeyIndex, double NewValue)
		{
			FKeyPosition& Position = CurveData.PositionsToSet[KeyIndex];
			Position.OutputValue = NewValue;
		});
		
		CurveModel->SetKeyPositions(CurveData.KeyHandles, CurveData.PositionsToSet);
	}
}
}