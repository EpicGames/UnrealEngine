// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Mirror/TangentSelectionFlattener.h"

#include "CurveEditor.h"
#include "Misc/Mirror/MirrorUtils.h"

namespace UE::CurveEditor
{
namespace TangentFlattenDetail
{
static void FilterByMirrorableTangents(
	const TConstArrayView<FKeyHandle> KeysInCurves, const FCurveModelID& InCurveId, FCurveEditor& InCurveEditor,
	TArray<FKeyHandle>& OutMirroredKeys, TArray<FKeyAttributes>& OutInitialAttributes, TArray<FVector2D>& OutTangents, TArray<double>& OutHeights
	)
	{
		const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
		if (!ensure(CurveModel))
		{
			return;
		}
	
		const int32 NumKeys = KeysInCurves.Num();
		OutMirroredKeys.Reserve(NumKeys);
		OutTangents.Reserve(NumKeys);
		OutHeights.Reserve(NumKeys);
	
		for (int32 Index = 0; Index < NumKeys; ++Index)
		{
			const FKeyHandle& KeyHandle = KeysInCurves[Index];
			if (!CurveModel)
			{
				continue;
			}

			FKeyAttributes KeyAttribute;
			CurveModel->GetKeyAttributesExcludingAutoComputed(
				TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyAttributes>(&KeyAttribute, 1)
				);

			const bool bHasValidMode = KeyAttribute.HasTangentMode()
				&& (KeyAttribute.GetTangentMode() == RCTM_User || KeyAttribute.GetTangentMode() == RCTM_Break);
			if (!bHasValidMode)
			{
				continue;
			}

			OutMirroredKeys.Add(KeyHandle);
			if (KeyAttribute.HasArriveTangent() && KeyAttribute.HasLeaveTangent())
			{
				OutTangents.Add({ KeyAttribute.GetArriveTangent(), KeyAttribute.GetLeaveTangent() });
			}
			else if (KeyAttribute.HasArriveTangent())
			{
				OutTangents.Add({ KeyAttribute.GetArriveTangent(), 0.0 });
			}

			FKeyPosition KeyPosition;
			CurveModel->GetKeyPositions(TConstArrayView<FKeyHandle>(&KeyHandle, 1), TArrayView<FKeyPosition>(&KeyPosition, 1));
			OutHeights.Add(KeyPosition.OutputValue);
		}
	
		OutInitialAttributes.SetNumUninitialized(OutMirroredKeys.Num());
		CurveModel->GetKeyAttributes(OutMirroredKeys, OutInitialAttributes);
}
}

bool FTangentSelectionFlattener::ResetFromSelection(const FCurveEditor& InCurveEditor)
{
	CachedCurveData.Empty(CachedCurveData.Num());
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InCurveEditor.GetSelection().GetAll())
	{
		AddTangents(InCurveEditor, Pair.Key, Pair.Value.AsArray());
	}
	return !CachedCurveData.IsEmpty();
}

bool FTangentSelectionFlattener::AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys)
{
	const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!CurveModel)
	{
		return false;
	}
		
	FMirrorableTangentInfo TangentInfo = FilterMirrorableTangents(InCurveEditor, InCurveId, InKeys);
	if (!TangentInfo)
	{
		return false;
	}
		
	const auto[Min, Max] = FindMinMaxHeight(TangentInfo.MirrorableKeys, *CurveModel);
	const double SelectionHeight = Max - Min;
	if (FMath::IsNearlyZero(SelectionHeight))
	{
		return false;
	}

	// We're going to have 2 fake edges: the moved "edge" is the height different of min and max values in selection. The 2nd is simply zero.
	// Effectively, we'll squish the tangents based on how much the selection is squished vertically. 0 height means tangent is 0.
	constexpr double MidpointEdgeHeight = 0;
	CachedCurveData.Add(InCurveId, FCurveTangentMirrorData(MoveTemp(TangentInfo), SelectionHeight, MidpointEdgeHeight));
	return true;
}

void FTangentSelectionFlattener::ComputeMirroringParallel(const FCurveEditor& InCurveEditor, bool bTopHasCrossedBottomEdge)
{
	// If the top edge has crossed the bottom edge since we were initialized, we need to mirror. In that case Alpha is in range [-1, 0].
	const double Sign = bTopHasCrossedBottomEdge ? -1 : 1;
	
	for (TPair<FCurveModelID, FCurveTangentMirrorData>& Pair : CachedCurveData)
	{
		FCurveModel* CurveModel = InCurveEditor.FindCurve(Pair.Key);
		if (!CurveModel)
		{
			continue;
		}
		
		FCurveTangentMirrorData& CurveData = Pair.Value;
		// Like described above, the tangents are squished as much as the selection height difference is squished. 
		const auto[Min, Max] = FindMinMaxHeight(CurveData.KeyHandles, *CurveModel);
		const double SelectionHeight = Max - Min;
		RecomputeMirroringParallel(InCurveEditor, Pair.Key, CurveData, SelectionHeight * Sign);
	}
}
}
