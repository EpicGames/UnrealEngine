// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Mirror/TangentSingleKeyFlattener.h"

#include "Misc/Mirror/MirrorUtils.h"
#include "Misc/Mirror/SingleKeyMirrorData.h"

namespace UE::CurveEditor
{
namespace TangentFlattenDetail
{
static FKeyNeighborData AnalyzeKey(const FKeyHandle& InKey, const FCurveModel& InCurveModel)
{
	TOptional<FKeyHandle> Before, After;
	InCurveModel.GetNeighboringKeys(InKey, Before, After);
	
	TArray<FKeyHandle, TInlineAllocator<3>> Keys;
	TArray<FKeyPosition, TInlineAllocator<3>> Positions;
	if (Before)
	{
		Keys.Add(*Before);
	}
	if (After)
	{
		Keys.Add(*After);
	}
	Keys.Add(InKey);
	Positions.SetNumUninitialized(Keys.Num());
	InCurveModel.GetKeyPositions(Keys, Positions);
	
	const int32 BeforeIndex = Before ? 0 : INDEX_NONE;
	const int32 AfterIndex = BeforeIndex + 1;
	const int32 CurrentIndex = Keys.Num() - 1;
	return FKeyNeighborData
	{
		Positions[CurrentIndex].OutputValue,
		Before ? Positions[BeforeIndex].OutputValue : TOptional<double>{},
		After ? Positions[AfterIndex].OutputValue : TOptional<double>{},
	};
}

/** Gets the neighbor to compute the initial interpolation distance to. */
static TOptional<ESide> GetInitialAnchorSide(const FKeyNeighborData& InData)
{
	return InData.LeftKeyHeight 
		? ESide::Left 
		: InData.RightKeyHeight ? ESide::Right : TOptional<ESide>{};
}

static ESide Opposite(ESide InSide) { return InSide == ESide::Left ? ESide::Right : ESide::Left; }

static TOptional<FCurveBounds> FindMinDistance(
	const FKeyNeighborData& InData, ESide InSide, const FKeyHandle& InKey, const FCurveModel& InCurveModel
	)
{
	const TOptional<double> HeightToUse = InData.Choose(InSide);
	if (!HeightToUse)
	{
		return {};
	}
	
	FKeyPosition CurrentKeyPosition{};
	InCurveModel.GetKeyPositions({ InKey } , MakeArrayView(&CurrentKeyPosition, 1));
	
	const double CurrentY = CurrentKeyPosition.OutputValue;
	return FCurveBounds{ FMath::Min(CurrentY, *HeightToUse), FMath::Max(CurrentY, *HeightToUse) };
}

static FCurveBounds FindDistanceToPointButDefaultToOtherPoint(
	const FKeyNeighborData& InData, ESide InPreferredSide, const FKeyHandle& InKey, const FCurveModel& InCurveModel
	)
{
	if (const TOptional<FCurveBounds> LeftBounds = FindMinDistance(InData, InPreferredSide, InKey, InCurveModel))
	{
		return *LeftBounds;
	}
	if (const TOptional<FCurveBounds> RightBounds = FindMinDistance(InData, Opposite(InPreferredSide), InKey, InCurveModel))
	{
		return *RightBounds;
	}
	return {};
}

/** @return Whether the center key is now on the other side of the neighbor than it was initially. */
static bool HasCenterFlipped(const FKeyNeighborData& InData, ESide InSide, const FKeyHandle& InKey, const FCurveModel& InCurveModel)
{
	const TOptional<double> HeightToUse = InData.Choose(InSide);
	if (!HeightToUse)
	{
		return false;
	}
	
	FKeyPosition CurrentKeyPosition{};
	InCurveModel.GetKeyPositions({ InKey } , MakeArrayView(&CurrentKeyPosition, 1));
	
	const double CurrentCenterHeight = CurrentKeyPosition.OutputValue;
	const bool bCenterWasInitiallyAbove = InData.CenterInitialHeight >= *HeightToUse;
	const bool bHasFlipped = (bCenterWasInitiallyAbove && CurrentCenterHeight < *HeightToUse) 
		|| (!bCenterWasInitiallyAbove && CurrentCenterHeight > *HeightToUse);
	return bHasFlipped;
}

static FCurveBounds ComputeDistanceToNeighbor(
	const FKeyNeighborData& InNeighbourData, const FKeyHandle& InKey, const FCurveModel& InCurveModel, ESide InAnchorPoint
	)
{
	TOptional<FKeyHandle> Before, After;
	InCurveModel.GetNeighboringKeys(InKey, Before, After);
	const TOptional<double> NeighbourHeight = InNeighbourData.Choose(InAnchorPoint);
	if (!NeighbourHeight)
	{
		return FCurveBounds{};
	}
	
	FKeyPosition CurrentPosition; 
	InCurveModel.GetKeyPositions({ InKey }, MakeArrayView(&CurrentPosition, 1));
	
	const double CurrentY = CurrentPosition.OutputValue;
	return FCurveBounds{ FMath::Min(CurrentY, *NeighbourHeight), FMath::Max(CurrentY, *NeighbourHeight) };
}

static TOptional<double> ComputeDeltaEdgeMovementForSingleKey(
	const FSingleKeyMirrorData& InMirrorData, const FCurveModel& InCurveModel,
	ESide InKeyToMirrorTo, ESide InAnchorPoint
	)
{
	// The neighbor key we're trying to flatten the tangent to does not exist? Do nothing. 
	const TOptional<double> TargetHeight = InMirrorData.NeighborData.Choose(InKeyToMirrorTo);
	if (!TargetHeight)
	{
		return {};
	}
	
	const TOptional<FCurveBounds> CurveBounds = ComputeDistanceToNeighbor(InMirrorData.NeighborData, InMirrorData.GetKeyHandle(), InCurveModel, InAnchorPoint);
	if (!ensure(CurveBounds))
	{
		return {};
	}
	
	const double DeltaEdge = CurveBounds->Delta();
	// The user is interpolating towards the key that we set up the interpolation distance...
	if (InKeyToMirrorTo == InAnchorPoint)
	{
		// ... yes: nothing to translate
		return DeltaEdge;
	}
	
	// ... no: the interpolation distance needs to be converted to the other key.
	const double InitialEdgeDistance = InMirrorData.InitialDistanceToAnchorPoint;
	const double AbsoluteMoved = FMath::Abs(DeltaEdge - InitialEdgeDistance);
	
	const double DeltaHeight = FMath::Abs(InMirrorData.NeighborData.CenterInitialHeight - *TargetHeight);
	if (FMath::IsNearlyZero(DeltaHeight))
	{
		return {};
	}
	
	const double MovedRelativeToRight = AbsoluteMoved / DeltaHeight;
	const double MovedAbsToLeft = InitialEdgeDistance - MovedRelativeToRight * InitialEdgeDistance;
	return MovedAbsToLeft;
}
}

bool FTangentSingleKeyFlattener::ResetFromSelection(const FCurveEditor& InCurveEditor)
{
	CachedCurveData.Empty(CachedCurveData.Num());
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InCurveEditor.GetSelection().GetAll())
	{
		AddTangents(InCurveEditor, Pair.Key, Pair.Value.AsArray());
	}
	return !CachedCurveData.IsEmpty();
}

bool FTangentSingleKeyFlattener::AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys)
{
	const FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!CurveModel)
	{
		return false;
	}
		
	FMirrorableTangentInfo TangentInfo = FilterMirrorableTangents(InCurveEditor, InCurveId, InKeys);
	if (TangentInfo.MirrorableKeys.Num() != 1)
	{
		return false;
	}
	
	const FKeyNeighborData KeyData = TangentFlattenDetail::AnalyzeKey(TangentInfo.MirrorableKeys[0], *CurveModel);
	const TOptional<ESide> InitialAnchorPoint = TangentFlattenDetail::GetInitialAnchorSide(KeyData);
	if (!InitialAnchorPoint)
	{
		return false;
	}
	
	const FKeyHandle CenterKey = InKeys[0];
	const auto[Min, Max] = TangentFlattenDetail::FindDistanceToPointButDefaultToOtherPoint(KeyData, *InitialAnchorPoint, CenterKey, *CurveModel);
	const double DistanceToAnchorPoint = Max - Min;
	if (FMath::IsNearlyZero(DistanceToAnchorPoint))
	{
		return false;
	}

	// We're going to have 2 fake edges: the height of the moved "edge" is the delta of min and max values in selection. The 2nd edge is 0.
	// Effectively, as the selection is squished, the virtual edges move closer. The closer the virtual edges, the more the tangents are squished vertically.
	// 0 height means tangent is 0.
	constexpr double MidpointEdgeHeight = 0;
	FCurveTangentMirrorData MirrorData(MoveTemp(TangentInfo), DistanceToAnchorPoint, MidpointEdgeHeight);
	CachedCurveData.Add(InCurveId, FSingleKeyMirrorData(CenterKey, KeyData, MoveTemp(MirrorData), *InitialAnchorPoint, DistanceToAnchorPoint));
	return true;
}

void FTangentSingleKeyFlattener::ComputeMirroringParallel(const FCurveEditor& InCurveEditor, ESide InKeyToMirrorTo)
{
	for (TPair<FCurveModelID, FSingleKeyMirrorData>& Pair : CachedCurveData)
	{
		FCurveModel* CurveModel = InCurveEditor.FindCurve(Pair.Key);
		if (!CurveModel)
		{
			continue;
		}
		
		FSingleKeyMirrorData& MirrorData = Pair.Value;
		
		// If the moved key has crossed the target key since, we need to mirror the tangent. In that case Alpha is in range [-1, 0].
		const bool bFlip = TangentFlattenDetail::HasCenterFlipped(MirrorData.NeighborData, InKeyToMirrorTo, MirrorData.GetKeyHandle(), *CurveModel);
		const double Sign = InKeyToMirrorTo == ESide::Right // Not quite sure why this is needed... need to investigate. Probably correct but cannot derive it right now.
			? 1 
			: bFlip ? -1 : 1;
		
		const TOptional<double> BlendHeight = TangentFlattenDetail::ComputeDeltaEdgeMovementForSingleKey(
			MirrorData, *CurveModel, InKeyToMirrorTo, MirrorData.AnchorKeySide
			);
		if (BlendHeight)
		{
			RecomputeMirroringParallel(InCurveEditor, Pair.Key, MirrorData.TangentMirrorData, *BlendHeight * Sign);
		}
	}
}
}