// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmartSnap.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "CurveEditor.h"

#include <cmath>

namespace UE::CurveEditor
{
bool CanSmartSnapSelection(const FCurveEditorSelection& InSelection)
{
	return Algo::AnyOf(InSelection.GetAll(), [](const TPair<FCurveModelID, FKeyHandleSet>& Pair)
	{
		const FKeyHandleSet& KeySelection = Pair.Value;
		return Algo::AnyOf(KeySelection.AsArray(), [&KeySelection](const FKeyHandle& InHandle)
		{
			return KeySelection.PointType(InHandle) == ECurvePointType::Key;
		});
	});
}
	
namespace SmartSnapDetail
{
static TArray<FKeyHandle> ReturnOnlyKeys(const FKeyHandleSet& InSelection)
{
	TArray<FKeyHandle> Result;
	Result.Reserve(InSelection.Num());
	for (const FKeyHandle& KeyHandle : InSelection.AsArray())
	{
		if (InSelection.PointType(KeyHandle) == ECurvePointType::Key)
		{
			Result.Add(KeyHandle);
		}
	}
	return Result;
}

static FFrameRate GetCurveEditorFrameRate(const FCurveEditor& InCurveEditor)
{
	const TSharedPtr<ITimeSliderController> Controller = InCurveEditor.GetTimeSliderController();
	return Controller ? Controller->GetDisplayRate() : FFrameRate{};
}
}
	
void EnumerateSmartSnappableKeys(
	const FCurveEditor& InCurveEditor,
	const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn,
	TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect,
	const TFunctionRef<void(const FCurveModelID& CurveModelId, FCurveModel& CurveModel, const FSmartSnapResult& SnapResult)>& InProcessSmartSnapping
	)
{
	const FFrameRate FrameRate = SmartSnapDetail::GetCurveEditorFrameRate(InCurveEditor);
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		FCurveModel* CurveModel = InCurveEditor.FindCurve(Pair.Key);
		if (!CurveModel)
		{
			continue;
		}

		// Exclude tangent handles
		const FKeyHandleSet& KeySelection = Pair.Value;
		const bool bSelectionContainsOnlyKeys = Algo::AllOf(KeySelection.AsArray(), [&KeySelection](const FKeyHandle& InHandle)
		{
			return KeySelection.PointType(InHandle) == ECurvePointType::Key;
		});
		// Avoid TArray allocation when user has only selected key handles
		const TArray<FKeyHandle> FilteredKeys = bSelectionContainsOnlyKeys ? TArray<FKeyHandle>{} : SmartSnapDetail::ReturnOnlyKeys(KeySelection);
		const TConstArrayView<FKeyHandle> Keys = bSelectionContainsOnlyKeys ? KeySelection.AsArray() : FilteredKeys;
		const int32 NumKeys = Keys.Num();
		if (NumKeys == 0)
		{
			continue;
		}
		
		TArray<FKeyPosition> Positions;
		Positions.SetNumUninitialized(NumKeys);
		CurveModel->GetKeyPositions(Keys, Positions);

		const FSmartSnapResult SnappingResult = ComputeSmartSnap(*CurveModel, Keys, Positions, FrameRate);
		for (const FKeyHandle& InHandle : SnappingResult.UpdatedKeys)
		{
			OutKeysToSelect.FindOrAdd(Pair.Key).Add(InHandle, ECurvePointType::Key);
		}
		InProcessSmartSnapping(Pair.Key, *CurveModel, SnappingResult);
	}
}

namespace SmartSnapDetail
{
struct FFrameData
{
	FKeyHandle ClosestHandle = FKeyHandle::Invalid();
	FFrameTime AbsDistToFrame;
};

static TMap<FFrameNumber, FFrameData> ComputeClosestFrames(
	TConstArrayView<FKeyHandle> InHandles, TConstArrayView<FKeyPosition> InPositions, const FFrameRate& InFrameRate
	)
{
	TMap<FFrameNumber, FFrameData> FrameToData;
	for (int32 Index = 0; Index < InHandles.Num(); ++Index)
	{
		const FKeyHandle& KeyHandle = InHandles[Index];
		const FFrameTime SubFrame = InFrameRate.AsFrameTime(InPositions[Index].InputValue);
		const FFrameTime FrameNumber = SubFrame.RoundToFrame();
		
		const FFrameTime AbsDistToFrame = FMath::Max(SubFrame, FrameNumber) - FMath::Min(SubFrame, FrameNumber);
		FFrameData& FrameData = FrameToData.FindOrAdd(FrameNumber.FrameNumber, FFrameData{ KeyHandle, AbsDistToFrame });
		const bool bIsFirstKeyOnFrame = FrameData.ClosestHandle == KeyHandle; 
		if (bIsFirstKeyOnFrame)
		{
			continue;
		}
		
		const bool bIsCloserToFrame = AbsDistToFrame < FrameData.AbsDistToFrame;
		if (bIsCloserToFrame)
		{
			FrameData.ClosestHandle = KeyHandle;
			FrameData.AbsDistToFrame = AbsDistToFrame;
		}
	}
	return FrameToData;
}
static FSmartSnapResult MoveKeysOntoFrames(
	const FCurveModel& InModel, const TMap<FFrameNumber, FFrameData>& FrameToData, const FFrameRate& InFrameRate
	)
{
	FSmartSnapResult Result;
	for (const TPair<FFrameNumber, FFrameData>& Frame : FrameToData)
	{
		const FKeyHandle& KeyHandle = Frame.Value.ClosestHandle;
		const FFrameTime FrameNumber = Frame.Key;
		
		FKeyPosition Position;
		Position.InputValue = InFrameRate.AsSeconds(FrameNumber);
		InModel.Evaluate(Position.InputValue, Position.OutputValue);
		Result.NewPositions.Add(Position);
		Result.UpdatedKeys.Add(KeyHandle);
	}
	return Result;
}
}

FSmartSnapResult ComputeSmartSnap(
	const FCurveModel& InModel, TConstArrayView<FKeyHandle> InHandles, TConstArrayView<FKeyPosition> InPositions, const FFrameRate& InFrameRate
	)
{
	check(InHandles.Num() == InPositions.Num());

	// 1. We'll compute all the frames covered by the keys, and the single key that is closest to it.
	// Example: If key 1 is at 2.6 and key 2 at 2.7, we'd move key 2 to frame 3.0.
	// This retains the shape of the curve a bit better (as opposed to taking a 'random' one without criteria).
	const TMap<FFrameNumber, SmartSnapDetail::FFrameData> FrameToData = SmartSnapDetail::ComputeClosestFrames(InHandles, InPositions, InFrameRate);
	// 2. The key closest to its assigned frame is moved there by evaluating the curve.
	FSmartSnapResult Result = MoveKeysOntoFrames(InModel, FrameToData, InFrameRate);

	// 3.1 All keys were moved? Done.
	const int32 NumToRemove = InHandles.Num() - Result.NewPositions.Num();
	if (!NumToRemove)
	{
		return Result;
	}
	// 3.2 All keys that were not moved are removed.
	Result.RemovedKeys.Reserve(NumToRemove);
	for (const FKeyHandle& Handle : InHandles)
	{
		if (!Result.UpdatedKeys.Contains(Handle))
		{
			Result.RemovedKeys.Add(Handle);
		}
	}
	
	return Result;
}

void ApplySmartSnap(FCurveModel& InModel, const FSmartSnapResult& InSmartSnap, double InCurrentTime)
{
	InModel.RemoveKeys(InSmartSnap.RemovedKeys, InCurrentTime);
	InModel.SetKeyPositions(InSmartSnap.UpdatedKeys, InSmartSnap.NewPositions);
}
}
