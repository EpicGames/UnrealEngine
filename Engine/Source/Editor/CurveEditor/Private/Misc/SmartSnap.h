// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Curves/KeyHandle.h"
#include "Containers/Array.h"
#include "Templates/FunctionFwd.h"

struct FKeyHandleSet;
class FCurveEditor;
class FCurveModel;
class ITimeSliderController;
struct FCurveEditorSelection;
struct FCurveModelID;

namespace UE::CurveEditor
{
/** @return Whether the selection contains keys that can be snapped. */
bool CanSmartSnapSelection(const FCurveEditorSelection& InSelection);

struct FSmartSnapResult
{
	TArray<FKeyHandle> RemovedKeys;
	TArray<FKeyHandle> UpdatedKeys;
	TArray<FKeyPosition> NewPositions;
};

/** Calls InProcessSmartSnapping for each curve on which smart snapping can be performed. */
void EnumerateSmartSnappableKeys(
	const FCurveEditor& InCurveEditor,
	const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn,
	TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect,
	const TFunctionRef<void(const FCurveModelID& CurveModelId, FCurveModel& CurveModel, const FSmartSnapResult& SnapResult)>& InProcessSmartSnapping
	);
	
/** Computes how to modify the curve model: Tries to snap keys to the closest whole frame placing the key on the intersection of curve and an imaginary vertical frame.*/
FSmartSnapResult ComputeSmartSnap(
	const FCurveModel& InModel, TConstArrayView<FKeyHandle> InHandles, TConstArrayView<FKeyPosition> InPositions, const FFrameRate& InFrameRate
	);

/** Applies the computed smart snapping to InModel. */
void ApplySmartSnap(
	FCurveModel& InModel, const FSmartSnapResult& InSmartSnap, double CurrentTime = 0.0
	);
}
