// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/CurveEditorCommandChange.h"

#include "Containers/UnrealString.h"
#include "CurveEditorTransactionObject.h"

namespace UE::CurveEditor
{
bool FCurveEditorCommandChange::HasExpired(UObject* InObject) const
{
	return !GetCurveEditor().IsValid();
}

FString FCurveEditorCommandChange::ToString() const
{
	return TEXT("FCurveEditorCommandChange");
}

TSharedPtr<FCurveEditor> FCurveEditorCommandChange::GetCurveEditor() const
{
	return WeakCurveEditor.Pin();
}
}
