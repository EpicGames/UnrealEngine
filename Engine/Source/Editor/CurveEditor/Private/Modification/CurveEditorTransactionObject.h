// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SharedPointer.h"
#include "CurveEditorTransactionObject.generated.h"

class FCurveEditor;

/**
 * The object type that the transaction system associates FCommandChanges for the FCurveEditor with.
 *
 * By design, the transaction system requires a UObject to associate FCommandChanges with. This is usually an asset.
 * Conceptually, instances of this class will act as an "asset" and the associated FCurveEditor contains the "asset data", i.e. its  FCurveModels.
 */
UCLASS(MinimalAPI)
class UCurveEditorTransactionObject : public UObject
{
	GENERATED_BODY()
public:

	/** The curve editor this object is associated with. Used by commands to retrieve the */
	TWeakPtr<FCurveEditor> OwningCurveEditor;
};
