// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveAttributeChangeUtils.h"

#include "CurveModel.h"
#include "Modification/Keys/Data/CurveAttributeChangeData.h"

namespace UE::CurveEditor::CurveAttributes
{
FCurveAttributeChangeData_PerCurve Diff(const FCurveAttributes& InOriginal, const FCurveAttributes& InTarget)
{
	return FCurveAttributeChangeData_PerCurve{ InOriginal, InTarget };
}

void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FCurveAttributeChangeData& InDeltaChange)
{
	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : InCurves)
	{
		if (const FCurveAttributeChangeData_PerCurve* Change = InDeltaChange.ChangeData.Find(Pair.Key))
		{
			Pair.Value->SetCurveAttributes(Change->AfterChange);
		}
	}
}

void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FCurveAttributeChangeData& InDeltaChange)
{
	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : InCurves)
	{
		if (const FCurveAttributeChangeData_PerCurve* Change = InDeltaChange.ChangeData.Find(Pair.Key))
		{
			Pair.Value->SetCurveAttributes(Change->BeforeChange);
		}
	}
}
}
