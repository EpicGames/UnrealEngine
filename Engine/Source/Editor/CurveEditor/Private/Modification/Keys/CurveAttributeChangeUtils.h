// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
namespace UE::CurveEditor { struct FCurveAttributeChangeData; }
namespace UE::CurveEditor { struct FCurveAttributeChangeData_PerCurve; }
struct FCurveAttributes;
struct FCurveModelID;

namespace UE::CurveEditor::CurveAttributes
{
/**
 * Computes the delta change to get from InOriginal to InTarget.
 * 
 * @param InOriginal The original curve attributes.
 * @param InTarget The new curve attributes.
 */
FCurveAttributeChangeData_PerCurve Diff(const FCurveAttributes& InOriginal, const FCurveAttributes& InTarget);

/** Applies InDeltaChange to InCurves (redo). */
void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FCurveAttributeChangeData& InDeltaChange);
/** Reverts InDeltaChange from InCurves (undo). */
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FCurveAttributeChangeData& InDeltaChange);
}
