// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Keys/CurveChangeDiff.h"

#include "CurveAttributeChangeUtils.h"
#include "CurveEditor.h"
#include "CurveEditorTrace.h"
#include "CurveModel.h"
#include "KeyAttributeChangeUtils.h"
#include "KeyExistenceUtils.h"
#include "Misc/ScopeExit.h"
#include "Modification/Keys/Data/CurveKeyDataMap.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/ECurveChangeFlags.h"
#include "Modification/Keys/MoveKeysChangeUtils.h"
#include "SCurveEditor.h"
#include "Diff/AllCurvesGenericChangeBuilder.h"
#include "Modification/Keys/Diff/IMultiCurveChangeVisitor.h"
#include "Modification/Keys/Diff/ISingleCurveChangeVisitor.h"

namespace UE::CurveEditor
{
namespace Private
{
static void HandleOp_MoveKeys(
	const ECurveChangeFlags Operations, const FCurveModel& CurveModel, const FCurveKeyData& SavedState,
	ISingleCurveChangeVisitor& InCurveChangeVisitor
	)
{
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::MoveKeys) || !SavedState || !ensureAlways(SavedState.HasKeyPositions()))
	{
		return;
	}

	TArray<FKeyPosition> CurrentPositions;
	CurrentPositions.SetNum(SavedState.KeyHandles.Num());
	CurveModel.GetKeyPositions(SavedState.KeyHandles, CurrentPositions);
			
	if (FMoveKeysChangeData_PerCurve PerCurveData = MoveKeys::Diff(SavedState.KeyHandles, SavedState.KeyPositions, CurrentPositions))
	{
		InCurveChangeVisitor.ProcessMoveKeys(MoveTemp(PerCurveData));
	}
}

static void HandleOp_AddedKeys(
	const ECurveChangeFlags Operations, const FCurveModel& CurveModel, const FCurveKeyData& SavedState,
	ISingleCurveChangeVisitor& InCurveChangeVisitor
)
{
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::AddKeys))
	{
		return;
	}

	if (FCurveKeyData AddedKeys = KeyInsertion::Diff(SavedState.KeyHandles, CurveModel))
	{
		InCurveChangeVisitor.ProcessAddKeys(MoveTemp(AddedKeys));
	}
}

static TOptional<FCurveKeyData> ComputeRemovedKeys(const ECurveChangeFlags Operations, const FCurveModel& CurveModel, const FCurveKeyData& SavedState)
{
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::RemoveKeys) || !SavedState
		|| !ensureAlways(SavedState.HasKeyPositions()) || !ensureAlways(SavedState.HasKeyAttributes()))
	{
		return {};
	}

	FCurveKeyData Removed = KeyRemoval::Diff(SavedState.KeyHandles, SavedState.KeyPositions, SavedState.KeyAttributes, CurveModel);
	return Removed ? Removed : TOptional<FCurveKeyData>();
}
	
static void HandleOp_KeyAttributes(
	const ECurveChangeFlags Operations, const FCurveModel& CurveModel, const FCurveKeyData& SavedState,
	ISingleCurveChangeVisitor& InCurveChangeVisitor
)
{
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::KeyAttributes) || !SavedState || !ensureAlways(SavedState.HasKeyAttributes()))
	{
		return;
	}

	TArray<FKeyAttributes> CurrentAttributes;
	CurrentAttributes.SetNum(SavedState.KeyHandles.Num());
	CurveModel.GetKeyAttributesExcludingAutoComputed(SavedState.KeyHandles, CurrentAttributes);

	if (FKeyAttributeChangeData_PerCurve PerCurveData = KeyAttributes::Diff(SavedState.KeyHandles, SavedState.KeyAttributes, CurrentAttributes))
	{
		InCurveChangeVisitor.ProcessKeyAttributesChange(MoveTemp(PerCurveData));
	}
}

static void HandleOp_CurveAttributes(
	const ECurveChangeFlags Operations, const FCurveModel& CurveModel, const FCurveDiffingData& SavedState,
	ISingleCurveChangeVisitor& InCurveChangeVisitor
)
{
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::CurveAttributes) || !SavedState || !ensureAlways(SavedState.HasCurveAttributes()))
	{
		return;
	}

	FCurveAttributes CurrentAttributes;
	CurveModel.GetCurveAttributes(CurrentAttributes);
	
	if (FCurveAttributeChangeData_PerCurve PerCurveData = CurveAttributes::Diff(*SavedState.CurveAttributes, CurrentAttributes))
	{
		InCurveChangeVisitor.ProcessCurveAttributesChange(MoveTemp(PerCurveData));
	}
}

/** Returns a copy of InSavedState minus the keys in InRemovedKeys.  */
static FCurveKeyData CleanseOfRemovedKeys(
	TConstArrayView<FKeyHandle> InRemovedKeys, const FCurveKeyData& InSavedState
	)
{
	SCOPED_CURVE_EDITOR_TRACE(CleanseRemovedKeys);
	const int32 TargetNum =  InSavedState.KeyHandles.Num() - InRemovedKeys.Num();
	if (TargetNum <= 0)
	{
		return {};
	}
	
	FCurveKeyData Result;
	Result.KeyHandles.Reserve(TargetNum);
	const bool bDoAttributes = InSavedState.HasKeyAttributes();
	if (bDoAttributes)
	{
		Result.KeyAttributes.Reserve(TargetNum);
	}
	const bool bDoPositions = InSavedState.HasKeyPositions();
	if (bDoPositions)
	{
		Result.KeyPositions.Reserve(TargetNum);
	} 
	
	for (int32 Index = 0; Index < InSavedState.KeyHandles.Num() && Result.KeyHandles.Num() < TargetNum; ++Index)
	{
		const FKeyHandle KeyHandle = InSavedState.KeyHandles[Index];
		if (InRemovedKeys.Contains(KeyHandle))
		{
			continue;
		}

		Result.KeyHandles.Add(KeyHandle);
		if (bDoAttributes)
		{
			Result.KeyAttributes.Add(InSavedState.KeyAttributes[Index]);
		}
		if (bDoPositions)
		{
			Result.KeyPositions.Add(InSavedState.KeyPositions[Index]);
		}
	}
	return Result;
}

/** Checks whether any keys in InSavedState were removed from CurveModel and if so, returns a copy of the state minus the removed key data. */
static TOptional<FCurveKeyData> CleanseStateIfKeysWereRemoved(
	const FCurveKeyData& InSavedState, const ECurveChangeFlags Operations, const TOptional<FCurveKeyData>& InComputedKeyRemovalData,
	const FCurveModel& CurveModel
	)
{
	if (InComputedKeyRemovalData)
	{
		return CleanseOfRemovedKeys(InComputedKeyRemovalData->KeyHandles, InSavedState);
	}

	// If InKeyRemovalData contained no removal data but RemoveKeys is set, that means we already analyzed removal and don't have to do it again.
	// Effectively, this implies that no keys have been removed.
	if (!EnumHasAnyFlags(Operations, ECurveChangeFlags::RemoveKeys))
	{
		const TArray<FKeyHandle> RemovedKeys = KeyRemoval::Diff(InSavedState.KeyHandles, CurveModel);
		return CleanseOfRemovedKeys(RemovedKeys, InSavedState);
	}
	
	return {};
}
}
	
FGenericCurveChangeData ComputeDiff(const FCurveEditor& InCurveEditor, const ECurveChangeFlags InOperations, const FCurvesSnapshot& InSnapshot)
{
	FAllCurvesGenericChangeBuilder Builder(InCurveEditor);
	ProcessChanges(InCurveEditor, InOperations, InSnapshot, Builder);
	return Builder.CurveChangeData;
}
	
void ProcessChanges(
	const FCurveEditor& InCurveEditor, const ECurveChangeFlags InOperations, const FCurvesSnapshot& InSnapshot,
	IMultiCurveChangeVisitor& InChangeVisitor
	)
{
	SCOPED_CURVE_EDITOR_TRACE(ComputeDiff);
	
	InChangeVisitor.PreProcessChanges();
	ON_SCOPE_EXIT{ InChangeVisitor.PostProcessChanges(); };
	
	FGenericCurveChangeData Change;
	for (const TPair<FCurveModelID, FCurveDiffingData>& StatePair : InSnapshot.CurveData)
	{
		const FCurveModelID& CurveId = StatePair.Key;
		const FCurveDiffingData& SavedState = StatePair.Value;
		InChangeVisitor.ProcessChange(CurveId, [&InCurveEditor, InOperations, &CurveId, &SavedState]
			(ISingleCurveChangeVisitor& InSingleCurveChangeVisitor)
		{
			const FCurveModel* CurveModel = InCurveEditor.FindCurve(CurveId);
			if (!CurveModel)
			{
				return;
			}

			Private::HandleOp_CurveAttributes(InOperations, *CurveModel, SavedState, InSingleCurveChangeVisitor);
			Private::HandleOp_AddedKeys(InOperations, *CurveModel, SavedState, InSingleCurveChangeVisitor);
			
			// Compute removed keys before MoveKeys and KeyAttributes... 
			TOptional<FCurveKeyData> RemovedKeys = Private::ComputeRemovedKeys(InOperations, *CurveModel, SavedState);
			// ... so we can reuse that info to tell MoveKeys and KeyAttributes to skip analyzing the keys that were removed.
			const TOptional<FCurveKeyData> StateMinusRemovedKeys = Private::CleanseStateIfKeysWereRemoved(
				SavedState, InOperations, RemovedKeys, *CurveModel
				);
			if (RemovedKeys)
			{
				// Now that CleanseStateIfKeysWereRemoved has used RemovedKeys, we can MoveTemp it and pass it to the visitor.
				InSingleCurveChangeVisitor.ProcessRemoveKeys(MoveTemp(*RemovedKeys));
			}
				
			if (StateMinusRemovedKeys)
			{
				// The operations should ignore the keys that were removed since that data is now irrelevant.
				// Not ignoring would cause them to report a change when there is none.
				// Example: If we didn't cleanse keys removed by the operation, HandleOp_MoveKeys would try to call GetKeyPositions on FKeyHandles
				// that were removed by the operation.
				// GetKeyPositions would return default-initialized FKeyPositions. It would then proceed to compare the default-initialized state
				// against the key value from before the change, and conclude that the keys removed by the operation were "moved" (false change).
				Private::HandleOp_MoveKeys(InOperations, *CurveModel, *StateMinusRemovedKeys, InSingleCurveChangeVisitor);
				Private::HandleOp_KeyAttributes(InOperations, *CurveModel, *StateMinusRemovedKeys, InSingleCurveChangeVisitor);
			}
			else
			{
				// Nothing was removed, so just pass in the same state
				Private::HandleOp_MoveKeys(InOperations, *CurveModel, SavedState, InSingleCurveChangeVisitor);
				Private::HandleOp_KeyAttributes(InOperations, *CurveModel, SavedState, InSingleCurveChangeVisitor);
			}
		});
	}
}

FGenericCurveChangeData FCurveChangeDiff::ComputeDiff() const
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	return CurveEditorPin ? CurveEditor::ComputeDiff(*CurveEditorPin, Operations, Snapshot) : FGenericCurveChangeData();
	
}

void FCurveChangeDiff::ProcessDiffs(IMultiCurveChangeVisitor& InChangeBuilder) const
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		ProcessChanges(*CurveEditorPin, Operations, Snapshot, InChangeBuilder);
	}
}
}
