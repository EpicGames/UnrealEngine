// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Keys/CurvesSnapshotBuilder.h"

#include "CurveEditor.h"
#include "CurveEditorTrace.h"

namespace UE::CurveEditor
{
namespace Private
{
static bool CanTrackCurve(const FCurveModel& Model)
{
	// Read only curves cannot be modified so it is pointless to track them
	return !Model.IsReadOnly();
}

static void AddPositionsAndAttributes(const FCurveModel& InCurveModel, FCurveDiffingData& InSavedState, ECurveChangeFlags InOperations)
{
	if (EnumHasAnyFlags(InOperations, ECurveChangeFlags::MoveKeys | ECurveChangeFlags::RemoveKeys))
	{
		InSavedState.KeyPositions.SetNum(InSavedState.KeyHandles.Num());
		InCurveModel.GetKeyPositions(InSavedState.KeyHandles, InSavedState.KeyPositions);
	}

	if (EnumHasAnyFlags(InOperations, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::RemoveKeys))
	{
		InSavedState.KeyAttributes.SetNum(InSavedState.KeyHandles.Num());
		InCurveModel.GetKeyAttributesExcludingAutoComputed(InSavedState.KeyHandles, InSavedState.KeyAttributes);
	}

	if (EnumHasAnyFlags(InOperations, ECurveChangeFlags::CurveAttributes))
	{
		InSavedState.CurveAttributes.Emplace();
		InCurveModel.GetCurveAttributes(*InSavedState.CurveAttributes);
	}
}
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackCurve(const FCurveModelID& InCurveId)
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackCurve);
	
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	const FCurveModel* CurveModel = CurveEditorPin ? CurveEditorPin->FindCurve(InCurveId) : nullptr;
	if (CurveModel && Private::CanTrackCurve(*CurveModel))
	{
		// Allow overwriting of previous data.
		FCurveDiffingData& SavedState = Snapshot.CurveData.Add(InCurveId);
		CurveModel->GetAllKeys(SavedState.KeyHandles);
		Private::AddPositionsAndAttributes(*CurveModel, SavedState, CapturedDataFlags);
	}
	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackCurves(TConstArrayView<FCurveModelID> InCurves)
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackCurves);
	for (const FCurveModelID& CurveId : InCurves)
	{
		TrackCurve(CurveId);
	}
	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackCurves(const TSet<FCurveModelID>& InCurves)
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackCurves);
	for (const FCurveModelID& CurveId : InCurves)
	{
		TrackCurve(CurveId);
	}
	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackCurve(const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> Keys)
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackCurve);
	// Add keys needs to compare against all keys to properly detect added keys.
	if (!ensure(!EnumHasAnyFlags(CapturedDataFlags, ECurveChangeFlags::AddKeys)))
	{
		return *this;
	}
	
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	const FCurveModel* CurveModel = CurveEditorPin ? CurveEditorPin->FindCurve(InCurveId) : nullptr;
	if (CurveModel && Private::CanTrackCurve(*CurveModel))
	{
		// Allow overwriting of previous data.
		FCurveDiffingData& SavedState = Snapshot.CurveData.Add(InCurveId);

		SavedState.KeyHandles = TArray<FKeyHandle>(Keys);
		Private::AddPositionsAndAttributes(*CurveModel, SavedState, CapturedDataFlags);
	}
	
	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackSelectedKeys()
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackSelectedKeys);
	// Add keys needs to compare against all keys to properly detect added keys.
	if (!ensure(!EnumHasAnyFlags(CapturedDataFlags, ECurveChangeFlags::AddKeys)))
	{
		return *this;
	}

	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!ensure(CurveEditorPin))
	{
		return *this;
	}

	const TMap<FCurveModelID, FKeyHandleSet>& SelectionMap = CurveEditorPin->GetSelection().GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectionMap)
	{
		TrackCurve(Pair.Key, Pair.Value.AsArray());
	}

	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackSelectedCurves()
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackSelectedCurves);
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!ensure(CurveEditorPin))
	{
		return *this;
	}

	const TMap<FCurveModelID, FKeyHandleSet>& SelectionMap = CurveEditorPin->GetSelection().GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectionMap)
	{
		TrackCurve(Pair.Key);
	}

	return *this;
}

FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackAllCurves()
{
	SCOPED_CURVE_EDITOR_TRACE(FCurvesSnapshotBuilder::TrackAllCurves);
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		TrackCurves(CurveEditorPin->GetCurves());
	}
	return *this;
}
}
