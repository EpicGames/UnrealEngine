// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllCurvesGenericChangeBuilder.h"

#include "AppendPerCurveGenericChangeVisitor.h"
#include "Modification/Keys/Diff/ISingleCurveChangeVisitor.h"

namespace UE::CurveEditor
{
void FAllCurvesGenericChangeBuilder::ProcessChange(
	const FCurveModelID& InCurveModel, TFunctionRef<void(ISingleCurveChangeVisitor& InVisitor)> InProcessCallback
	)
{
	FAppendPerCurveGenericChangeVisitor Visitor(InCurveModel, CurveChangeData);
	InProcessCallback(Visitor);
}
}
