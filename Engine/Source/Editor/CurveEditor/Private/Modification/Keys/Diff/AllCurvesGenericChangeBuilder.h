// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/Diff/IMultiCurveChangeVisitor.h"

class FCurveEditor;

namespace UE::CurveEditor
{
/** Aggregates all curve changes into a single FGenericCurveChangeData. */
class FAllCurvesGenericChangeBuilder : public IMultiCurveChangeVisitor
{
public:

	FGenericCurveChangeData CurveChangeData;

	explicit FAllCurvesGenericChangeBuilder(const FCurveEditor& InCurveEditor UE_LIFETIMEBOUND) : CurveEditor(InCurveEditor) {}
	
	//~ Begin ICurvesDiffer Interface
	virtual void ProcessChange(const FCurveModelID& InCurveModel, TFunctionRef<void(ISingleCurveChangeVisitor&)> InProcessCallback) override;
	//~ End ICurvesDiffer Interface

private:

	/** Curve editor instance. Caller is responsible for keeping it alive. */
	const FCurveEditor& CurveEditor;
};
}


