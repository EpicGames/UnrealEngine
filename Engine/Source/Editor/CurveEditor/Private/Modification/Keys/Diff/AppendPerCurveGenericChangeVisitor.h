// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/Diff/ISingleCurveChangeVisitor.h"

namespace UE::CurveEditor
{
class FAppendPerCurveGenericChangeVisitor : public ISingleCurveChangeVisitor
{
	const FCurveModelID CurveId;
	FGenericCurveChangeData& CurveChangeData;
public:

	explicit FAppendPerCurveGenericChangeVisitor(const FCurveModelID InCurveId, FGenericCurveChangeData& InCurveChangeData UE_LIFETIMEBOUND)
		: CurveId(InCurveId)
		, CurveChangeData(InCurveChangeData)
	{}

	virtual void ProcessMoveKeys(FMoveKeysChangeData_PerCurve&& InData) override
	{
		CurveChangeData.MoveKeysData.ChangedCurves.Add(CurveId, MoveTemp(InData));
	}

	virtual void ProcessAddKeys(FCurveKeyData&& InData) override
	{
		CurveChangeData.AddKeysData.SavedCurveState.Add(CurveId, MoveTemp(InData));
	}

	virtual void ProcessRemoveKeys(FCurveKeyData&& InData) override
	{
		CurveChangeData.RemoveKeysData.SavedCurveState.Add(CurveId, MoveTemp(InData));
	}

	virtual void ProcessKeyAttributesChange(FKeyAttributeChangeData_PerCurve&& InData) override
	{
		CurveChangeData.KeyAttributeData.ChangedCurves.Add(CurveId, MoveTemp(InData));
	}

	virtual void ProcessCurveAttributesChange(FCurveAttributeChangeData_PerCurve&& InData) override
	{
		CurveChangeData.CurveAttributeData.ChangeData.Add(CurveId, MoveTemp(InData));
	}
};
}
