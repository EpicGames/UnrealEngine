// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiGenericCommandAppender.h"

#include "CurveEditor.h"
#include "AppendPerCurveGenericChangeVisitor.h"
#include "Modification/Keys/GenericCurveChangeCommand.h"

namespace UE::CurveEditor
{
namespace Private
{
static void AppendChanges(const FCurveModelID& InCurveId, FGenericCurveChangeData& InOutTarget, FGenericCurveChangeData&& InAppendedChanges)
{
	if (FMoveKeysChangeData_PerCurve* Change_Move = InAppendedChanges.MoveKeysData.ChangedCurves.Find(InCurveId))
	{
		InOutTarget.MoveKeysData.ChangedCurves.Add(InCurveId, MoveTemp(*Change_Move));
	}
	if (FCurveKeyData* Change_Add = InAppendedChanges.AddKeysData.SavedCurveState.Find(InCurveId))
	{
		InOutTarget.AddKeysData.SavedCurveState.Add(InCurveId, MoveTemp(*Change_Add));
	}
	if (FCurveKeyData* Change_Remove = InAppendedChanges.RemoveKeysData.SavedCurveState.Find(InCurveId))
	{
		InOutTarget.RemoveKeysData.SavedCurveState.Add(InCurveId, MoveTemp(*Change_Remove));
	}
	if (FKeyAttributeChangeData_PerCurve* Change_Move = InAppendedChanges.KeyAttributeData.ChangedCurves.Find(InCurveId))
	{
		InOutTarget.KeyAttributeData.ChangedCurves.Add(InCurveId, MoveTemp(*Change_Move));
	}
	if (FCurveAttributeChangeData_PerCurve* Change_Move = InAppendedChanges.CurveAttributeData.ChangeData.Find(InCurveId))
	{
		InOutTarget.CurveAttributeData.ChangeData.Add(InCurveId, MoveTemp(*Change_Move));
	}
}
}
	
void FMultiGenericCommandAppender::ProcessChange(const FCurveModelID& InCurveModel, TFunctionRef<void(ISingleCurveChangeVisitor&)> InProcessCallback)
{
	FGenericCurveChangeData CurveChanges;
	FAppendPerCurveGenericChangeVisitor Visitor(InCurveModel, CurveChanges);
	InProcessCallback(Visitor);
	
	if (CurveChanges)
	{
		ProcessDiff(InCurveModel, MoveTemp(CurveChanges));
	}
}

void FMultiGenericCommandAppender::PostProcessChanges()
{
	// If there were any FCurveModels that are not bound to any UObject, just append to the dummy transaction object.
	// There is no package so we don't need to worry about any dirty flags.
	if (ChangesToCurvesWithoutUObject)
	{
		AllocationSize += ChangesToCurvesWithoutUObject.GetAllocatedSize() + sizeof(FGenericCurveChangeCommand);
		CurveEditor->GetTransactionManager()->AppendCurveChange(MoveTemp(ChangesToCurvesWithoutUObject));
	}
}

void FMultiGenericCommandAppender::ProcessDiff(const FCurveModelID& InCurveId, FGenericCurveChangeData&& InData)
{
	check(InData.HasChanges());
	
	FCurveModel* CurveModel = CurveEditor->FindCurve(InCurveId);
	if (!CurveModel)
	{
		return;
	}

	InData.Shrink();
	
	if (UObject* OwningObject = CurveModel->GetOwningObject())
	{
		AllocationSize += InData.GetAllocatedSize() + sizeof(FGenericCurveChangeCommand);
		CurveEditor->GetTransactionManager()->AppendCurveChange(OwningObject, MoveTemp(InData));
	}
	else
	{
		Private::AppendChanges(InCurveId, ChangesToCurvesWithoutUObject, MoveTemp(InData));
	}
}
}
