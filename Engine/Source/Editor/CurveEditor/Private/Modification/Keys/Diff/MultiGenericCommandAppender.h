// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/Diff/IMultiCurveChangeVisitor.h"

class FCurveEditor;

namespace UE::CurveEditor
{
class FTransactionManager;
struct FGenericCurveChangeData;

/**
 * Handles appending FGenericCurveChangeCommands to the transaction system.
 * - StoreUndo is called for each FCurveModel that returns non-null from FCurveModel::GetOwningObject.
 * These objects will have their package's dirty flag stored to the undo buffer.
 * - One catch all FGenericCurveChangeCommand is created for all other FCurveModels that returned null from FCurveModel::GetOwningObject.
 */
class FMultiGenericCommandAppender : public IMultiCurveChangeVisitor
{
public:

	explicit FMultiGenericCommandAppender(TSharedPtr<FCurveEditor> InCurveEditor)
		: CurveEditor(MoveTemp(InCurveEditor))
	{}

	SIZE_T GetAllocationSize() const { return AllocationSize; }

	//~ Begin ICurvesDiffer Interface
	virtual void ProcessChange(const FCurveModelID& InCurveModel, TFunctionRef<void(ISingleCurveChangeVisitor&)> InProcessCallback) override;
	virtual void PostProcessChanges() override;
	//~ End ICurvesDiffer Interface

private:

	/** The curve editor to which we append the change commands. */
	TSharedPtr<FCurveEditor> CurveEditor;

	/** Holds the changes made to FCurveModel's that returned no UObject from FCurveModel::GetOwningObject. */
	FGenericCurveChangeData ChangesToCurvesWithoutUObject;

	/** How much memory was allocated by all undo changes. */
	SIZE_T AllocationSize = 0;

	void ProcessDiff(const FCurveModelID& InCurveId, FGenericCurveChangeData&& InData);
};
}
