// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericCurveChangeCommand.h"

#include "Algo/AllOf.h"
#include "CurveEditor.h"
#include "HAL/IConsoleManager.h"
#include "Modification/Keys/GenericCurveChangeUtils.h"
#include "SCurveEditor.h"
#include "Modification/Resolution/ResolveCurveModelPasskey.h"

namespace UE::CurveEditor
{
namespace Private
{
static TAutoConsoleVariable<bool> CVarValidateCurveExistenceOnUndoRedo(
	TEXT("CurveEditor.ValidateCurveExistenceOnUndoRedo"),
	true,
	TEXT("Validates that the curves referenced by undo / redo commands exist and logs a failing ensure if not. This helps find bugs with undo / redo.")
	);

template<typename TPredicate>
concept CDoesCurveExistCallable = std::is_invocable_r_v<bool, TPredicate, const FCurveModelID&>;

/** @return Whether all curves in InDataMap exists according to InPredicate. */
template<typename TValue, CDoesCurveExistCallable Predicate>
static bool AllCurvesExist(const TMap<FCurveModelID, TValue>& InDataMap, Predicate&& InPredicate)
{
	return Algo::AllOf(InDataMap, [&InPredicate](const TPair<FCurveModelID, TValue>& Pair)
	{
		return InPredicate(Pair.Key);
	});
}

/** Validates that all FCurveModelIDs referenced by FGenericCurveChangeData have a matching FCurveModel. */
template<CDoesCurveExistCallable Predicate>
static void ValidateCurveExistence(const FGenericCurveChangeData& InDeltaChange, Predicate&& InPredicate)
{
	if (!CVarValidateCurveExistenceOnUndoRedo.GetValueOnAnyThread())
	{
		return;
	}

	const bool bAllExist = AllCurvesExist(InDeltaChange.MoveKeysData.ChangedCurves, InPredicate)
		&& AllCurvesExist(InDeltaChange.AddKeysData.SavedCurveState, InPredicate)
		&& AllCurvesExist(InDeltaChange.RemoveKeysData.SavedCurveState, InPredicate)
		&& AllCurvesExist(InDeltaChange.KeyAttributeData.ChangedCurves, InPredicate)
		&& AllCurvesExist(InDeltaChange.CurveAttributeData.ChangeData, InPredicate);

	// Hitting this means that the transaction contains data for FCurveModel that no longer is in the FCurveEditor. See CL description.
	// See also FCurveEditorInitParams::ResolveCurveModelDelegate.
	ensureMsgf(bAllExist, TEXT("The curve change references some curves that do not exist in FCurveEditor. Investigate."));
	UE_CLOGF(!bAllExist, LogCurveEditor, Warning, "The curve change references some curves that do not exist in FCurveEditor. Investigate.");
}

/**
 * Goes through undo data and finds FCurveModels that are not currently in FCurveEditor::CurveData.
 * The missing FCurveModels are appended to OutInstantiatedCurves.
 */
template<typename TValue>
static void InstantiateMissingCurves(
	const FCurveEditor& InCurveEditor, const TMap<FCurveModelID, TValue>& InReferencingData,
	TMap<FCurveModelID, TUniquePtr<FCurveModel>>& OutInstantiatedCurves
	)
{
	for (const TPair<FCurveModelID, TValue>& DataPair : InReferencingData)
	{
		const FCurveModelID& CurveId = DataPair.Key;
		if (InCurveEditor.GetCurves().Contains(CurveId)) // We're only interested in missing curves
		{
			continue;
		}

		if (TUniquePtr<FCurveModel> InstantiatedCurveModel = ResolveCurveModel(InCurveEditor, CurveId))
		{
			OutInstantiatedCurves.Emplace(CurveId, MoveTemp(InstantiatedCurveModel));
		}
	}
}

/**
 * Invokes InProcessCurves once for already existing and missing curves, i.e.
 * - the curves currently in FCurveEditor::CurveData (existing), and
 * - a temporary TMap of FCurveModelIDs that were missing from FCurveEditor::CurveData but that were referenced by the undo data (missing)
 */
template<typename TProcessCurves> requires std::is_invocable_v<TProcessCurves, TMap<FCurveModelID, TUniquePtr<FCurveModel>>>
void ProcessExistingAndMissingCurves(const FCurveEditor& InCurveEditor, const FGenericCurveChangeData& InDeltaChange, TProcessCurves&& InProcessCurves)
{
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& RegisteredCurves = InCurveEditor.GetCurves();
	TMap<FCurveModelID, TUniquePtr<FCurveModel>> TemporaryCurves;
	{
		InstantiateMissingCurves(InCurveEditor, InDeltaChange.MoveKeysData.ChangedCurves, TemporaryCurves);
		InstantiateMissingCurves(InCurveEditor, InDeltaChange.AddKeysData.SavedCurveState, TemporaryCurves);
		InstantiateMissingCurves(InCurveEditor, InDeltaChange.RemoveKeysData.SavedCurveState, TemporaryCurves);
		InstantiateMissingCurves(InCurveEditor, InDeltaChange.KeyAttributeData.ChangedCurves, TemporaryCurves);
		InstantiateMissingCurves(InCurveEditor, InDeltaChange.CurveAttributeData.ChangeData, TemporaryCurves);
	}

	// We'll log warnings if the undo data cannot be applied due to a missing FCurveModel. We do this so we can flag potential bugs to UE developers.
	Private::ValidateCurveExistence(InDeltaChange, [&RegisteredCurves, &TemporaryCurves](const FCurveModelID& InCurveId)
	{
		return RegisteredCurves.Contains(InCurveId) || TemporaryCurves.Contains(InCurveId);
	});
	
	// The keys of TMap<FCurveModelIDs> are completely distinct (we constructed them that way), so we can call ApplyChange / RevertChange twice here.
	// So there is no risk of double applying / reverting any curve changes as the keys are distinct.
	InProcessCurves(RegisteredCurves);
	InProcessCurves(TemporaryCurves);
}
}
	
void FGenericCurveChangeCommand::Apply(UObject* Object)
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor())
	{
		Private::ProcessExistingAndMissingCurves(*CurveEditor, DeltaChange, [this](const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves)
		{
			GenericCurveChange::ApplyChange(InCurves, DeltaChange);
		});
	}
}

void FGenericCurveChangeCommand::Revert(UObject* Object)
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor())
	{
		Private::ProcessExistingAndMissingCurves(*CurveEditor, DeltaChange, [this](const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves)
		{
			GenericCurveChange::RevertChange(InCurves, DeltaChange);
		});
	}
}

SIZE_T FGenericCurveChangeCommand::GetSize() const
{
	return DeltaChange.GetAllocatedSize() + sizeof(FGenericCurveChangeCommand);
}

FString FGenericCurveChangeCommand::ToString() const
{
	return TEXT("FGenericCurveChangeCommand");
}
}
