// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modification/CurveEditorCommandChange.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"

namespace UE::CurveEditor
{
/**
 * Captures any changes to keys including and limited to:
 * - Move keys,
 * - Add new keys,
 * - Remove keys,
 * - Change key attributes,
 * - Change curve attributes
 */
class FGenericCurveChangeCommand : public FCurveEditorCommandChange
{
public:

	explicit FGenericCurveChangeCommand(TSharedPtr<FCurveEditor> InCurveEditor, FGenericCurveChangeData InDeltaChange)
		: FCurveEditorCommandChange(MoveTemp(InCurveEditor))
		// Make sure to Shrink() whatever is passed to us so it takes up as little memory as possible.
		, DeltaChange([InDeltaChange = MoveTemp(InDeltaChange)]() mutable { InDeltaChange.Shrink(); return InDeltaChange; }())
	{}

	//~ Begin FChange Interface
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual SIZE_T GetSize() const override;
	virtual FString ToString() const override;
	//~ End FChange Interface

private:

	/** The change made to the selection. */
	const FGenericCurveChangeData DeltaChange;
};
}
