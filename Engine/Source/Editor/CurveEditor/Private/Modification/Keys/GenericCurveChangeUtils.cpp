// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Keys/GenericCurveChangeUtils.h"

#include "CurveAttributeChangeUtils.h"
#include "CurveEditor.h"
#include "CurveEditorTrace.h"
#include "KeyAttributeChangeUtils.h"
#include "KeyExistenceUtils.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/MoveKeysChangeUtils.h"

namespace UE::CurveEditor::GenericCurveChange
{
void ApplyChange(
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FGenericCurveChangeData& InDeltaChange, const double InCurrentSliderPos
	)
{
	SCOPED_CURVE_EDITOR_TRACE(ApplyGenericChange);

	// Order matters. Must move keys before adding them in KeyInsertion::ApplyChange.
	// If we added keys first and a key we're about to move is already at the position, we'd end up not creating any key.
	// This is because we're not allowed to have keys stacked on the same x-position.
	// Example: First, move key 909 from x=0 to x=1. Then, add key 891 from to position x=0.
	
	// Note: What about e.g. moving key 230 from x=0 to x=1 and adding key 450 to position x=1? We needn't worry about this because it's an invalid
	// operation to begin with. All FGenericCurveChangeData are constructed from changes that were performed.
	// This hypothetical operation shouldn't have occured in the first place.
	MoveKeys::ApplyChange(InCurves, InDeltaChange.MoveKeysData);
	
	KeyRemoval::ApplyChange(InCurves, InDeltaChange.RemoveKeysData, InCurrentSliderPos);
	KeyInsertion::ApplyChange(InCurves, InDeltaChange.AddKeysData);
	KeyAttributes::ApplyChange(InCurves, InDeltaChange.KeyAttributeData);
	CurveAttributes::ApplyChange(InCurves, InDeltaChange.CurveAttributeData);
}

void RevertChange(
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FGenericCurveChangeData& InDeltaChange, const double InCurrentSliderPos
	)
{
	SCOPED_CURVE_EDITOR_TRACE(RevertGenericChange);
	
	// Order matters. Must move keys before adding them back in KeyRemoval::ApplyChange.
	// If we moved a key on top of another key's time, that replaces the key.
	// If we tried adding a key back before moving the key that currently happens to be there (but is moved by our up), we end up with a single key
	// instead of with two.
	// This is because we're not allowed to have keys stacked on the same x-position.
	// Example: First, move key 909 from x=1 to x=0. Then, remove key 891 from position x=0.
	
	// Note: What about e.g. moving key 230 from x=0 to x=1 and adding key 450 to position x=1? We needn't worry about this because it's an invalid
	// operation to begin with. All FGenericCurveChangeData are constructed from changes that were performed.
	// This hypothetical operation shouldn't have occured in the first place.
	MoveKeys::RevertChange(InCurves, InDeltaChange.MoveKeysData);
	
	KeyInsertion::RevertChange(InCurves, InDeltaChange.AddKeysData, InCurrentSliderPos);
	KeyRemoval::RevertChange(InCurves, InDeltaChange.RemoveKeysData);
	KeyAttributes::RevertChange(InCurves, InDeltaChange.KeyAttributeData);
	CurveAttributes::RevertChange(InCurves, InDeltaChange.CurveAttributeData);
}

void ApplyChange(const FCurveEditor& InCurveEditor, const FGenericCurveChangeData& InDeltaChange)
{
	const TSharedPtr<ITimeSliderController> TimeController = InCurveEditor.GetTimeSliderController();
	const double CurrentTime = TimeController
		? TimeController->GetTickResolution().AsSeconds(TimeController->GetScrubPosition())
		: 0.0;
	ApplyChange(InCurveEditor.GetCurves(), InDeltaChange, CurrentTime);
}
	
void RevertChange(const FCurveEditor& InCurveEditor, const FGenericCurveChangeData& InDeltaChange)
{
	const TSharedPtr<ITimeSliderController> TimeController = InCurveEditor.GetTimeSliderController();
	const double CurrentTime = TimeController
		? TimeController->GetTickResolution().AsSeconds(TimeController->GetScrubPosition())
		: 0.0;
	RevertChange(InCurveEditor.GetCurves(), InDeltaChange, CurrentTime);
}
}
