// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyAttributeChangeUtils.h"

#include "CurveEditorTrace.h"
#include "CurveModel.h"
#include "Modification/Keys/Data/KeyAttributeChangeData.h"

#include <type_traits>

namespace UE::CurveEditor::KeyAttributes
{
FKeyAttributeChangeData_PerCurve Diff(
	const TConstArrayView<FKeyHandle> InKeysBeforeChange,
	const TConstArrayView<FKeyAttributes> InOriginal, const TConstArrayView<FKeyAttributes> InTarget
)
{
	SCOPED_CURVE_EDITOR_TRACE(Diff_KeyAttributes);
	
	FKeyAttributeChangeData_PerCurve ChangeData;
	const int32 Num = InKeysBeforeChange.Num();
	if (!ensure(InOriginal.Num() == Num && InTarget.Num() == Num))
	{
		return ChangeData;
	}
	
	ChangeData.Handles.Reserve(Num);
	ChangeData.BeforeChange.Reserve(Num);
	ChangeData.AfterChange.Reserve(Num);
	
	for (int32 Index = 0; Index < Num; ++Index)
	{
		if (InOriginal[Index] != InTarget[Index])
		{
			ChangeData.Handles.Add(InKeysBeforeChange[Index]);
			ChangeData.BeforeChange.Add(InOriginal[Index]);
			ChangeData.AfterChange.Add(InTarget[Index]);
		}
	}
	
	return ChangeData;
}

namespace Private
{
template<typename TGetChange> requires std::is_invocable_r_v<TConstArrayView<FKeyAttributes>, TGetChange, const FKeyAttributeChangeData_PerCurve&>
static void SetKeyAttributes(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FKeyAttributeChangeData& InDeltaChange, TGetChange&& InGetChange)
{
	for (const TPair<FCurveModelID, FKeyAttributeChangeData_PerCurve>& Pair : InDeltaChange.ChangedCurves)
	{
		const TUniquePtr<FCurveModel>* FoundCurveModel = InCurves.Find(Pair.Key);
		if (!FoundCurveModel || !FoundCurveModel->Get())
		{
			continue;
		}
		FCurveModel& CurveModel = *FoundCurveModel->Get();
		const FKeyAttributeChangeData_PerCurve& PerCurveData = Pair.Value;
		CurveModel.SetKeyAttributes(PerCurveData.Handles, InGetChange(PerCurveData));
	}
}
}

void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FKeyAttributeChangeData& InDeltaChange)
{
	Private::SetKeyAttributes(InCurves, InDeltaChange, [](const FKeyAttributeChangeData_PerCurve& Change)
	{
		return Change.AfterChange;
	});}
	
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FKeyAttributeChangeData& InDeltaChange)
{
	Private::SetKeyAttributes(InCurves, InDeltaChange, [](const FKeyAttributeChangeData_PerCurve& Change)
	{
		return Change.BeforeChange;
	});
}
}
