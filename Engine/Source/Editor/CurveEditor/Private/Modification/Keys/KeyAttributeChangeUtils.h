// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
class IBufferedCurveModel;
namespace UE::CurveEditor { struct FKeyAttributeChangeData; }
namespace UE::CurveEditor { struct FKeyAttributeChangeData_PerCurve; }
struct FCurveModelID;
struct FKeyAttributes;
struct FKeyHandle;
struct FKeyPosition;
template<typename Allocator> class TBitArray;

namespace UE::CurveEditor::KeyAttributes
{
/**
 * Computes the delta change to get from InOriginal to InTarget.
 * 
 * @param InKeysBeforeChange The keys for which to check whether they have changed.
 * @param InOriginal The original positions of the keys.
 * @param InTarget The new positions of the keys.
 */
FKeyAttributeChangeData_PerCurve Diff(
	const TConstArrayView<FKeyHandle> InKeysBeforeChange,
	const TConstArrayView<FKeyAttributes> InOriginal,
	const TConstArrayView<FKeyAttributes> InTarget
	);

/** Applies InDeltaChange to InCurves (redo). */
void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FKeyAttributeChangeData& InDeltaChange);
/** Reverts InDeltaChange from InCurves (undo). */
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FKeyAttributeChangeData& InDeltaChange);
}
