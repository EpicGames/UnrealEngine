// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyExistenceUtils.h"

#include "CurveEditorTrace.h"
#include "CurveModel.h"
#include "Algo/Contains.h"
#include "Modification/Keys/Data/AddKeysChangeData.h"
#include "Modification/Keys/Data/RemoveKeysChangeData.h"

namespace UE::CurveEditor
{
void AddKeys(FCurveModel& InCurveModel, const FCurveKeyData& InKeyChange)
{
	InCurveModel.PutKeys(InKeyChange.KeyHandles, InKeyChange.KeyPositions, InKeyChange.KeyAttributes);
}
	
void RemoveKeys(FCurveModel& InCurveModel, const FCurveKeyData& InKeyChange, const double InCurrentSliderTime)
{
	InCurveModel.RemoveKeys(InKeyChange.KeyHandles, InCurrentSliderTime);
}

namespace KeyExistenceDetail
{
static const FCurveKeyData* FindCurveChange(const FCurveModelID& InCurveId, const FAddKeysChangeData& InChange)
{
	return InChange.SavedCurveState.Find(InCurveId);
}
static const FCurveKeyData* FindCurveChange(const FCurveModelID& InCurveId, const FRemoveKeysChangeData& InChange)
{
	return InChange.SavedCurveState.Find(InCurveId);
}

/** Shorthand util for applying lambda to all curve models that have FKeyDataSnapshot_PerCurve saved from them. */
template<typename TChangeData, typename TApply> requires std::is_invocable_v<TApply, FCurveModel&, const FCurveKeyData&>
static void ApplyKeyData(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const TChangeData& InChangeData, TApply&& InApplyFunctor)
{
	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : InCurves)
	{
		const TUniquePtr<FCurveModel>& CurveModel = CurvePair.Value;
		const FCurveKeyData* PerCurveChange = FindCurveChange(CurvePair.Key, InChangeData);
		if (ensure(CurveModel) && PerCurveChange)
		{
			InApplyFunctor(*CurveModel, *PerCurveChange);
		}
	}
}

template<typename TCallback> requires std::is_invocable_v<TCallback, int32>
static void ForEachRemovedKeyIndex(TConstArrayView<FKeyHandle> InOriginalKeys, const FCurveModel& InCurveModel, TCallback&& InCallback)
{
	SCOPED_CURVE_EDITOR_TRACE(Diff_RemovedKeys);
	
	TArray<FKeyHandle> NewAllKeys;
	{
		SCOPED_CURVE_EDITOR_TRACE(Diff_RemovedKeys_GetAndSortKeys);
		InCurveModel.GetAllKeys(NewAllKeys);
		NewAllKeys.Sort(); // Sorting ...
	}
	
	SCOPED_CURVE_EDITOR_TRACE(Diff_RemovedKeys_DoDiff);
	TArray<FKeyHandle> Result;
	for (int32 Index = 0; Index < InOriginalKeys.Num(); ++Index)
	{
		const FKeyHandle KeyHandle = InOriginalKeys[Index];

		// ... and performing binary search speeds this up (profiled).
		// In a test with 110.000 keys, we were able to get the time down from 8.1 ms to 6.5 ms.
		const int32 FoundIndex = Algo::BinarySearch(NewAllKeys, KeyHandle);
		const bool bIsStillContained = FoundIndex != INDEX_NONE;
		if (!bIsStillContained)
		{
			InCallback(Index);
		}
	}
}
}

namespace KeyInsertion
{
FCurveKeyData Diff(TConstArrayView<FKeyHandle> InOriginalKeys, const FCurveModel& InCurveModel)
{
	SCOPED_CURVE_EDITOR_TRACE(Diff_AddedKeys);
	
	const TArray<FKeyHandle> NewAllKeys = InCurveModel.GetAllKeys();
	
	FCurveKeyData Snapshot;
	TArray<FKeyHandle>& Keys = Snapshot.KeyHandles;
	for (int32 Index = 0; Index < NewAllKeys.Num(); ++Index)
	{
		const FKeyHandle KeyHandle = NewAllKeys[Index];
		
		// This could be sped up by using binary search maybe but not by much (for 110.000 keys, you can save 2ms).
		// Let's not require the caller to pass it in sorted.
		if (!InOriginalKeys.Contains(KeyHandle))
		{
			Keys.Add(KeyHandle);
		}
	}
	
	if (!Keys.IsEmpty())
	{
		Snapshot.KeyPositions.SetNum(Keys.Num());
		Snapshot.KeyAttributes.SetNum(Keys.Num());
		
		InCurveModel.GetKeyPositions(Keys, Snapshot.KeyPositions);
		InCurveModel.GetKeyAttributesExcludingAutoComputed(Keys, Snapshot.KeyAttributes);
	}
	return Snapshot;
}

void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FAddKeysChangeData& InDeltaChange)
{
	KeyExistenceDetail::ApplyKeyData(InCurves, InDeltaChange, [](FCurveModel& Curve, const FCurveKeyData& PerCurveData)
	{
		AddKeys(Curve, PerCurveData);
	});
}
void RevertChange(
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FAddKeysChangeData& InDeltaChange, const double InCurrentSliderTime
	)
{
	KeyExistenceDetail::ApplyKeyData(InCurves, InDeltaChange, [InCurrentSliderTime](FCurveModel& Curve, const FCurveKeyData& PerCurveData)
	{
		RemoveKeys(Curve, PerCurveData, InCurrentSliderTime);
	});
}
}

namespace KeyRemoval
{
FCurveKeyData Diff(
	TConstArrayView<FKeyHandle> InOriginalKeys, TConstArrayView<FKeyPosition> InOriginalPositions, TConstArrayView<FKeyAttributes> InOriginalAttrs,
	const FCurveModel& InCurveModel
	)
{
	FCurveKeyData Snapshot;
	KeyExistenceDetail::ForEachRemovedKeyIndex(InOriginalKeys, InCurveModel,
		[&InOriginalKeys, &InOriginalPositions, &InOriginalAttrs, &Snapshot](int32 Index)
		{
			Snapshot.KeyHandles.Add(InOriginalKeys[Index]);
			Snapshot.KeyPositions.Add(InOriginalPositions[Index]);
			Snapshot.KeyAttributes.Add(InOriginalAttrs[Index]);
		});
	return Snapshot;
}

TArray<FKeyHandle> Diff(TConstArrayView<FKeyHandle> InOriginalKeys, const FCurveModel& InCurveModel)
{
	TArray<FKeyHandle> Result;
	KeyExistenceDetail::ForEachRemovedKeyIndex(InOriginalKeys, InCurveModel, [&InOriginalKeys, &Result](int32 Index)
	{
		Result.Add(InOriginalKeys[Index]);
	});
	return Result;
}

void ApplyChange(
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FRemoveKeysChangeData& InDeltaChange, const double InCurrentSliderTime
	)
{
	KeyExistenceDetail::ApplyKeyData(InCurves, InDeltaChange, [InCurrentSliderTime](FCurveModel& Curve, const FCurveKeyData& PerCurveData)
	{
		RemoveKeys(Curve, PerCurveData, InCurrentSliderTime);
	});
}
	
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FRemoveKeysChangeData& InDeltaChange)
{
	KeyExistenceDetail::ApplyKeyData(InCurves, InDeltaChange, [](FCurveModel& Curve, const FCurveKeyData& PerCurveData)
	{
		AddKeys(Curve, PerCurveData);
	});
}
}
}
