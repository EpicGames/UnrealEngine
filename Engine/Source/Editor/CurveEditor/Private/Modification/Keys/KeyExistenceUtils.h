// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
namespace UE::CurveEditor { struct FCurveKeyData; }
namespace UE::CurveEditor { struct FRemoveKeysChangeData; }
namespace UE::CurveEditor{ struct FAddKeysChangeData;}
struct FCurveModelID;
struct FKeyAttributes;
struct FKeyHandle;
struct FKeyPosition;

namespace UE::CurveEditor
{
/** Adds the keys saved in InKeyChange to InCurveModel. */
void AddKeys(FCurveModel& InCurveModel, const FCurveKeyData& InKeyChange);
/** Removes the keys saved in InKeyChange from InCurveModel. */
void RemoveKeys(FCurveModel& InCurveModel, const FCurveKeyData& InKeyChange, const double InCurrentSliderTime = 0);

namespace KeyInsertion
{
/**
 * Returns the keys that are new - comparing InOriginalKeys to the current state of InCurveModel.
 * @return The keys that have been added to InCurveModel. */
FCurveKeyData Diff(TConstArrayView<FKeyHandle> InOriginalKeys, const FCurveModel& InCurveModel);

/** Adds the keys in InDeltaChange to the curves. */
void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FAddKeysChangeData& InDeltaChange);
/** Removes the keys in InDeltaChange from the curves. */
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FAddKeysChangeData& InDeltaChange,
	const double InCurrentSliderTime = 0
	);
}

namespace KeyRemoval
{
/**
 * Returns the keys that have been removed - comparing InOriginalKeys to the current state of InCurveModel.
 * @return The keys that have been removed from InCurveModel
 */
FCurveKeyData Diff(
	TConstArrayView<FKeyHandle> InOriginalKeys, TConstArrayView<FKeyPosition> InOriginalPositions, TConstArrayView<FKeyAttributes> InOriginalAttrs,
	const FCurveModel& InCurveModel
	);

/** @return The keys that have been removed from InCurveModel. */
TArray<FKeyHandle> Diff(TConstArrayView<FKeyHandle> InOriginalKeys, const FCurveModel& InCurveModel);

/** Removes the keys in InDeltaChange from the curves. */
void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FRemoveKeysChangeData& InDeltaChange,
	const double InCurrentSliderTime = 0
	);
/** Adds the keys in InDeltaChange to the curves. */
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FRemoveKeysChangeData& InDeltaChange);
}
}
