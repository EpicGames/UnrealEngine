// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Keys/MoveKeysChangeUtils.h"

#include "CurveEditorTrace.h"
#include "CurveModel.h"
#include "Modification/Keys/Data/MoveKeysChangeData.h"

namespace UE::CurveEditor::MoveKeys
{
namespace Private
{
template<typename TTranslateOp> requires std::is_invocable_r_v<FKeyPosition, TTranslateOp, const FKeyPosition&, const FVector2f&>
static void CombineData(
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves,
	const FMoveKeysChangeData& InDeltaChange,
	TTranslateOp&& InTranslateFunc
	)
{
	for (const TPair<FCurveModelID, FMoveKeysChangeData_PerCurve>& Pair : InDeltaChange.ChangedCurves)
	{
		const TUniquePtr<FCurveModel>* FoundCurveModel = InCurves.Find(Pair.Key);
		if (!FoundCurveModel || !FoundCurveModel->Get())
		{
			continue;
		}
		FCurveModel& CurveModel = *FoundCurveModel->Get();
	
		const FMoveKeysChangeData_PerCurve& Data = Pair.Value;
		const int32 Num = Data.Handles.Num();
		check(Data.Translations.Num() == Num);
	
		TArray<FKeyPosition> CurrentPositions;
		CurrentPositions.SetNumUninitialized(Num);
		CurveModel.GetKeyPositions(Data.Handles, CurrentPositions);
	
		for (int32 Index = 0; Index < Data.Handles.Num(); ++Index)
		{
			CurrentPositions[Index] = InTranslateFunc(CurrentPositions[Index], Data.Translations[Index]);
		}

		CurveModel.SetKeyPositions(Data.Handles, CurrentPositions, EPropertyChangeType::ValueSet);
	}
}
}
FMoveKeysChangeData_PerCurve Diff(
	const TConstArrayView<FKeyHandle> InKeys,
	const TConstArrayView<FKeyPosition> InOriginal, const TConstArrayView<FKeyPosition> InTarget
	)
{
	SCOPED_CURVE_EDITOR_TRACE(Diff_MovedKeys);
	
	const int32 Num = InKeys.Num();
	if (!ensure(InOriginal.Num() == Num && InTarget.Num() == Num))
	{
		return {};
	}

	FMoveKeysChangeData_PerCurve Result;
	Result.Handles.Reserve(Num);
	Result.Translations.Reserve(Num);
	for (int32 Index = 0; Index < Num; ++Index)
	{
		const FKeyPosition& Original = InOriginal[Index];
		const FKeyPosition& Target = InTarget[Index];
		const FVector2f Delta = FVector2f(Target.InputValue - Original.InputValue, Target.OutputValue - Original.OutputValue);

		if (!Delta.IsNearlyZero())
		{
			Result.Handles.Add(InKeys[Index]);
			Result.Translations.Add(Delta);
		}
	}
	
	return Result;
}

void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FMoveKeysChangeData& InDeltaChange)
{
	Private::CombineData(InCurves, InDeltaChange, [](const FKeyPosition& Position, const FVector2f Delta)
	{
		return FKeyPosition(Position.InputValue + Delta.X, Position.OutputValue + Delta.Y);
	});
}

void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FMoveKeysChangeData& InDeltaChange)
{
	Private::CombineData(InCurves, InDeltaChange, [](const FKeyPosition& Position, const FVector2f Delta)
	{
		return FKeyPosition(Position.InputValue - Delta.X, Position.OutputValue - Delta.Y);
	});
}
}
