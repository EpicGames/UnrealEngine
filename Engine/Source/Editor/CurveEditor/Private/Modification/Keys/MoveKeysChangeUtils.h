// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
class IBufferedCurveModel;
namespace UE::CurveEditor { struct FMoveKeysChangeData; }
namespace UE::CurveEditor { struct FMoveKeysChangeData_PerCurve; }
struct FCurveModelID;
struct FKeyHandle;
struct FKeyPosition;
template<typename Allocator> class TBitArray;

namespace UE::CurveEditor::MoveKeys
{
/**
 * Computes the delta change to get from InOriginal to InTarget.
 * 
 * @param InKeysBeforeChange The keys for which to check whether they have moved.
 * @param InOriginal The original positions of the keys.
 * @param InTarget The new positions of the keys.
 */
FMoveKeysChangeData_PerCurve Diff(
	const TConstArrayView<FKeyHandle> InKeysBeforeChange,
	const TConstArrayView<FKeyPosition> InOriginal,
	const TConstArrayView<FKeyPosition> InTarget
	);

/** Applies InDeltaChange to InCurves (redo). */
void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FMoveKeysChangeData& InDeltaChange);

/** Reverts InDeltaChange from InCurves (undo). */
void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FMoveKeysChangeData& InDeltaChange);
}
