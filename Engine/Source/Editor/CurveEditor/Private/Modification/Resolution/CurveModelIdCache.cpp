// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveModelIdCache.h"

#include "CurveEditorTrace.h"
#include "CurveModel.h"

namespace UE::CurveEditor
{
namespace IdCacheDetail
{
static TOptional<FCurveModelID> FindByOwner(const FCurveModelIdCache& InCache, const FCurveMetaDataIdentifiers& InMetaData)
{
	UObject* Owner = InMetaData.Owner.Get();
	const TArray<FCurveModelID>* CurveIndices = InCache.OwnerToCurvesCache.Find(Owner);
	if (!CurveIndices)
	{
		return {};
	}

	for (const FCurveModelID& CurveId : *CurveIndices)
	{
		const FCurveMetaDataIdentifiers* MetaData = FindMetaData(InCache, CurveId);
		if (MetaData && InMetaData == *MetaData)
		{
			return CurveId;
		}
	}

	return {};
}

static TOptional<FCurveModelID> FindByLinearSearch(const FCurveModelIdCache& InCache, const FCurveMetaDataIdentifiers& InMetaData)
{
	for (int32 Index = 0; Index < InCache.CurveMetaData.Num(); ++Index)
	{
		const FCurveModelIdCache::FCurveData& Data = InCache.CurveMetaData[Index];
		if (Data.MetaData == InMetaData)
		{
			return Data.CurveId;
		}
	}
	
	return {};
}

static TOptional<FCurveModelID> FindCurveId(const FCurveModelIdCache& InCache, const FCurveMetaDataIdentifiers& InMetaData)
{
	if (const TOptional<FCurveModelID> OwnerSeachResult = FindByOwner(InCache, InMetaData))
	{
		return OwnerSeachResult;
	}

	if (const TOptional<FCurveModelID> SearchAllResult = FindByLinearSearch(InCache, InMetaData))
	{
		return SearchAllResult;
	}

	return {};
}

/** Inserts the curve data into presorted InCache maintaining the sort order (by FCurveModelID). */
static int32 AddCurveModelId(FCurveModelIdCache& InCache, FCurveModelIdCache::FCurveData InData)
{
	const int32 InsertIndex = Algo::LowerBound(InCache.CurveMetaData, InData,
		[](const FCurveModelIdCache::FCurveData& Left, const FCurveModelIdCache::FCurveData& Right)
		{
			return Left.CurveId < Right.CurveId;
		});
	InCache.CurveMetaData.Insert(MoveTemp(InData), InsertIndex);
	return InsertIndex;
}
}
	
FCurveModelID InitCurveModelIdWithReusedOrNewId(
	FCurveModelIdCache& InCache, FCurveModel& InCurveModel, const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InExistingCurves
	)
{
	SCOPED_CURVE_EDITOR_TRACE(GetOrInitCurveModelId);
	if (InCurveModel.GetId().IsSet())
	{
		return *InCurveModel.GetId();
	}

	FCurveMetaDataIdentifiers MetaData(InCurveModel);
	const TOptional<FCurveModelID> ReusedId = IdCacheDetail::FindCurveId(InCache, MetaData);
	if (ReusedId
		// If this fails, adjust your curves to have unique identifier permutations (intention name, etc.), or call FCurveModel::InitCurveId before
		// FCurveEditor::AddCurve. Undo / redo operations may not work properly until you fix it. See CL description.
		&& ensure(!InExistingCurves.Contains(*ReusedId)))
	{
		InCurveModel.InitCurveId(*ReusedId);
		return *ReusedId;
	}
	
	UObject* Owner = MetaData.Owner.Get();
	const FCurveModelID NewId = InCurveModel.GetOrInitId();
	IdCacheDetail::AddCurveModelId(InCache, { NewId, MoveTemp(MetaData)});
	if (Owner)
	{
		TArray<FCurveModelID>& Indices = InCache.OwnerToCurvesCache.FindOrAdd(Owner);
		Indices.Add(NewId);
	}
	return NewId;
}

const FCurveMetaDataIdentifiers* FindMetaData(const FCurveModelIdCache& InCache, const FCurveModelID& InCurveId)
{
	const TArray<FCurveModelIdCache::FCurveData>& Data = InCache.CurveMetaData;
	const int32 Index = Algo::BinarySearchBy(Data, InCurveId,
		[](const FCurveModelIdCache::FCurveData& InData){ return InData.CurveId; }
		);
	return Data.IsValidIndex(Index) ? &Data[Index].MetaData : nullptr;
}
}