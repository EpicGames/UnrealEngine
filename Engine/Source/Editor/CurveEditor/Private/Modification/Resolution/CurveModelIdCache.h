// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "Modification/Resolution/CurveMetaDataIdentifiers.h"
#include "UObject/ObjectKey.h"

class FCurveModel;

namespace UE::CurveEditor
{
/**
 * Keeps track of curve metadata so FCurveModelID are reused when conceptually the same FCurveModels are re-added to FCurveEditor.
 * This is relevant for command-based undo / redo to work correctly.
 *
 * ====== Background ======
 * Consider the following scenario
 * 1. User clicks a curve to view it. AddCurve is called: a curve representing X may be added with ID=42,
 * 2. User does an undo-able action. FGenericCurveChangeCommand saves change data mapped to ID=42.
 * 3. User clicks another curve. RemoveCurve is called: the curve with ID=42 is removed.
 * 4. User clicks the X curve again to view it again. AddCurve is called again: the curve representing X is adde back to FCurveEditor.
 * In order for FGenericCurveChangeCommand to continue working, in step 4, the curve must be added back using the same FCurveModelID.
 * 
 * Systems can be explicit by calling FCurveModel::InitCurveId before calling FCurveEditor::AddCurve.
 * However, there are 2 issues with this approach:
 * 1. that is cumbersome to work with,
 * 2. command-based undo was added later so many older editor systems currently do not bother to re-use FCurveModelIDs.
 *
 * ===== Class purpose ======
 * So CurveEditor provides a "service" to such systems: provided that the FCurveModel they re-add has the same metadata (owner object, short name, etc.),
 * CurveEditor will automatically attempt to re-use the ID that the re-added curve had when it was last added. That covers a lot of use cases.
 * Systems for which this logic is not enough should just call FCurveModel::InitCurveId before calling FCurveEditor::AddCurve.
 */
struct FCurveModelIdCache
{
	/**
	 * Maps UObjects to the FCurveModels they own or used to own.
	 * Since most FCurveModel have an owning UObject, we can speed up search using this map.
	 */
	TMap<TObjectKey<UObject>, TArray<FCurveModelID>> OwnerToCurvesCache;
	
	struct FCurveData
	{
		FCurveModelID CurveId;
		FCurveMetaDataIdentifiers MetaData;
	};
	/** Holds all curve data ever added to FCurveEditor. Never removed from. Sorted by FCurveModelID to allow for binary search. */
	TArray<FCurveData> CurveMetaData;
};

/**
 * Called by FCurveEditor::AddCurve to init the FCurveModelID for the new curve.
 * Tries to reuse an old FCurveModelID if matches the metadata of an old curve.
 * Priority order:
 * - Use InCurveModel's FCurveModel::CurveId, if set. 
 * - Search whether InCurveModel's metadata matches that of CurveMetaData in FCurveEditor in the past, then use the cached FCurveModelID.
 * - Generate a new ID and cache.
 *
 * @param InCache Cache holding the metadata
 * @param InCurveModel The curve about to be added
 * @param InExistingCurves Curves that already exist. Used to handle clashing metadata (i.e. when InCurveModel's metadata already is in use)
 */
FCurveModelID InitCurveModelIdWithReusedOrNewId(
	FCurveModelIdCache& InCache, FCurveModel& InCurveModel,
	const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InExistingCurves
	);

/**
 * Performs binary search to metadata associated with InCurveId.
 * @return Metadata associated with InCurveId
 */
const FCurveMetaDataIdentifiers* FindMetaData(const FCurveModelIdCache& InCache, const FCurveModelID& InCurveId);
}


