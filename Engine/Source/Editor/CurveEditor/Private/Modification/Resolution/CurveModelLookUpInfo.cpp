// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Resolution/CurveModelLookUpInfo.h"

#include "CurveEditor.h"
#include "CurveModelIdCache.h"

namespace UE::CurveEditor
{
const FCurveMetaDataIdentifiers* FCurveModelLookUpArgs::GetCurveMetaData() const
{
	FCurveModelIdCache* Cache = CurveEditor.GetCurveMetaDataCache();
	checkSlow(Cache); // Did not call FCurveEditor::InitCurveEditor?
	return Cache ? FindMetaData(*Cache, CurveID) : nullptr;
}
}
