// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"

namespace UE::CurveEditor
{
/** Tries to re-create a FCurveModel that was in this FCurveEditor previously but has since been removed. */
TUniquePtr<FCurveModel> ResolveCurveModel(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId);
	
/** Implements passkey pattern to restrict access to FCurveEditor::ResolveCurveModel. */
class FResolveCurveModelPasskey
{
	// Effectively make FCurveEditor::ResolveCurve available everywhere and only in CurveEditor.
	// The whole point of this contraption is to avoid exposing FCurveEditor::ResolveCurve as part of the public API interface.
	friend TUniquePtr<FCurveModel> ResolveCurveModel(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId);
	
	static TUniquePtr<FCurveModel> ResolveCurveModel(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId)
	{
		return InCurveEditor.ResolveCurve(InCurveId);
	}
};

inline TUniquePtr<FCurveModel> ResolveCurveModel(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId)
{
	return FResolveCurveModelPasskey::ResolveCurveModel(InCurveEditor, InCurveId);
}
}
