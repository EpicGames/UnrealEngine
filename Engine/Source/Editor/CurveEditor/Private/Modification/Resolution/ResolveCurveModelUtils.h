// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

class FCurveModel;
class FCurveEditor;
struct FCurveModelID;

namespace UE::CurveEditor
{

	
	
}
