// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionChangeCommand.h"

#include "CurveEditor.h"
#include "Modification/Selection/SelectionChangeUtils.h"
#include "Selection/ScopedSelectionChangeEventSuppression.h"

namespace UE::CurveEditor
{
void FSelectionChangeCommand::Apply(UObject* Object)
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor())
	{
		const FScopedSelectionChangeEventSuppression SelectionChangedSuppressor(CurveEditor.ToSharedRef());
		ApplySelectionChange(CurveEditor->Selection, DeltaChange);
	}
}

void FSelectionChangeCommand::Revert(UObject* Object)
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor())
	{
		const FScopedSelectionChangeEventSuppression SelectionChangedSuppressor(CurveEditor.ToSharedRef());
		RevertSelectionChange(CurveEditor->Selection, DeltaChange);
	}
}

SIZE_T FSelectionChangeCommand::GetSize() const
{
	return sizeof(FSelectionChangeCommand) + DeltaChange.GetAllocatedSize();
}

FString FSelectionChangeCommand::ToString() const
{
	return TEXT("FSelectionChangeCommand");
}
}
