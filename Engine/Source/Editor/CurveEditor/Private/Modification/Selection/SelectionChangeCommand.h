// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionDeltaChange.h"
#include "Modification/CurveEditorCommandChange.h"

namespace UE::CurveEditor
{
/** Changes that changes the curve editor's selection. */
class FSelectionChangeCommand : public FCurveEditorCommandChange
{
public:

	explicit FSelectionChangeCommand(TSharedPtr<FCurveEditor> InCurveEditor, FSelectionDeltaChange InDeltaChange)
		: FCurveEditorCommandChange(MoveTemp(InCurveEditor))
		, DeltaChange([InDeltaChange = MoveTemp(InDeltaChange)]() mutable { InDeltaChange.Shrink(); return InDeltaChange; }())
	{}
	
	//~ Begin FChange Interface
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual SIZE_T GetSize() const override;
	virtual FString ToString() const override;
	//~ End FChange Interface

private:

	/** The change made to the selection. */
	const FSelectionDeltaChange DeltaChange;
};
}

