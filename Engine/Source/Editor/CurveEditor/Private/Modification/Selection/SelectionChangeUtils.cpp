// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionChangeUtils.h"

#include "CurveEditorSelection.h"
#include "CurveEditorSelectionPrivate.h"
#include "Modification/Selection/SelectionDeltaChange.h"

#include <type_traits>

namespace UE::CurveEditor
{
namespace SelectionDiffDetail
{
/** Goes through every key in InOriginalSelection and sees which keys are missing from InTarget. The missing keys are added to the array InGetKeys returns. */
template<typename TGetKeyArray>
requires std::is_invocable_r_v<TArray<TPair<FKeyHandle, ECurvePointType>>&, TGetKeyArray, FCurveSelectionDeltaChange&>
static void DetectMissingKeys(
	const TMap<FCurveModelID, FKeyHandleSet>& InOriginalSelection, const FCurveEditorSelection& InTarget,
	FSelectionDeltaChange& OutDeltaChange, TGetKeyArray&& InGetKeys
	)
{
	for (const TPair<FCurveModelID, FKeyHandleSet>& OriginalCurvePair : InOriginalSelection)
	{
		const FCurveModelID& CurveId = OriginalCurvePair.Key;
		const FKeyHandleSet& OriginalKeySet = OriginalCurvePair.Value; 
		const FKeyHandleSet* TargetKeySet = InTarget.FindForCurve(CurveId);

		// Should not really happen, but we'll handle the case anyway.
		if (OriginalKeySet.Num() == 0 && (!TargetKeySet || TargetKeySet->Num() == 0))
		{
			continue;
		}	

		// Fully missing from target?
		if (!TargetKeySet
			// We'd expect that FindForCurve would have returned nullptr if Num() == 0 but we'll handle the case anyway.
			|| TargetKeySet->Num() == 0)
		{
			FCurveSelectionDeltaChange& CurveChange = OutDeltaChange.ChangedCurves.Add(CurveId);
			TArray<TPair<FKeyHandle, ECurvePointType>>& Keys = InGetKeys(CurveChange);
			Keys.Reserve(OriginalKeySet.Num());
			for (const FKeyHandle& RemovedKey : OriginalKeySet.AsArray())
			{
				Keys.Emplace(RemovedKey, OriginalKeySet.PointType(RemovedKey));
			}
			continue;
		}

		// Find keys that were are missing or changed point type
		for (const FKeyHandle& OriginalKey : OriginalKeySet.AsArray())
		{
			// If the key handle is still in there but the point type has changed, we'll add it back again below.
			const ECurvePointType PointType = OriginalKeySet.PointType(OriginalKey);
			if (!TargetKeySet->Contains(OriginalKey, PointType))
			{
				TArray<TPair<FKeyHandle, ECurvePointType>>& Keys = InGetKeys(OutDeltaChange.ChangedCurves.FindOrAdd(CurveId));
				Keys.Emplace(OriginalKey, PointType);
			}
		}
	}
}
	
template<typename TCallback>
concept CGetKeys = std::is_invocable_r_v<const TArray<TPair<FKeyHandle, ECurvePointType>>&, TCallback, const FCurveSelectionDeltaChange&>;

void AddKeysInternal(FCurveEditorSelection& Selection, FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys)
{
	KeySelection::FAddInternal(Selection, CurveID, PointType, Keys);
}

void SetSerialNumberInternal(FCurveEditorSelection& Selection, uint32 NewSerialNumber)
{
	KeySelection::FSetSerialNumber(Selection, NewSerialNumber);
}
	
/** Removes keys from the array InGetRemovedKeys returns, and adds keys to the array InGetAddedKeys returns. */
template<CGetKeys TGetAddedKeys, CGetKeys TGetRemovedKeys>
static void ModifySelection(
	FCurveEditorSelection& InOutChange, const FSelectionDeltaChange& InDeltaChange, TGetAddedKeys&& InGetAddedKeys, TGetRemovedKeys&& InGetRemovedKeys
	)
{
	for (const TPair<FCurveModelID, FCurveSelectionDeltaChange>& ChangePair : InDeltaChange.ChangedCurves)
	{
		const FCurveModelID& CurveId = ChangePair.Key;
		const TArray<TPair<FKeyHandle, ECurvePointType>>& AddedKeys = InGetAddedKeys(ChangePair.Value);
		const TArray<TPair<FKeyHandle, ECurvePointType>>& RemovedKeys = InGetRemovedKeys(ChangePair.Value);
	
		for (const TPair<FKeyHandle, ECurvePointType>& RemovedPair : RemovedKeys)
		{
			InOutChange.Remove(CurveId, RemovedPair.Value, RemovedPair.Key);
		}
		for (const TPair<FKeyHandle, ECurvePointType>& AddedPair : AddedKeys)
		{
			const TConstArrayView<FKeyHandle> Keys(&AddedPair.Key, 1);
			AddKeysInternal(InOutChange, CurveId, AddedPair.Value, Keys);
		}
	}
}
}
	
FSelectionDeltaChange DiffSelection(const FCurveEditorSelection& InOriginal, const FCurveEditorSelection& InTarget)
{
	FSelectionDeltaChange DeltaChange;
	const TMap<FCurveModelID, FKeyHandleSet>& OriginalSelection = InOriginal.GetAll();
	const TMap<FCurveModelID, FKeyHandleSet>& TargetSelection = InTarget.GetAll();
	
	// Go through everything in InOriginal and see what's missing in InTarget -> effectively computes what was removed.
	SelectionDiffDetail::DetectMissingKeys(OriginalSelection, InTarget, DeltaChange,
		[](FCurveSelectionDeltaChange& CurveChanges) -> TArray<TPair<FKeyHandle, ECurvePointType>>&
		{ return CurveChanges.RemovedKeys; }
		);
	// Go through everything in InTarget and see what's missing in InOriginal -> effectively computes what was added.
	SelectionDiffDetail::DetectMissingKeys(TargetSelection, InOriginal, DeltaChange,
		[](FCurveSelectionDeltaChange& CurveChanges) -> TArray<TPair<FKeyHandle, ECurvePointType>>&
		{ return CurveChanges.AddedKeys; }
	);

	DeltaChange.OldSerialNumber = InOriginal.GetSerialNumber();
	DeltaChange.NewSerialNumber = InTarget.GetSerialNumber();
	
	return DeltaChange;
}

void ApplySelectionChange(FCurveEditorSelection& InOutChange, const FSelectionDeltaChange& InDeltaChange)
{
	SelectionDiffDetail::ModifySelection(InOutChange, InDeltaChange,
		[](const FCurveSelectionDeltaChange& CurveChange){ return CurveChange.AddedKeys; },
		[](const FCurveSelectionDeltaChange& CurveChange){ return CurveChange.RemovedKeys; }
		);

	SelectionDiffDetail::SetSerialNumberInternal(InOutChange, InDeltaChange.NewSerialNumber);
}
	
void RevertSelectionChange(FCurveEditorSelection& InOutChange, const FSelectionDeltaChange& InDeltaChange)
{
	SelectionDiffDetail::ModifySelection(InOutChange, InDeltaChange,
		// Swapping added and removed keys here effectively reverts the change.
		[](const FCurveSelectionDeltaChange& CurveChange){ return CurveChange.RemovedKeys; },
		[](const FCurveSelectionDeltaChange& CurveChange){ return CurveChange.AddedKeys; }
		);
	
	SelectionDiffDetail::SetSerialNumberInternal(InOutChange, InDeltaChange.OldSerialNumber);
}
}
