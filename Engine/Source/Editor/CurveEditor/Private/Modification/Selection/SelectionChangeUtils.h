// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

enum class ECurvePointType : uint8;
struct FCurveModelID;
struct FCurveEditorSelection;
struct FKeyHandle;

namespace UE::CurveEditor
{
struct FSelectionDeltaChange;

/** Computes the delta change to get from InOriginal to InTarget. */
FSelectionDeltaChange DiffSelection(const FCurveEditorSelection& InOriginal, const FCurveEditorSelection& InTarget);

/** Applies InDeltaChange to InOutSelection (redo operation). */
void ApplySelectionChange(FCurveEditorSelection& InOutChange, const FSelectionDeltaChange& InDeltaChange);
/** Reverts InDeltaChange from InOutSelection (undo operation).  */
void RevertSelectionChange(FCurveEditorSelection& InOutChange, const FSelectionDeltaChange& InDeltaChange);

// Private - needed for FAddInternal.
namespace SelectionDiffDetail { void AddKeysInternal(FCurveEditorSelection&, FCurveModelID, ECurvePointType, TArrayView<const FKeyHandle>); }
namespace SelectionDiffDetail { void SetSerialNumberInternal(FCurveEditorSelection& Selection, uint32 NewSerialNumber); }
}
