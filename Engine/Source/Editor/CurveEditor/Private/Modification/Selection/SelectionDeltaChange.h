// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"

namespace UE::CurveEditor
{
/** Changes made to a single curve's selection */
struct FCurveSelectionDeltaChange
{
	// Using TArray instead of TMap because we'll be iterating through all entries frequently. TArray is much faster for that.
	
	/** The keys that were added and its point type. */
	TArray<TPair<FKeyHandle, ECurvePointType>> AddedKeys;
	/**
	 * The keys that were removed.
	 * Can contain a key handle that is in AddedKeys. In that case, the point type was changed.
	 */
	TArray<TPair<FKeyHandle, ECurvePointType>> RemovedKeys;

	SIZE_T GetAllocatedSize() const
	{
		return AddedKeys.GetAllocatedSize() + RemovedKeys.GetAllocatedSize();
	}

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		AddedKeys.Shrink();
		RemovedKeys.Shrink();
	}
};

/** Changes made to the curve editor selection */
struct FSelectionDeltaChange
{
	/** The per curve changes. */
	TMap<FCurveModelID, FCurveSelectionDeltaChange> ChangedCurves;

	/** Serial number of selection before the change. */
	uint32 OldSerialNumber = 0;
	/** Serial number of selection after the change. */
	uint32 NewSerialNumber = 0;

	bool HasChanges() const { return !ChangedCurves.IsEmpty() || OldSerialNumber != NewSerialNumber; }
	operator bool() const { return HasChanges(); }

	SIZE_T GetAllocatedSize() const
	{
		return ChangedCurves.GetAllocatedSize()
			+ Algo::TransformAccumulate(ChangedCurves, [](const TPair<FCurveModelID, FCurveSelectionDeltaChange>& Pair)
			{
				return Pair.Value.GetAllocatedSize();
			}, 0);
	}

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		ChangedCurves.Shrink();
		for (TPair<FCurveModelID, FCurveSelectionDeltaChange>& Pair : ChangedCurves)
		{
			Pair.Value.Shrink();
		}
	}
};
}
