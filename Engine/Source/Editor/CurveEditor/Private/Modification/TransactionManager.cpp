// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/TransactionManager.h"

#include "CurveEditorTransactionObject.h"
#include "ScopedTransaction.h"
#include "Keys/GenericCurveChangeCommand.h"
#include "Misc/ITransaction.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::CurveEditor
{
FTransactionManager::FTransactionManager(TWeakPtr<FCurveEditor> InCurveEditor)
	: DummyTransactionObject(NewObject<UCurveEditorTransactionObject>(GetTransientPackage(), UCurveEditorTransactionObject::StaticClass(), NAME_None, RF_Transient | RF_Transactional))
{
	DummyTransactionObject->OwningCurveEditor = MoveTemp(InCurveEditor);
}

void FTransactionManager::AppendCurveChange(UObject* InObject, FGenericCurveChangeData InChange, const FText& InDescription) const
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = DummyTransactionObject->OwningCurveEditor.Pin())
	{
		// Problem: OnCurvesChangedDelegate should be called after calling StoreUndo in case any follow-up changes by the subscribers. Otherwise, we
		// may cause some subtle ordering issues during undo / redo.
		// StoreUndo takes ownership over the TUniquePtr so we cannot access the FGenericCurveChangeData from the FGenericCurveChangeCommand.
		// Copying here is a bit unfortunate, but it's the easiest solution without having to implement some advanced queueing system.
		const FGenericCurveChangeData CopiedChange = OnCurvesChangedDelegate.IsBound() ? InChange : FGenericCurveChangeData();
		
		// Start a transaction...
		const FScopedTransaction Transaction(InDescription);
		AppendChange(
			InObject,
			MakeUnique<FGenericCurveChangeCommand>(CurveEditorPin.ToSharedRef(), MoveTemp(InChange)),
			InDescription
			);
		// ... in case any follow-up changes want to add anything to the transaction.
		OnCurvesChangedDelegate.Broadcast(CopiedChange);
	}
}

void FTransactionManager::AppendCurveChange(FGenericCurveChangeData InChange, const FText& InDescription) const
{
	AppendCurveChange(DummyTransactionObject, MoveTemp(InChange), InDescription);
}

void FTransactionManager::AppendChange(UObject* InObject, TUniquePtr<FCurveEditorCommandChange> InChange, const FText& InDescription) const
{
	const FScopedTransaction Transaction(InDescription);
	if (GUndo // GUndo can be reset by external systems, e.g. during world tear down.
		&& ensure(InChange))
	{
		GUndo->StoreUndo(InObject, MoveTemp(InChange));
		
		OnCommandAppendedDelegate.Broadcast();
	}
}
	
void FTransactionManager::AppendChange(TUniquePtr<FCurveEditorCommandChange> InChange, const FText& InDescription) const
{
	AppendChange(DummyTransactionObject, MoveTemp(InChange), InDescription);
}

void FTransactionManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DummyTransactionObject);
}

FString FTransactionManager::GetReferencerName() const
{
	return TEXT("FTransactionManager");
}
}

