// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifyCurveObjectDetector.h"

#include "Algo/AnyOf.h"
#include "CurveEditor.h"
#include "Modification/Keys/Data/CurvesSnapshot.h"
#include "SCurveEditor.h"

namespace UE::CurveEditor
{
FModifyCurveObjectDetector::FModifyCurveObjectDetector(
	TWeakPtr<FCurveEditor> InCurveEditor, EModifyDetectionLevel InLevel, FString InWarningMessage, TArray<FCurveModelID> InTrackedCurves
	)
	: WeakCurveEditor(MoveTemp(InCurveEditor))
	, TrackedCurves(MoveTemp(InTrackedCurves))
	, DetectionLevel(InLevel)
	, WarningMessage(MoveTemp(InWarningMessage))
{
	FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FModifyCurveObjectDetector::OnObjectModified);
}

FModifyCurveObjectDetector::FModifyCurveObjectDetector(
	TWeakPtr<FCurveEditor> InCurveEditor, EModifyDetectionLevel InLevel, FString InWarningMessage, const FCurvesSnapshot& InSnapshot
	)
	: FModifyCurveObjectDetector(
		MoveTemp(InCurveEditor),
		InLevel,
		MoveTemp(InWarningMessage),
		[&InSnapshot]
		{
			TArray<FCurveModelID> Curves;
			InSnapshot.CurveData.GenerateKeyArray(Curves);
			return Curves;
		}())
{}

FModifyCurveObjectDetector::~FModifyCurveObjectDetector()
{
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
}

void FModifyCurveObjectDetector::OnObjectModified(UObject* Object) const
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!Object || !CurveEditorPin || DetectionLevel == EModifyDetectionLevel::None)
	{
		return;
	}

	const int32 bWasModified = Algo::AnyOf(CurveEditorPin->GetCurves(), [Object](const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair)
	{
		return Pair.Value && Pair.Value->GetOwningObject() == Object;
	});
	switch (DetectionLevel)
	{
	case EnsureAlways: ensureAlways(!bWasModified); [[fallthrough]]; // Will trigger double ensure once... whatever.
	case Ensure: ensure(!bWasModified);
	case Warn: UE_CLOGF(bWasModified, LogCurveEditor, Warning, "%ls", *WarningMessage)break;
		
	case None: [[fallthrough]];
	default: break;
	}
}
}
