// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

struct FCurveModelID;
class FCurveEditor;

namespace UE::CurveEditor
{
struct FCurvesSnapshot;

enum EModifyDetectionLevel : uint8 { None, Warn, Ensure, EnsureAlways };
	
/**
 * Logs warnings, ensures, etc. if any FCurveModel::GetOwningObject() is modified.
 * @see FScopedCurveChange.
 */
class FModifyCurveObjectDetector : public FNoncopyable
{
public:

	explicit FModifyCurveObjectDetector(
		TWeakPtr<FCurveEditor> InCurveEditor, EModifyDetectionLevel InLevel, FString InWarningMessage,
		TArray<FCurveModelID> InTrackedCurves
		);
	explicit FModifyCurveObjectDetector(
		TWeakPtr<FCurveEditor> InCurveEditor, EModifyDetectionLevel InLevel, FString InWarningMessage,
		const FCurvesSnapshot& InSnapshot
		);
	
	~FModifyCurveObjectDetector();

private:

	/** Used to get the owning objects for the curves. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** The curves that should not be modified. */
	const TArray<FCurveModelID> TrackedCurves;

	/** Determines the action to take when modification is detected. */
	const EModifyDetectionLevel DetectionLevel;
	/** Message to log in the warning */
	const FString WarningMessage;

	void OnObjectModified(UObject* Object) const;
};
}

