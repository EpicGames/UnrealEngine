// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Utils/ScopedChangeBase.h"

#include "CoreGlobals.h"
#include "CurveEditor.h"

namespace UE::CurveEditor
{
FScopedChangeBase::FScopedChangeBase(TWeakPtr<FCurveEditor> InCurveEditor, const FText& InDescription)
	: WeakCurveEditor(MoveTemp(InCurveEditor))
	, Description(InDescription)
	, bWasStartedInTransaction(GUndo != nullptr)
{}

bool FScopedChangeBase::IsCancelled() const
{
	return bIsCancelled || (GUndo == nullptr && bWasStartedInTransaction);
}

void FScopedChangeBase::TryAppendCommand(TUniquePtr<FCurveEditorCommandChange> InCommand) const
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (CurveEditorPin && !IsCancelled())
	{
		CurveEditorPin->GetTransactionManager()->AppendChange(MoveTemp(InCommand), Description);
	}
}
}
