// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Utils/ScopedCurveChange.h"

#include "CurveEditor.h"
#include "CurveModel.h"
#include "ModifyCurveObjectDetector.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Text.h"
#include "Misc/ITransaction.h"
#include "Misc/ScopeExit.h"
#include "Modification/Keys/Data/GenericCurveChangeData.h"
#include "Modification/Keys/GenericCurveChangeCommand.h"
#include "Modification/Keys/GenericCurveChangeUtils.h"
#include "Modification/Keys/SnapshotPerfDebugUtils.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "SCurveEditor.h"
#include "Modification/Keys/Diff/MultiGenericCommandAppender.h"
#include "UObject/Object.h"

namespace UE::CurveEditor
{
static TAutoConsoleVariable<bool> CVarLogUndoKeyChangeSize(
	TEXT("CurveEditor.LogDiffPerfStats"),
	false,
	TEXT("Whenever a key change is made, whether to log debug info (time spent diffing, bytes used for the diff, the final bytes on undo stack size).")
	);

static TAutoConsoleVariable<bool> CVarUseNewUndoSystem(
	TEXT("CurveEditor.UseNewUndoSystem"), true, TEXT("Whether to use the new undo system. Warning: There may still be some bugs.")
	);

static TAutoConsoleVariable<int32> CVarModifyDetectionLevel(
	TEXT("CurveEditor.UndoModifyDetectionLevel"), EModifyDetectionLevel::None, TEXT("During FScopedCurveChange, detect Modify() calls made to UObjects that own FCurveModels. Modify is not safe to call during FScopedCurveChange.\n0 - No detection\n1 - log a warning\n2 - Ensure & log warning\n3 - EnsureAlways & log warning")
);

const TCHAR* ModifyDetectionWarning = TEXT(
	"Modify() was called on curve object during FScopedCurveChange. "
	"This can lead to subtle undo / redo bugs. Avoid the Modify() call by introducing a command or deferring the operation if possible."
	);

template<typename TCallback> requires std::is_invocable_v<TCallback, FCurveModel&>
static void ForEachCurve(FCurveEditor& InCurveEditorPin, const FCurveChangeDiff& InChangeDiff, TCallback&& InCallback)
{
	for (const TPair<FCurveModelID, FCurveDiffingData>& Pair : InChangeDiff.GetSnapshot().CurveData)
	{
		if (FCurveModel* CurveModel = InCurveEditorPin.FindCurve(Pair.Key))
		{
			InCallback(*CurveModel);
		}
	}
}
	
FScopedCurveChange::FScopedCurveChange(
	TWeakPtr<FCurveEditor> InCurveEditor, FCurveChangeDiff InKeyChangeDiff, const FText& InDescription, EScopedKeyChangeFlags InFlags
	)
	: FScopedChangeBase(MoveTemp(InCurveEditor), InDescription.IsEmpty() ? NSLOCTEXT("CurveEditor", "ChangeKeys", "Change Keys") : InDescription)
	, Transaction(Description)
	, ChangeDiff(MoveTemp(InKeyChangeDiff))
	, Flags(InFlags)
	// Calling Modify() on the FCurveModel's owning objects during FScopedCurveChange can cause subtle undo bugs.
	// The intention here is to alert developers of that happening and fix it.
	// @see class doc string
	, CurveModificationDetector(CVarUseNewUndoSystem.GetValueOnAnyThread()
		? MakePimpl<FModifyCurveObjectDetector>(
			WeakCurveEditor, static_cast<EModifyDetectionLevel>(CVarModifyDetectionLevel.GetValueOnAnyThread()), ModifyDetectionWarning, ChangeDiff.GetSnapshot())
		: nullptr
		)
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!ensure(CurveEditorPin))
	{
		return;
	}
	
	if (CVarUseNewUndoSystem.GetValueOnAnyThread())
	{
		// In the new undo system, we want to avoid the FCurveModel::GetOwningObject()->Modify() from being called.
		// So we tell the curve models to invoke any PostChange-type of callbacks, which may cause Modify() calls.
		// @see class doc string for why.
		ForEachCurve(*CurveEditorPin, ChangeDiff, [](FCurveModel& CurveModel){ CurveModel.OpenChangeScope(); });
	}
	else
	{
		// If the new undo system is disabled, just do what we did before: call Modify().
		ForEachCurve(*CurveEditorPin, ChangeDiff, [](FCurveModel& CurveModel){ CurveModel.Modify(); });
	}
}

FScopedCurveChange::FScopedCurveChange(
	FCurvesSnapshotBuilder InDataToDiff, const FText& InDescription, EScopedKeyChangeFlags InFlags
	)
	: FScopedCurveChange(
		InDataToDiff.WeakCurveEditor,
		FCurveChangeDiff(InDataToDiff.WeakCurveEditor, MoveTemp(InDataToDiff.Snapshot), InDataToDiff.CapturedDataFlags),
		InDescription, InFlags
		)
{}

FScopedCurveChange::~FScopedCurveChange()
{
	if (CVarUseNewUndoSystem.GetValueOnAnyThread())
	{
		FinalizeChangesForNewUndoSystem();
	}
}
	
void FScopedCurveChange::FinalizeChangesForNewUndoSystem()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}
	
	// Match the constructor's OpenChangeScope call.
	// CloseChangeScope may invoke some internal logic that causes Modify() calls on the FCurveModel::GetOwningObject.
	// So it's important this call only takes places after we have appended our undo commands.
	ON_SCOPE_EXIT
	{
		CurveModificationDetector.Reset(); // We've added the commands by now, if any. Stop warning about Modify() before CloseChangeScope.
		ForEachCurve(*CurveEditorPin, ChangeDiff, [](FCurveModel& CurveModel){ CurveModel.CloseChangeScope(); });
	};

	const bool bRevertOnCancelled = EnumHasAnyFlags(Flags, EScopedKeyChangeFlags::RevertOnCancel);
	const bool bHasBeenCancelled = IsCancelled();
	if (bHasBeenCancelled)
	{
		Transaction.Cancel();
	}
	// Early out without expensive call to ComputeDiff.
	if (bHasBeenCancelled && !bRevertOnCancelled)
	{
		return;
	}

	FAutoScopedDurationTimer TimeMeasurement;
	const auto LogPerfData = [this, &TimeMeasurement](SIZE_T InAllocationSize)
	{
		UE_CLOGF(CVarLogUndoKeyChangeSize.GetValueOnAnyThread(), LogCurveEditor, Log, "Change'%ls' --- Undo size: %ls ---\t Diff: Time %f ms, %ls",
			GUndo ? *GUndo->GetContext().Title.ToString() : *Description.ToString(),
			*FText::AsMemory(InAllocationSize).ToString(),
			FTimespan::FromSeconds(TimeMeasurement.GetTime()).GetTotalMilliseconds(),
			*DumpSnapshotPerfData(ChangeDiff.GetSnapshot(), ECurvesSnapshotPerfFlags::All)
		); 
	};
	
	if (!bHasBeenCancelled)
	{
		FMultiGenericCommandAppender CommandAppender(CurveEditorPin);
		ChangeDiff.ProcessDiffs(CommandAppender);
		LogPerfData(CommandAppender.GetAllocationSize()); 
	}
	else
	{
		const FGenericCurveChangeData DeltaChange = ChangeDiff.ComputeDiff();
		GenericCurveChange::RevertChange(*CurveEditorPin, DeltaChange);
		LogPerfData(DeltaChange.GetAllocatedSize()); 
	}
}
}
