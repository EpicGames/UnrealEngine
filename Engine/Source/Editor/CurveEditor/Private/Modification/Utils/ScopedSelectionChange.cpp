// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modification/Utils/ScopedSelectionChange.h"

#include "CurveEditor.h"
#include "Internationalization/Internationalization.h"
#include "Modification/Selection/SelectionChangeCommand.h"
#include "Modification/Selection/SelectionChangeUtils.h"

#define LOCTEXT_NAMESPACE "FScopedSelectionChange"

namespace UE::CurveEditor
{
FScopedSelectionChange::FScopedSelectionChange(TWeakPtr<FCurveEditor> InCurveEditor, EScopedSelectionChangeFlags InFlags)
	: FScopedSelectionChange(MoveTemp(InCurveEditor), LOCTEXT("ChangeSelection", "Change selection"), InFlags)
{}

FScopedSelectionChange::FScopedSelectionChange(TWeakPtr<FCurveEditor> InCurveEditor, const FText& InDescription, EScopedSelectionChangeFlags InFlags)
	: FScopedChangeBase(MoveTemp(InCurveEditor), InDescription)
	, Flags(InFlags)
	
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}

	OriginalSelection = CurveEditorPin->Selection;
	
	if (EnumHasAnyFlags(Flags, EScopedSelectionChangeFlags::SuppressOnSelectionChangedEvent))
	{
		SelectionChangedSuppressor.Emplace(CurveEditorPin.ToSharedRef());
	}
}

FScopedSelectionChange::~FScopedSelectionChange()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (IsCancelled() || !CurveEditorPin)
	{
		return;
	}

	// Early out without expensive call to DiffSelection.
	const bool bRevertOnCancelled = EnumHasAnyFlags(Flags, EScopedSelectionChangeFlags::RevertOnCancel);
	const bool bHasBeenCancelled = IsCancelled();
	if (bHasBeenCancelled && !bRevertOnCancelled)
	{
		return;
	}

	FSelectionDeltaChange DeltaChange = DiffSelection(OriginalSelection, CurveEditorPin->Selection);
	if (!DeltaChange)
	{
		return;
	}
	
	if (!bHasBeenCancelled)
	{
		// Start a transaction...
		const FScopedTransaction Transaction(Description);
		
		TryAppendCommand(MakeUnique<FSelectionChangeCommand>(CurveEditorPin, MoveTemp(DeltaChange)));
		
		// ... so that if the suppression counter is decremented to 0, the event fires as part of the transaction.
		// This way any changes made will be recorded as part of the same transaction.
		SelectionChangedSuppressor.Reset(); 
	}
	else
	{
		RevertSelectionChange(CurveEditorPin->Selection, DeltaChange);
	}
}
}

#undef LOCTEXT_NAMESPACE