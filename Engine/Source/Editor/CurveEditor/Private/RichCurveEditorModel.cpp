// Copyright Epic Games, Inc. All Rights Reserved.

#include "RichCurveEditorModel.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "RichCurveKeyProxy.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

class FCurveEditor;

namespace UE::CurveEditor::RichModelDetail
{
static bool IsAuto(ERichCurveTangentMode TangentMode)
{
	return (TangentMode == RCTM_Auto || TangentMode == RCTM_SmartAuto);
}

/**
 * Shared implementation for getting keys.
 *
 * Typically, attributes reflect the settings that the user has manually configured for the keys, so certain attributes may remain unset.
 * For instance, when TangentMode == RCTM_Auto, tangents and weights are automatically computed, meaning attributes like ArriveTangent are not explicitly set.
 * Setting ArriveTangent would imply a user-defined value, which is incompatible with TangentMode == RCTM_Auto.
 *
 * Sometimes API users still need a way to get the auto-computed values, like those of tangents, though.
 * For this, use bAllAttributes parameter:
 * - true: get all attributes, even the auto-computed ones. The values are useful for e.g. UI visualization. Do not pass them to SetKeyAttributes.
 * - false: get only the attributes that were set by the user (i.e. skip returning auto-computed values). You can pass them to SetKeyAttributes, e.g. for copy paste.
 */
template<bool bAllAttributes>
static void GetKeyAttributes(
	const FRichCurve& InRichCurve,
	TArrayView<const FKeyHandle> InKeys,
	TArrayView<FKeyAttributes> OutAttributes
	)
{
	const TArray<FRichCurveKey>& AllKeys = InRichCurve.GetConstRefOfKeys();
	if (AllKeys.Num() == 0)
	{
		return;
	}

	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		if (InRichCurve.IsKeyHandleValid(InKeys[Index]))
		{
			const FRichCurveKey& ThisKey    = InRichCurve.GetKeyRef(InKeys[Index]);
			FKeyAttributes&      Attributes = OutAttributes[Index];

			Attributes.SetInterpMode(ThisKey.InterpMode);

			// If the previous key is cubic, show the arrive tangent handle even if this key is constant
			const FKeyHandle PreviousKeyHandle = InRichCurve.GetPreviousKey(InKeys[Index]);
			const bool bGetArriveTangent = bAllAttributes
				&& InRichCurve.IsKeyHandleValid(PreviousKeyHandle) && InRichCurve.GetKeyRef(PreviousKeyHandle).InterpMode == RCIM_Cubic;
			if (bGetArriveTangent)
			{
				Attributes.SetArriveTangent(ThisKey.ArriveTangent);
			}

			if (ThisKey.InterpMode != RCIM_Constant && ThisKey.InterpMode != RCIM_Linear)
			{
				Attributes.SetTangentMode(ThisKey.TangentMode);

				// The remaining settings (arrive / leave tangent, arrive / leave weight, weight mode) can only be user specified, i.e. cannot be set when in auto.
				// Setting any of them would imply they specified by the user, which would contradict the setting that it's auto. 
				if (!bAllAttributes && IsAuto(ThisKey.TangentMode))
				{
					continue;
				}
				
				Attributes.SetArriveTangent(ThisKey.ArriveTangent);
				Attributes.SetLeaveTangent(ThisKey.LeaveTangent);

				if (ThisKey.InterpMode == RCIM_Cubic)
				{
					Attributes.SetTangentWeightMode(ThisKey.TangentWeightMode);
					if (ThisKey.TangentWeightMode != RCTWM_WeightedNone)
					{
						Attributes.SetArriveTangentWeight(ThisKey.ArriveTangentWeight);
						Attributes.SetLeaveTangentWeight(ThisKey.LeaveTangentWeight);
					}
				}
			}
		}
	}
	
}
}


void RefineCurvePoints(const FRichCurve& RichCurve, double TimeThreshold, float ValueThreshold, TArray<TTuple<double, double>>& InOutPoints)
{
	const float InterpTimes[] = { 0.25f, 0.5f, 0.6f };

	for (int32 Index = 0; Index < InOutPoints.Num() - 1; ++Index)
	{
		TTuple<double, double> Lower = InOutPoints[Index];
		TTuple<double, double> Upper = InOutPoints[Index + 1];

		if ((Upper.Get<0>() - Lower.Get<0>()) >= TimeThreshold)
		{
			bool bSegmentIsLinear = true;

			TTuple<double, double> Evaluated[UE_ARRAY_COUNT(InterpTimes)] = { TTuple<double, double>(0, 0) };

			for (int32 InterpIndex = 0; InterpIndex < UE_ARRAY_COUNT(InterpTimes); ++InterpIndex)
			{
				double& EvalTime = Evaluated[InterpIndex].Get<0>();

				EvalTime = FMath::Lerp(Lower.Get<0>(), Upper.Get<0>(), InterpTimes[InterpIndex]);

				float Value = RichCurve.Eval(EvalTime);

				const float LinearValue = FMath::Lerp(Lower.Get<1>(), Upper.Get<1>(), InterpTimes[InterpIndex]);
				if (bSegmentIsLinear)
				{
					bSegmentIsLinear = FMath::IsNearlyEqual(Value, LinearValue, ValueThreshold);
				}

				Evaluated[InterpIndex].Get<1>() = Value;
			}

			if (!bSegmentIsLinear)
			{
				// Add the point
				InOutPoints.Insert(Evaluated, UE_ARRAY_COUNT(Evaluated), Index + 1);
				--Index;
			}
		}
	}
}

/**
 * Buffered curve implementation for a rich curve, stores a copy of the rich curve in order to draw itself.
 */
class FRichBufferedCurveModel : public IBufferedCurveModel
{
public:
	FRichBufferedCurveModel(const FRichCurve& InRichCurve, TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes,
		const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
		: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
		, RichCurve(InRichCurve)
	{}

	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override
	{
		const double StartTimeSeconds = ScreenSpace.GetInputMin();
		const double EndTimeSeconds = ScreenSpace.GetInputMax();
		const double TimeThreshold = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerInput());
		const double ValueThreshold = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerOutput());

		InterpolatingPoints.Add(MakeTuple(StartTimeSeconds, double(RichCurve.Eval(StartTimeSeconds))));

		for (const FRichCurveKey& Key : RichCurve.GetConstRefOfKeys())
		{
			if (Key.Time > StartTimeSeconds && Key.Time < EndTimeSeconds)
			{
				InterpolatingPoints.Add(MakeTuple(double(Key.Time), double(Key.Value)));
			}
		}

		InterpolatingPoints.Add(MakeTuple(EndTimeSeconds, double(RichCurve.Eval(EndTimeSeconds))));

		int32 OldSize = InterpolatingPoints.Num();
		do
		{
			OldSize = InterpolatingPoints.Num();
			RefineCurvePoints(RichCurve, TimeThreshold, ValueThreshold, InterpolatingPoints);
		} while (OldSize != InterpolatingPoints.Num());
	}

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		OutValue = RichCurve.Eval(InTime);
		return true;
	}

private:
	FRichCurve RichCurve;
};

FRichCurveEditorModel::FRichCurveEditorModel( UObject* InOwner)
	: WeakOwner(InOwner), ClampInputRange(TRange<double>(TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max()))
{
}

const void* FRichCurveEditorModel::GetCurve() const
{
	if(IsValid())
	{
		return &GetReadOnlyRichCurve();
	}
	return nullptr;
}

void FRichCurveEditorModel::Modify()
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->SetFlags(RF_Transactional);
			Owner->Modify();
		}
	}
}

void FRichCurveEditorModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InKeyAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	check(InKeyPositions.Num() == InKeyAttributes.Num() && (!OutKeyHandles || OutKeyHandles->Num() == InKeyPositions.Num()));

	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->Modify();

			TArray<FKeyHandle> NewKeyHandles;
			NewKeyHandles.SetNumUninitialized(InKeyPositions.Num());

			FRichCurve& RichCurve = GetRichCurve();

			for (int32 Index = 0; Index < InKeyPositions.Num(); ++Index)
			{
				FKeyPosition   Position   = InKeyPositions[Index];
				FKeyAttributes Attributes = InKeyAttributes[Index];

				const TRange<double> InputRange = ClampInputRange.Get();

				constexpr bool bUnwindRotation = false;
				constexpr float Tolearance = 0.0f; 
				const FKeyHandle NewHandle = RichCurve.UpdateOrAddKey(FMath::Clamp(Position.InputValue, InputRange.GetLowerBoundValue(), InputRange.GetUpperBoundValue()), Position.OutputValue, bUnwindRotation, Tolearance);
				if (NewHandle != FKeyHandle::Invalid())
				{
					NewKeyHandles[Index] = NewHandle;
					if (OutKeyHandles)
					{
						(*OutKeyHandles)[Index] = NewHandle;
					}
				}
			}

			// We reuse SetKeyAttributes here as there is complex logic determining which parts of the attributes are valid to pass on.
			// For now we need to duplicate the new key handle array due to API mismatch. This will auto-calculate tangents if required.
			SetKeyAttributes(NewKeyHandles, InKeyAttributes);

			CurveModifiedDelegate.Broadcast();
		}
	}
}

bool FRichCurveEditorModel::Evaluate(double Time, double& OutValue) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			OutValue = GetReadOnlyRichCurve().Eval(Time);
			return true;
		}
	}

	return false;
}

void FRichCurveEditorModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->Modify();
			FRichCurve& RichCurve = GetRichCurve();

			double CurrentValue = 0.0;
			Evaluate(InCurrentTime, CurrentValue);

			for (FKeyHandle Handle : InKeys)
			{
				if (RichCurve.IsKeyHandleValid(Handle))
				{
					RichCurve.DeleteKey(Handle);
				}
			}
			if (RichCurve.GetNumKeys() == 0)
			{
				RichCurve.DefaultValue = CurrentValue;
			}

			CurveModifiedDelegate.Broadcast();
		}
	}
}

void FRichCurveEditorModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InOutPoints) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const double StartTimeSeconds = ScreenSpace.GetInputMin();
			const double EndTimeSeconds   = ScreenSpace.GetInputMax();
			const double TimeThreshold    = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerInput());
			const double ValueThreshold   = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerOutput());

			const FRichCurve& RichCurve = GetReadOnlyRichCurve();

			InOutPoints.Add(MakeTuple(StartTimeSeconds, double(RichCurve.Eval(StartTimeSeconds))));

			for (const FRichCurveKey& Key : RichCurve.GetConstRefOfKeys())
			{
				if (Key.Time > StartTimeSeconds && Key.Time < EndTimeSeconds)
				{
					InOutPoints.Add(MakeTuple(double(Key.Time), double(Key.Value)));
				}
			}

			InOutPoints.Add(MakeTuple(EndTimeSeconds, double(RichCurve.Eval(EndTimeSeconds))));

			int32 OldSize = InOutPoints.Num();
			do
			{
				OldSize = InOutPoints.Num();
				RefineCurvePoints(RichCurve, TimeThreshold, ValueThreshold, InOutPoints);
			}
			while(OldSize != InOutPoints.Num());
		}
	}
}

void FRichCurveEditorModel::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			for (auto It = RichCurve.GetKeyHandleIterator(); It; ++It)
			{
				if(RichCurve.IsKeyHandleValid(*It))
				{
					const FRichCurveKey& Key = RichCurve.GetKeyRef(*It);
					if (Key.Time >= MinTime && Key.Time <= MaxTime && Key.Value >= MinValue && Key.Value <= MaxValue)
					{
						OutKeyHandles.Add(*It);
					}
				}
			}
		}
	}
}

void FRichCurveEditorModel::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	if (PointType == ECurvePointType::ArriveTangent || PointType == ECurvePointType::LeaveTangent)
	{
		OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.TangentHandle");
		OutDrawInfo.ScreenSize = FVector2D(9, 9);
	}
	else
	{
		// All keys are the same size by default
		OutDrawInfo.ScreenSize = FVector2D(11, 11);

		ERichCurveInterpMode KeyType = (IsValid() && GetReadOnlyRichCurve().IsKeyHandleValid(InKeyHandle)) ? GetReadOnlyRichCurve().GetKeyRef(InKeyHandle).InterpMode.GetValue() : RCIM_None;
		switch (KeyType)
		{
		case ERichCurveInterpMode::RCIM_Constant:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.ConstantKey");
			break;
		case ERichCurveInterpMode::RCIM_Linear:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.LinearKey");
			break;
		case ERichCurveInterpMode::RCIM_Cubic:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.CubicKey");
			break;
		default:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.Key");
			break;
		}
	}
}

void FRichCurveEditorModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			for (int32 Index = 0; Index < InKeys.Num(); ++Index)
			{
				if (RichCurve.IsKeyHandleValid(InKeys[Index]))
				{
					const FRichCurveKey& Key = RichCurve.GetKeyRef(InKeys[Index]);

					OutKeyPositions[Index].InputValue  = Key.Time;
					OutKeyPositions[Index].OutputValue = Key.Value;
				}
			}
		}
	}
}

void FRichCurveEditorModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	if (IsReadOnly())
	{
		return;
	}

	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->Modify();

			FRichCurve& RichCurve = GetRichCurve();
			for (int32 Index = 0; Index < InKeys.Num(); ++Index)
			{
				FKeyHandle Handle = InKeys[Index];
				if (RichCurve.IsKeyHandleValid(Handle))
				{
					// Set key time last so we don't have to worry about the key handle changing
					RichCurve.GetKey(Handle).Value = InKeyPositions[Index].OutputValue;
					const TRange<double> InputRange = ClampInputRange.Get();
					RichCurve.SetKeyTime(Handle, FMath::Clamp(InKeyPositions[Index].InputValue, InputRange.GetLowerBoundValue(), InputRange.GetUpperBoundValue()));
				}
			}
			RichCurve.AutoSetTangents();
			FPropertyChangedEvent PropertyChangeStruct(nullptr, ChangeType);
			Owner->PostEditChangeProperty(PropertyChangeStruct);

			CurveModifiedDelegate.Broadcast();
		}
	}
}

void FRichCurveEditorModel::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	if (UObject* Owner = WeakOwner.Get(); Owner && IsValid())
	{
		const FRichCurve& RichCurve = GetReadOnlyRichCurve();
		UE::CurveEditor::RichModelDetail::GetKeyAttributes<true>(RichCurve, InKeys, OutAttributes);
	}
}

void FRichCurveEditorModel::GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	if (UObject* Owner = WeakOwner.Get(); Owner && IsValid())
	{
		const FRichCurve& RichCurve = GetReadOnlyRichCurve();
		UE::CurveEditor::RichModelDetail::GetKeyAttributes<false>(RichCurve, InKeys, OutAttributes);
	}
}

void FRichCurveEditorModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
	if (IsReadOnly())
	{
		return;
	}

	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			FRichCurve& RichCurve = GetRichCurve();
			const TArray<FRichCurveKey>& AllKeys = RichCurve.GetConstRefOfKeys();
			if (AllKeys.Num() == 0)
			{
				return;
			}

			Owner->Modify();

			const FRichCurveKey* FirstKey = &AllKeys[0];
			const FRichCurveKey* LastKey  = &AllKeys.Last();

			bool bAutoSetTangents = false;

			for (int32 Index = 0; Index < InKeys.Num(); ++Index)
			{
				FKeyHandle KeyHandle = InKeys[Index];
				if (RichCurve.IsKeyHandleValid(KeyHandle))
				{
					FRichCurveKey*        ThisKey    = &RichCurve.GetKey(KeyHandle);
					const FKeyAttributes& Attributes = InAttributes[Index];

					if (Attributes.HasInterpMode())    { ThisKey->InterpMode  = Attributes.GetInterpMode();  bAutoSetTangents = true; }
					if (Attributes.HasTangentMode())
					{
						ThisKey->TangentMode = Attributes.GetTangentMode();
						if (ThisKey->TangentMode == RCTM_Auto)
						{
							ThisKey->TangentWeightMode = RCTWM_WeightedNone;
						}
						bAutoSetTangents = true;
					}
					if (Attributes.HasTangentWeightMode()) 
					{ 
						if (ThisKey->TangentWeightMode == RCTWM_WeightedNone) //set tangent weights to default use
						{
							const float OneThird = 1.0f / 3.0f;

							//calculate a tangent weight based upon tangent and time difference
							//calculate arrive tangent weight
							if (ThisKey != FirstKey)
							{
								const float X = ThisKey->Time - RichCurve.GetKey(RichCurve.GetPreviousKey(KeyHandle)).Time;
								const float Y = ThisKey->ArriveTangent *X;
								ThisKey->ArriveTangentWeight = FMath::Sqrt(X*X + Y*Y) * OneThird;
							}
							//calculate leave weight
							if(ThisKey != LastKey)
							{
								const float X = RichCurve.GetKey(RichCurve.GetNextKey(KeyHandle)).Time - ThisKey->Time;
								const float Y = ThisKey->LeaveTangent *X;
								ThisKey->LeaveTangentWeight = FMath::Sqrt(X*X + Y*Y) * OneThird;
							}
						}
						ThisKey->TangentWeightMode = Attributes.GetTangentWeightMode();

						if( ThisKey->TangentWeightMode != RCTWM_WeightedNone )
						{
							if (ThisKey->TangentMode != RCTM_User && ThisKey->TangentMode != RCTM_Break)
							{
								ThisKey->TangentMode = RCTM_User;
							}
						}
					}

					if (Attributes.HasArriveTangent())
					{
						if (ThisKey->TangentMode == RCTM_Auto)
						{
							ThisKey->TangentMode = RCTM_User;
							ThisKey->TangentWeightMode = RCTWM_WeightedNone;
						}

						ThisKey->ArriveTangent = Attributes.GetArriveTangent();
						if (ThisKey->InterpMode == RCIM_Cubic && ThisKey->TangentMode != RCTM_Break)
						{
							ThisKey->LeaveTangent = ThisKey->ArriveTangent;
						}
					}

					if (Attributes.HasLeaveTangent())
					{
						if (ThisKey->TangentMode == RCTM_Auto)
						{
							ThisKey->TangentMode = RCTM_User;
							ThisKey->TangentWeightMode = RCTWM_WeightedNone;
						}

						ThisKey->LeaveTangent = Attributes.GetLeaveTangent();
						if (ThisKey->InterpMode == RCIM_Cubic && ThisKey->TangentMode != RCTM_Break)
						{
							ThisKey->ArriveTangent = ThisKey->LeaveTangent;
						}
					}

					if (Attributes.HasArriveTangentWeight())
					{
						if (ThisKey->TangentMode == RCTM_Auto)
						{
							ThisKey->TangentMode = RCTM_User;
							ThisKey->TangentWeightMode = RCTWM_WeightedNone;
						}

						ThisKey->ArriveTangentWeight = Attributes.GetArriveTangentWeight();
						if (ThisKey->InterpMode == RCIM_Cubic && ThisKey->TangentMode != RCTM_Break)
						{
							ThisKey->LeaveTangentWeight = ThisKey->ArriveTangentWeight;
						}
					}

					if (Attributes.HasLeaveTangentWeight())
					{
				
						if (ThisKey->TangentMode == RCTM_Auto)
						{
							ThisKey->TangentMode = RCTM_User;
							ThisKey->TangentWeightMode = RCTWM_WeightedNone;
						}

						ThisKey->LeaveTangentWeight = Attributes.GetLeaveTangentWeight();
						if (ThisKey->InterpMode == RCIM_Cubic && ThisKey->TangentMode != RCTM_Break)
						{
							ThisKey->ArriveTangentWeight = ThisKey->LeaveTangentWeight;
						}
					}
				}
			}

			if (bAutoSetTangents)
			{
				RichCurve.AutoSetTangents();
			}

			FPropertyChangedEvent PropertyChangeStruct(nullptr, ChangeType);
			Owner->PostEditChangeProperty(PropertyChangeStruct);
			CurveModifiedDelegate.Broadcast();
		}
	}
}

void FRichCurveEditorModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			OutCurveAttributes.SetPreExtrapolation(RichCurve.PreInfinityExtrap);
			OutCurveAttributes.SetPostExtrapolation(RichCurve.PostInfinityExtrap);
		}
	}
}

void FRichCurveEditorModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->Modify();

			FRichCurve& RichCurve = GetRichCurve();

			if (InCurveAttributes.HasPreExtrapolation())
			{
				RichCurve.PreInfinityExtrap = InCurveAttributes.GetPreExtrapolation();
			}

			if (InCurveAttributes.HasPostExtrapolation())
			{
				RichCurve.PostInfinityExtrap = InCurveAttributes.GetPostExtrapolation();
			}

			FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
			Owner->PostEditChangeProperty(PropertyChangeStruct);
			CurveModifiedDelegate.Broadcast();
		}
	}
}

 void FRichCurveEditorModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
 {
	 if (UObject* Owner = WeakOwner.Get())
	 {
		 if (IsValid())
		 {
			 for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
			 {
				 URichCurveKeyProxy* NewProxy = NewObject<URichCurveKeyProxy>(GetTransientPackage(), NAME_None);

				 NewProxy->Initialize(InWeakCurveEditor, InCurveModelID, InKeyHandles[Index], WeakOwner);
				 OutObjects[Index] = NewProxy;
			 }
		 }
	 }
 }


TUniquePtr<IBufferedCurveModel> FRichCurveEditorModel::CreateBufferedCurveCopy() const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();

			TArray<FKeyHandle> TargetKeyHandles;
			for (auto It = RichCurve.GetKeyHandleIterator(); It; ++It)
			{
				if(RichCurve.IsKeyHandleValid(*It))
				{
					TargetKeyHandles.Add(*It);
				}
			}

			TArray<FKeyPosition> KeyPositions;
			KeyPositions.SetNumUninitialized(TargetKeyHandles.Num());
			TArray<FKeyAttributes> KeyAttributes;
			KeyAttributes.SetNumUninitialized(TargetKeyHandles.Num());
			GetKeyPositions(TargetKeyHandles, KeyPositions);
			GetKeyAttributes(TargetKeyHandles, KeyAttributes);

			double ValueMin = 0.f, ValueMax = 1.f;
			GetValueRange(ValueMin, ValueMax);

			return MakeUnique<FRichBufferedCurveModel>(RichCurve, MoveTemp(KeyPositions), MoveTemp(KeyAttributes), GetLongDisplayName().ToString(), ValueMin, ValueMax);
		}
	}

	return nullptr;
}

void FRichCurveEditorModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			float MinTimeFloat = 0.f, MaxTimeFloat = 0.f;
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			RichCurve.GetTimeRange(MinTimeFloat, MaxTimeFloat);

			MinTime = MinTimeFloat;
			MaxTime = MaxTimeFloat;
		}
	}
}

void FRichCurveEditorModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			float MinValueFloat = 0.f, MaxValueFloat = 0.f;
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			RichCurve.GetValueRange(MinValueFloat, MaxValueFloat);

			MinValue = MinValueFloat;
			MaxValue = MaxValueFloat;
		}
	}
}

int32 FRichCurveEditorModel::GetNumKeys() const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			return GetReadOnlyRichCurve().GetNumKeys();
		}
	}
	return 0;
}

void FRichCurveEditorModel::GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			const FRichCurve& RichCurve = GetReadOnlyRichCurve();
			if (RichCurve.IsKeyHandleValid(InKeyHandle))
			{
				FKeyHandle NextKeyHandle = RichCurve.GetNextKey(InKeyHandle);

				if (RichCurve.IsKeyHandleValid(NextKeyHandle))
				{
					OutNextKeyHandle = NextKeyHandle;
				}

				FKeyHandle PreviousKeyHandle = RichCurve.GetPreviousKey(InKeyHandle);

				if (RichCurve.IsKeyHandleValid(PreviousKeyHandle))
				{
					OutPreviousKeyHandle = PreviousKeyHandle;
				}
			}
		}
	}
}

TPair<ERichCurveInterpMode, ERichCurveTangentMode> FRichCurveEditorModel::GetInterpolationMode(const double& InTime, ERichCurveInterpMode DefaultInterpolationMode, ERichCurveTangentMode DefaultTangentMode) const
{
	if (IsValid())
	{
		const FRichCurve& RichCurve = GetReadOnlyRichCurve();

		if (!RichCurve.Keys.IsEmpty())
		{
			FKeyHandle ReferenceKeyHandle = FKeyHandle::Invalid();
			for (auto It = RichCurve.GetKeyHandleIterator(); It; ++It)
			{
				if (RichCurve.IsKeyHandleValid(*It))
				{
					const FRichCurveKey& Key = RichCurve.GetKeyRef(*It);
					if (Key.Time < InTime)
					{
						ReferenceKeyHandle = *It;
					}
					else
					{
						// we try to get the key just before the reference time, if it does not exist, we use the key right after
						if (!RichCurve.IsKeyHandleValid(ReferenceKeyHandle))
						{
							ReferenceKeyHandle = *It;
						}
						break;
					}
				}
			}

			if (RichCurve.IsKeyHandleValid(ReferenceKeyHandle))
			{
				TArray<FKeyAttributes> KeyAttributes;
				KeyAttributes.SetNum(1);
				GetKeyAttributes({ ReferenceKeyHandle }, KeyAttributes);

				ERichCurveInterpMode InterpMode = KeyAttributes[0].GetInterpMode();
				ERichCurveTangentMode TangentMode = KeyAttributes[0].HasTangentMode() ? KeyAttributes[0].GetTangentMode() : DefaultTangentMode;
				
				//if we are cubic, with anything but auto tangents we use the default instead, since they will give us flat tangents which aren't good
				if (InterpMode == ERichCurveInterpMode::RCIM_Cubic &&
					(TangentMode != ERichCurveTangentMode::RCTM_Auto && TangentMode != ERichCurveTangentMode::RCTM_SmartAuto))
				{
					TangentMode = DefaultTangentMode;
				}

				return TPair<ERichCurveInterpMode, ERichCurveTangentMode>(InterpMode, TangentMode);
			}
		}
	}

	return TPair<ERichCurveInterpMode, ERichCurveTangentMode>(DefaultInterpolationMode, DefaultTangentMode);
}

FRichCurveEditorModelRaw::FRichCurveEditorModelRaw(FRichCurve* InRichCurve, UObject* InOwner)
	: FRichCurveEditorModel(InOwner)
	, RichCurve(InRichCurve)
{
	checkf(RichCurve, TEXT("If is not valid to provide a null rich curve to this class"));
}

bool FRichCurveEditorModelRaw::IsReadOnly() const
{
	return ReadOnlyAttribute.Get(false);
}

void FRichCurveEditorModelRaw::SetIsReadOnly(TAttribute<bool> InReadOnlyAttribute)
{
	ReadOnlyAttribute = InReadOnlyAttribute;
}

FRichCurve& FRichCurveEditorModelRaw::GetRichCurve()
{
	return *RichCurve;
}

const FRichCurve& FRichCurveEditorModelRaw::GetReadOnlyRichCurve() const
{
	return *RichCurve;
}