// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CurveEditor.h"
#include "CurveEditorKeyProxy.h"
#include "Curves/RichCurve.h"
#include "RichCurveEditorModel.h"

#include "RichCurveKeyProxy.generated.h"

UCLASS()
class URichCurveKeyProxy : public UObject, public ICurveEditorKeyProxy
{
public:
	GENERATED_BODY()

	/**
	 * Initialize this key proxy object by caching the underlying key object, and retrieving the time/value each tick
	 */
	void Initialize(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, FKeyHandle InKeyHandle, TWeakObjectPtr<UObject> InWeakOwner)
	{
		WeakCurveEditor = InWeakCurveEditor;
		CurveModelID = InCurveModelID;
		KeyHandle  = InKeyHandle;
		WeakOwner  = InWeakOwner;

		if(UObject* Owner = WeakOwner.Get())
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (CurveEditor)
			{
				if (FRichCurveEditorModel* Model = static_cast<FRichCurveEditorModel*>(CurveEditor->FindCurve(CurveModelID)))
				{
					const FRichCurve& RichCurve = Model->GetReadOnlyRichCurve();
					if (RichCurve.IsKeyHandleValid(KeyHandle))
					{
						Value = RichCurve.GetKey(KeyHandle);
					}
				}
			}
		}
	}

private:

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		if(UObject* Owner = WeakOwner.Get())
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (CurveEditor)
			{
				if (FRichCurveEditorModel* Model = static_cast<FRichCurveEditorModel*>(CurveEditor->FindCurve(CurveModelID)))
				{
					FRichCurve& RichCurve = Model->GetRichCurve();
					if(RichCurve.IsKeyHandleValid(KeyHandle))
					{
						Owner->Modify();

						FRichCurveKey& ActualKey = RichCurve.GetKey(KeyHandle);
						const FKeyPosition KeyPosition(Value.Time, Value.Value);
						Model->SetKeyPositions({KeyHandle}, {KeyPosition}, PropertyChangedEvent.ChangeType);
				
						Owner->PostEditChangeProperty(PropertyChangedEvent);
						Model->OnCurveModified().Broadcast();
					}
				}
			}
		}	
	}

	virtual void UpdateValuesFromRawData() override
	{
		if(UObject* Owner = WeakOwner.Get())
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (CurveEditor)
			{
				if (FRichCurveEditorModel* Model = static_cast<FRichCurveEditorModel*>(CurveEditor->FindCurve(CurveModelID)))
				{
					const FRichCurve& RichCurve = Model->GetReadOnlyRichCurve();
					if (RichCurve.IsKeyHandleValid(KeyHandle))
					{
						Value = RichCurve.GetKey(KeyHandle);
					}
				}
			}
		}
	}

private:

	/** User-facing value of the key, applied to the actual key on PostEditChange, and updated every tick */
	UPROPERTY(EditAnywhere, Category="Key", meta=(ShowOnlyInnerProperties))
	FRichCurveKey Value;

private:

	/** Cached curve editor */
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	/** Cached curve model ID for this rich curve's key */
	FCurveModelID CurveModelID;
	/** Cached key handle that this key proxy relates to */
	FKeyHandle KeyHandle = FKeyHandle::Invalid();
	/** Cached owner in which the raw curve resides */
	TWeakObjectPtr<UObject> WeakOwner;
};