// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorPanel.h"

#include "Algo/Sort.h"
#include "CommonFrameRates.h"
#include "Containers/Array.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "CurveEditorEditObjectContainer.h"
#include "CurveEditorKeyProxy.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSettings.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorViewRegistry.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Filters/FilterUtils.h"
#include "Filters/CurveEditorBakeFilter.h"
#include "Filters/CurveEditorReduceFilter.h"
#include "Filters/SCurveEditorFilterPanel.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "ICurveEditorDragOperation.h"
#include "ICurveEditorExtension.h"
#include "ICurveEditorModule.h"
#include "ICurveEditorToolExtension.h"
#include "IPropertyRowGenerator.h"
#include "ISequencerWidgetsModule.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/PaintGeometry.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FrameRate.h"
#include "Modules/ModuleManager.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "SCurveEditorToolProperties.h"
#include "SCurveEditorView.h"
#include "SCurveEditorViewContainer.h"
#include "SCurveKeyDetailPanel.h"
#include "SGridLineSpacingList.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Algo/AnyOf.h"
#include "Filters/CurveEditorEulerFilter.h"
#include "Filters/PromotedFilterContainer.h"
#include "Misc/ResizeParamUtils.h"
#include "Misc/Attributes/AttributeAccumulationUtils.h"
#include "Misc/Attributes/SharedCurveInfoModel.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SFrameRatePicker.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
class SWidget;
class SWindow;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SCurveEditorPanel"

int32 GCurveEditorPinnedViews = 0;
static FAutoConsoleVariableRef CVarCurveEditorPinnedViews(
	TEXT("CurveEditor.PinnedViews"),
	GCurveEditorPinnedViews,
	TEXT("Whether pinning a curve should also cause it to be exclusively added to a pinned view or not (default: off), rather than simply always remain visible.")
);

int32 GCurveEditorMaxCurvesPerPinnedView = 0;
static FAutoConsoleVariableRef CVarCurveEditorMaxCurvesPerPinnedView(
	TEXT("CurveEditor.MaxCurvesPerPinnedView"),
	GCurveEditorMaxCurvesPerPinnedView,
	TEXT("When CurveEditor.PinnedViews is 1, defines the maximum number of curves allowed on a pinned view (0 for no maximum).")
);

/**
 * Implemented as a friend struct to SCurveEditorView to ensure that SCurveEditorPanel is the only thing that can add/remove curves from views
 * whilst disallowing access to any other private members;
 */
struct FCurveEditorPanelViewTracker
{
	static void AddCurveToView(SCurveEditorView* View, FCurveModelID InCurveID)
	{
		View->AddCurve(InCurveID);
	}
	static void RemoveCurveFromView(SCurveEditorView* View, FCurveModelID InCurveID)
	{
		View->RemoveCurve(InCurveID);
	}
};

SCurveEditorPanel::SCurveEditorPanel()
	: PendingFocus(FPendingWidgetFocus::MakeNoTextEdit())
	, bNeedsRefresh(true)
	, CachedActiveCurvesSerialNumber(-1)
{
	EditObjects = MakeUnique<FCurveEditorEditObjectContainer>();
}

SCurveEditorPanel::~SCurveEditorPanel()
{
	// Attempt to close a dialog if it's open. It has a weak reference to us and doesn't work well when it's invalid.
	SCurveEditorFilterPanel::CloseDialog();
}

void SCurveEditorPanel::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
{
	GridLineTintAttribute = InArgs._GridLineTint;
	DisabledTimeSnapTooltipAttribute = InArgs._DisabledTimeSnapTooltip;
	WeakTabManager = InArgs._TabManager;

	CachedSelectionSerialNumber = 0;

	CurveEditor = InCurveEditor;

	CurveEditor->SetPanel(SharedThis(this));

	CurveEditor->BindCommands();
	CurveEditor->SetTimeSliderController(InArgs._ExternalTimeSliderController);

	CurveEditor->OnActiveToolChangedDelegate.AddSP(this, &SCurveEditorPanel::OnCurveEditorToolChanged);

	CommandList = MakeShared<FUICommandList>();
	CommandList->Append(InCurveEditor->GetCommands().ToSharedRef());
	
	ToolbarPromotedFilterBinder = MakeUnique<UE::CurveEditor::FPromotedFilterCommandBinder>(
		GetCurveEditor()->GetToolbarPromotedFilters().ToSharedRef(), CommandList.ToSharedRef(), CurveEditor.ToSharedRef()
		);

	BindCommands();

	ColumnFillCoefficients[0] = 0.3f;
	ColumnFillCoefficients[1] = 0.7f;

	if (CurveEditor->GetSettings())
	{
		ColumnFillCoefficients[0] = CurveEditor->GetSettings()->GetTreeViewWidth();
		ColumnFillCoefficients[1] = 1.f - CurveEditor->GetSettings()->GetTreeViewWidth();
	}

	TAttribute<float> FillCoefficient_0, FillCoefficient_1;
	{
		FillCoefficient_0.Bind(TAttribute<float>::FGetter::CreateSP(this, &SCurveEditorPanel::GetColumnFillCoefficient, 0));
		FillCoefficient_1.Bind(TAttribute<float>::FGetter::CreateSP(this, &SCurveEditorPanel::GetColumnFillCoefficient, 1));
	}

	// Create some Widgets
	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>("SequencerWidgets");
	TSharedPtr<SWidget> TopTimeSlider = SNullWidget::NullWidget;
	if (InArgs._ExternalTimeSliderController)
	{
		TopTimeSlider = SequencerWidgets.CreateTimeSlider(InArgs._ExternalTimeSliderController.ToSharedRef(), false /*bMirrorLabels*/);
	}

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar).Thickness(FVector2D(5.f, 5.f));

	TSharedRef<SWidget> MainContent = 
		SNew(SOverlay)

		// The main editing area
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				// Top Time Slider
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FLinearColor(.50f, .50f, .50f, 1.0f))
				.Padding(0)
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					TopTimeSlider.ToSharedRef()
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SAssignNew(ScrollBox, SScrollBox)
					.ExternalScrollbar(ScrollBar)

					+ SScrollBox::Slot()
					[
						// Main Curve View Area. The contents of this are dynamically filled based
						// on your current views.

						SAssignNew(CurveViewsContainer, SCurveEditorViewContainer, InCurveEditor)
						.ExternalTimeSliderController(InArgs._ExternalTimeSliderController)
						.MinimumPanelHeight(InArgs._MinimumViewPanelHeight)
					]
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				[
					ScrollBar
				]

				+ SOverlay::Slot()
				.Padding(10.0f, 10.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SAssignNew(ToolPropertiesPanel, SCurveEditorToolProperties, InCurveEditor, FCurveEditorToolID::Unset())
				]
			]
		]

		// An overlay for the main area which lets us put system-wide overlays
		+ SOverlay::Slot()
		[
			SNew(SOverlay)
			.Visibility(this, &SCurveEditorPanel::ShouldInstructionOverlayBeVisible)

			// Darker background
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FLinearColor::Black.CopyWithNewOpacity(0.35f))
			]

			// Text
			+ SOverlay::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CurveEditorTutorialOverlay", "Select a curve on the left to begin editing."))
					.Font(FCoreStyle::Get().GetFontStyle("FontAwesome.13"))
					.ColorAndOpacity(FLinearColor::White)
				]
			]
		];
	
	SharedCurveInfo = MakePimpl<UE::CurveEditor::FSharedCurveInfoModel>(InCurveEditor, CurveViewsContainer.ToSharedRef());
	
	if (InArgs._TreeContent.Widget != SNullWidget::NullWidget)
	{
		ChildSlot
		[
			SAssignNew(TreeViewSplitter, SSplitter)
			.Orientation(Orient_Horizontal)
			.Style(FAppStyle::Get(), "SplitterDark")
			.PhysicalSplitterHandleSize(3.0f)
			.OnSplitterFinishedResizing(this, &SCurveEditorPanel::OnSplitterFinishedResizing)

			+ SSplitter::Slot()
			.Value(FillCoefficient_0)
			.MinSize(1.f)
			.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SCurveEditorPanel::OnColumnFillCoefficientChanged, 0))
			[
				InArgs._TreeContent.Widget
			]

			+ SSplitter::Slot()
			.MinSize(1.f)
			.Value(FillCoefficient_1)
			.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SCurveEditorPanel::OnColumnFillCoefficientChanged, 1))
			[
				MainContent
			]
		];
	}
	else
	{
		ChildSlot
		[
			MainContent
		];
	}



	KeyDetailsView = SNew(SCurveKeyDetailPanel, CurveEditor.ToSharedRef())
		.IsEnabled(this, &SCurveEditorPanel::IsInlineEditPanelEditable);

	UpdateEditBox();

	UpdateAxisSnapping();

	// Initializes our Curve Views on the next Tick
	SetViewMode(ECurveEditorViewID::Absolute);
}

TArrayView<const TSharedPtr<SCurveEditorView>> SCurveEditorPanel::GetViews() const
{
	return CurveViewsContainer->GetViews();
}

void SCurveEditorPanel::ScrollBy(float Amount)
{
	ScrollBox->SetScrollOffset(ScrollBox->GetScrollOffset() + Amount);
}

void SCurveEditorPanel::BindCommands()
{
	// Interpolation and tangents
	{
		FExecuteAction SetConstant   = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Constant).SetTangentMode(RCTM_Auto), LOCTEXT("SetInterpConstant", "Set Interp Constant"));
		FExecuteAction SetLinear     = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Linear).SetTangentMode(RCTM_Auto),   LOCTEXT("SetInterpLinear",   "Set Interp Linear"));
		FExecuteAction SetCubicAuto  = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_Auto),    LOCTEXT("SetInterpCubic",    "Set Interp Auto"));
		FExecuteAction SetCubicSmartAuto = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_SmartAuto),	LOCTEXT("SetInterpSmartAuto", "Set Interp Smart Auto"));
		FExecuteAction SetCubicUser  = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_User),    LOCTEXT("SetInterpUser",     "Set Interp User"));
		FExecuteAction SetCubicBreak = FExecuteAction::CreateSP(this, &SCurveEditorPanel::HandleSetKeyAttributesCommand, FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_Break),   LOCTEXT("SetInterpBreak",    "Set Interp Break"));

		FExecuteAction    ToggleWeighted    = FExecuteAction::CreateSP(this, &SCurveEditorPanel::ToggleWeightedTangents);
		FCanExecuteAction CanToggleWeighted = FCanExecuteAction::CreateSP(this, &SCurveEditorPanel::CanToggleWeightedTangents);

		FIsActionChecked IsConstantCommon   = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonInterpolationMode, RCIM_Constant);
		FIsActionChecked IsLinearCommon     = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonInterpolationMode, RCIM_Linear);
		FIsActionChecked IsCubicAutoCommon  = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonTangentMode, RCIM_Cubic, RCTM_Auto);
		FIsActionChecked IsCubicSmartAutoCommon = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonTangentMode, RCIM_Cubic, RCTM_SmartAuto);
		FIsActionChecked IsCubicUserCommon  = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonTangentMode, RCIM_Cubic, RCTM_User);
		FIsActionChecked IsCubicBreakCommon = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonTangentMode, RCIM_Cubic, RCTM_Break);
		FIsActionChecked IsCubicWeightCommon = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonTangentWeightMode, RCIM_Cubic, RCTWM_WeightedBoth);

		FCanExecuteAction CanSetKeyTangent = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CanSetKeyInterpolation);

		int32 SupportedTangentTypes = CurveEditor->GetSupportedTangentTypes();
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicSmartAuto)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationCubicSmartAuto, SetCubicSmartAuto, CanSetKeyTangent, IsCubicSmartAutoCommon);
		};
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicAuto)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationCubicAuto, SetCubicAuto, CanSetKeyTangent, IsCubicAutoCommon);
		};
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicUser)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationCubicUser, SetCubicUser, CanSetKeyTangent, IsCubicUserCommon);
		}
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicBreak)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationCubicBreak, SetCubicBreak, CanSetKeyTangent, IsCubicBreakCommon);
		}
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationLinear)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationLinear, SetLinear, CanSetKeyTangent, IsLinearCommon);
		}
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationConstant)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationConstant, SetConstant, CanSetKeyTangent, IsConstantCommon);
		}
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicWeighted)
		{
			CommandList->MapAction(FCurveEditorCommands::Get().InterpolationToggleWeighted, ToggleWeighted, CanToggleWeighted, IsCubicWeightCommon);
		}
	}

	const FCanExecuteAction CanExtrapolate = FCanExecuteAction::CreateLambda([this](){ return CanSetKeyInterpolation(); });
	// Pre Extrapolation Modes
	{
		FExecuteAction SetCycle           = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPreExtrapolation(RCCE_Cycle),           LOCTEXT("SetPreExtrapCycle",           "Set Pre Extrapolation (Cycle)"));
		FExecuteAction SetCycleWithOffset = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPreExtrapolation(RCCE_CycleWithOffset), LOCTEXT("SetPreExtrapCycleWithOffset", "Set Pre Extrapolation (Cycle With Offset)"));
		FExecuteAction SetOscillate       = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPreExtrapolation(RCCE_Oscillate),       LOCTEXT("SetPreExtrapOscillate",       "Set Pre Extrapolation (Oscillate)"));
		FExecuteAction SetLinear          = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPreExtrapolation(RCCE_Linear),          LOCTEXT("SetPreExtrapLinear",          "Set Pre Extrapolation (Linear)"));
		FExecuteAction SetConstant        = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPreExtrapolation(RCCE_Constant),        LOCTEXT("SetPreExtrapConstant",        "Set Pre Extrapolation (Constant)"));

		FIsActionChecked IsCycleCommon           = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPreExtrapolationMode, RCCE_Cycle);
		FIsActionChecked IsCycleWithOffsetCommon = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPreExtrapolationMode, RCCE_CycleWithOffset);
		FIsActionChecked IsOscillateCommon       = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPreExtrapolationMode, RCCE_Oscillate);
		FIsActionChecked IsLinearCommon          = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPreExtrapolationMode, RCCE_Linear);
		FIsActionChecked IsConstantCommon        = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPreExtrapolationMode, RCCE_Constant);

		CommandList->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapCycle, SetCycle, CanExtrapolate, IsCycleCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset, SetCycleWithOffset, CanExtrapolate, IsCycleWithOffsetCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate, SetOscillate, CanExtrapolate, IsOscillateCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapLinear, SetLinear, CanExtrapolate, IsLinearCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapConstant, SetConstant, CanExtrapolate, IsConstantCommon);
	}

	// Post Extrapolation Modes
	{
		FExecuteAction SetCycle           = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPostExtrapolation(RCCE_Cycle),           LOCTEXT("SetPostExtrapCycle",           "Set Post Extrapolation (Cycle)"));
		FExecuteAction SetCycleWithOffset = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPostExtrapolation(RCCE_CycleWithOffset), LOCTEXT("SetPostExtrapCycleWithOffset", "Set Post Extrapolation (Cycle With Offset)"));
		FExecuteAction SetOscillate       = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPostExtrapolation(RCCE_Oscillate),       LOCTEXT("SetPostExtrapOscillate",       "Set Post Extrapolation (Oscillate)"));
		FExecuteAction SetLinear          = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPostExtrapolation(RCCE_Linear),          LOCTEXT("SetPostExtrapLinear",          "Set Post Extrapolation (Linear)"));
		FExecuteAction SetConstant        = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetCurveAttributes, FCurveAttributes().SetPostExtrapolation(RCCE_Constant),        LOCTEXT("SetPostExtrapConstant",        "Set Post Extrapolation (Constant)"));

		FIsActionChecked IsCycleCommon           = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPostExtrapolationMode, RCCE_Cycle);
		FIsActionChecked IsCycleWithOffsetCommon = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPostExtrapolationMode, RCCE_CycleWithOffset);
		FIsActionChecked IsOscillateCommon       = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPostExtrapolationMode, RCCE_Oscillate);
		FIsActionChecked IsLinearCommon          = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPostExtrapolationMode, RCCE_Linear);
		FIsActionChecked IsConstantCommon        = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareCommonPostExtrapolationMode, RCCE_Constant);

		CommandList->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapCycle, SetCycle, CanExtrapolate, IsCycleCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset, SetCycleWithOffset, CanExtrapolate, IsCycleWithOffsetCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate, SetOscillate, CanExtrapolate, IsOscillateCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapLinear, SetLinear, CanExtrapolate, IsLinearCommon);
		CommandList->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapConstant, SetConstant, CanExtrapolate, IsConstantCommon);
	}

	// Absolute, Stacked and Normalized views.
	{
		FExecuteAction SetViewAbsolute = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetViewMode, ECurveEditorViewID::Absolute);
		FIsActionChecked IsViewModeAbsolute = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareViewMode, ECurveEditorViewID::Absolute);

		FExecuteAction SetViewStacked = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetViewMode, ECurveEditorViewID::Stacked);
		FIsActionChecked IsViewModeStacked = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareViewMode, ECurveEditorViewID::Stacked);

		FExecuteAction SetViewNormalized = FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetViewMode, ECurveEditorViewID::Normalized);
		FIsActionChecked IsViewModeNormalized = FIsActionChecked::CreateSP(this, &SCurveEditorPanel::CompareViewMode, ECurveEditorViewID::Normalized);

		CommandList->MapAction(FCurveEditorCommands::Get().SetViewModeAbsolute, SetViewAbsolute, FCanExecuteAction(), IsViewModeAbsolute);
		CommandList->MapAction(FCurveEditorCommands::Get().SetViewModeStacked, SetViewStacked, FCanExecuteAction(), IsViewModeStacked);
		CommandList->MapAction(FCurveEditorCommands::Get().SetViewModeNormalized, SetViewNormalized, FCanExecuteAction(), IsViewModeNormalized);
	}

	{
		// Deselect Current Keys
		TSharedPtr<FCurveEditor> LocalCurveEditor = CurveEditor;
		FExecuteAction DeselectAllAction = FExecuteAction::CreateLambda([LocalCurveEditor] { if (LocalCurveEditor.IsValid())
			{
				const UE::CurveEditor::FScopedSelectionChange Transaction(LocalCurveEditor, LOCTEXT("DeselectAllKeys", "Deselect all keys"));
				LocalCurveEditor->GetSelection().Clear();
			} 
		});
		CommandList->MapAction(FCurveEditorCommands::Get().DeselectAllKeys, DeselectAllAction);
	}

	// Presets for Bake and Reduce 
	CommandList->MapAction(FCurveEditorCommands::Get().BakeCurve, FExecuteAction::CreateSP(this, &SCurveEditorPanel::ShowCurveFilterUI, TSubclassOf<UCurveEditorFilterBase>(UCurveEditorBakeFilter::StaticClass())));
	CommandList->MapAction(FCurveEditorCommands::Get().ReduceCurve, FExecuteAction::CreateSP(this, &SCurveEditorPanel::ShowCurveFilterUI, TSubclassOf<UCurveEditorFilterBase>(UCurveEditorReduceFilter::StaticClass())));

	// User Implementable Filter  just defaults to Bake since we know it exists...
	CommandList->MapAction(FCurveEditorCommands::Get().OpenUserImplementableFilterWindow, FExecuteAction::CreateSP(this, &SCurveEditorPanel::ShowCurveFilterUI, TSubclassOf<UCurveEditorFilterBase>(UCurveEditorBakeFilter::StaticClass())));
	
	// Axis Snapping
	{
		FUIAction SetSnappingNoneAction(FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetAxisSnapping, ECurveEditorSnapAxis::CESA_None));
		FUIAction SetSnappingHorizontalAction(FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetAxisSnapping, ECurveEditorSnapAxis::CESA_X));
		FUIAction SetSnappingVerticalAction(FExecuteAction::CreateSP(this, &SCurveEditorPanel::SetAxisSnapping, ECurveEditorSnapAxis::CESA_Y));

		CommandList->MapAction(FCurveEditorCommands::Get().SetAxisSnappingNone, SetSnappingNoneAction);
		CommandList->MapAction(FCurveEditorCommands::Get().SetAxisSnappingHorizontal, SetSnappingHorizontalAction);
		CommandList->MapAction(FCurveEditorCommands::Get().SetAxisSnappingVertical, SetSnappingVerticalAction);
	}

}

void SCurveEditorPanel::SetViewMode(const ECurveEditorViewID NewViewMode)
{
	DefaultViewID = NewViewMode;
	bNeedsRefresh = true;
}

bool SCurveEditorPanel::CompareViewMode(const ECurveEditorViewID InViewMode) const
{
	return DefaultViewID == InViewMode;
}

void SCurveEditorPanel::SetAxisSnapping(ECurveEditorSnapAxis InAxis)
{
	CurveEditor->GetSettings()->SetSnapAxis(InAxis);
	UpdateAxisSnapping();
}

void SCurveEditorPanel::UpdateAxisSnapping()
{
	FCurveEditorAxisSnap Snap = CurveEditor->GetAxisSnap();
	Snap.RestrictedAxisList = CurveEditor->GetSettings()->GetSnapAxis();
	CurveEditor->SetAxisSnap(Snap);
}

void SCurveEditorPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNeedsRefresh || CachedActiveCurvesSerialNumber != CurveEditor->GetActiveCurvesSerialNumber())
	{
		RebuildCurveViews();

		if (CurveEditor->ShouldAutoFrame())
		{
			CurveEditor->ZoomToFitCurves(CurveEditor->GetEditedCurves().Array());
		}
		bNeedsRefresh = false;
		CachedActiveCurvesSerialNumber = CurveEditor->GetActiveCurvesSerialNumber();
	}

	UpdateEditBox();
	UpdateTime();

	CachedSelectionSerialNumber = CurveEditor->Selection.GetSerialNumber();
}

void SCurveEditorPanel::ResetMinMaxes()
{
	//only reset the min/max if we have views since we will then get these values from them
	//otherwise if we didn't we would end up with everything back to 0,1 again.
	if (CurveViews.IsEmpty() == false)
	{
		LastOutputMin = DBL_MAX;
		LastOutputMax = DBL_MIN;
	}
}

void SCurveEditorPanel::RemoveCurveFromViews(FCurveModelID InCurveID)
{
	for (auto It = CurveViews.CreateKeyIterator(InCurveID); It; ++It)
	{
		SCurveEditorView* View = &It.Value().Get();
		//cache these so we can re-use it on reconstruction
		if (View)
		{
			if (View->GetOutputMin() < LastOutputMin)
			{
				LastOutputMin = View->GetOutputMin();
			}
			if (View->GetOutputMax() > LastOutputMax)
			{
				LastOutputMax = View->GetOutputMax();
			}
		}
		FCurveEditorPanelViewTracker::RemoveCurveFromView(View, InCurveID);
		It.RemoveCurrent();
	}
}

void SCurveEditorPanel::PostUndo()
{
	EditObjects->CurveIDToKeyProxies.Empty();

	// Force the edit box to update (ie. the value of the keys might have changed)
	CachedSelectionSerialNumber = 0;
	UpdateEditBox();

	// Reset the selection serial number so that time doesn't change since selection didn't really change on undo
	CachedSelectionSerialNumber = CurveEditor->Selection.GetSerialNumber();
}

void SCurveEditorPanel::AddView(TSharedRef<SCurveEditorView> ViewToAdd)
{
	ExternalViews.Add(ViewToAdd);
	bNeedsRefresh = true;
}

void SCurveEditorPanel::RemoveView(TSharedRef<SCurveEditorView> ViewToRemove)
{
	ExternalViews.Remove(ViewToRemove);
	bNeedsRefresh = true;
}

TSharedPtr<SCurveEditorView> SCurveEditorPanel::CreateViewOfType(FCurveModelID CurveModelID, ECurveEditorViewID ViewTypeID, bool bPinned)
{
	for (auto It = FreeViewsByType.CreateKeyIterator(ViewTypeID); It; ++It)
	{
		TSharedRef<SCurveEditorView> View = It.Value();

		if (!GCurveEditorPinnedViews || View->bPinned == bPinned)
		{
			FCurveEditorPanelViewTracker::AddCurveToView(&View.Get(), CurveModelID);
			CurveViews.Add(CurveModelID, View);

			if (!View->HasCapacity())
			{
				It.RemoveCurrent();
			}
			if (LastOutputMin != DBL_MAX && LastOutputMax != DBL_MIN)
			{
				View->SetOutputBounds(LastOutputMin, LastOutputMax);
			}

			return View;
		}
	}

	TSharedPtr<SCurveEditorView> View = FCurveEditorViewRegistry::Get().ConstructView(ViewTypeID, CurveEditor);
	if (View)
	{
		if (GCurveEditorPinnedViews && bPinned)
		{
			// Pinned views are always a fixed height
			View->bPinned = true;
			View->MaximumCapacity = View->MaximumCapacity == 0 ? GCurveEditorMaxCurvesPerPinnedView : FMath::Min(View->MaximumCapacity, GCurveEditorMaxCurvesPerPinnedView);
			if (!View->FixedHeight.IsSet())
			{
				View->FixedHeight = 100.f;
			}
		}
		View->ViewTypeID = ViewTypeID;
		FCurveEditorPanelViewTracker::AddCurveToView(View.Get(), CurveModelID);
		CurveViews.Add(CurveModelID, View.ToSharedRef());

		if (View->HasCapacity())
		{
			FreeViewsByType.Add(ViewTypeID, View.ToSharedRef());
		}
	}

	return View;
}

void SCurveEditorPanel::RebuildCurveViews()
{
	TSet<TSharedRef<SCurveEditorView>> Views = ExternalViews;

	for (const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : CurveEditor->GetCurves())
	{
		FCurveModel* Curve = CurvePair.Value.Get();
		FCurveModelID CurveID = CurvePair.Key;

		const bool bIsPinned = CurveEditor->IsCurvePinned(CurveID);

		bool bNeedsView = true;

		for (auto ViewIt = CurveViews.CreateKeyIterator(CurveID); ViewIt; ++ViewIt)
		{
			TSharedRef<SCurveEditorView> View = ViewIt.Value();
			// @todo: pinning - This code causes curves that have changed their pinned state to be re-added to a correctly (un)pinned views
			if (GCurveEditorPinnedViews && View->bPinned != bIsPinned)
			{
				// No longer the same pinned status as the view it's in - remove it so that it can get added to the correct view (or removed entirely)
				FCurveEditorPanelViewTracker::RemoveCurveFromView(&View.Get(), CurveID);
				ViewIt.RemoveCurrent();
			}
			else if (View->ViewTypeID == DefaultViewID || !EnumHasAnyFlags(View->ViewTypeID, ECurveEditorViewID::ANY_BUILT_IN))
			{
				// Keep this view if it is the default view or any other custom view
				Views.Add(View);
				bNeedsView = false;
			}
			else
			{
				// Built in view which is no longer the selected mode - remove it
				FCurveEditorPanelViewTracker::RemoveCurveFromView(&View.Get(), CurveID);
				ViewIt.RemoveCurrent();
			}
		}

		if (bNeedsView)
		{
			ECurveEditorViewID SupportedViews = Curve->GetSupportedViews();

			// Add to the default view if supported, else use the first supported view we can find.
			// @todo: this may require extra work if curves are ever to support multiple views but it's fine for now
			if (EnumHasAnyFlags(SupportedViews, DefaultViewID))
			{
				TSharedPtr<SCurveEditorView> NewView = CreateViewOfType(CurveID, DefaultViewID, bIsPinned);
				if (NewView)
				{
					Views.Add(NewView.ToSharedRef());
				}
				continue;
			}

			ECurveEditorViewID CustomView = ECurveEditorViewID::CUSTOM_START;
			while (CustomView >= ECurveEditorViewID::CUSTOM_START)
			{
				if (EnumHasAnyFlags(SupportedViews, ECurveEditorViewID(CustomView)))
				{
					TSharedPtr<SCurveEditorView> NewView = CreateViewOfType(CurveID, CustomView, bIsPinned);
					if (NewView)
					{
						Views.Add(NewView.ToSharedRef());
					}
				}
				CustomView = ECurveEditorViewID((__underlying_type(ECurveEditorViewID))(CustomView) << 1);
			}
		}
	}

	// Remove any empty views
	for (auto It = FreeViewsByType.CreateIterator(); It; ++It)
	{
		if (!It.Value()->bAllowEmpty && It.Value()->NumCurves() == 0)
		{
			It.RemoveCurrent();
		}
	}

	// Sort by pinned, then capacity
	TArray<TSharedRef<SCurveEditorView>> SortedViews = Views.Array();
	Algo::Sort(SortedViews, [](const TSharedPtr<SCurveEditorView>& A, const TSharedPtr<SCurveEditorView>& B)
		{
			if (A->SortBias == B->SortBias)
			{
				if (A->bPinned == B->bPinned)
				{
					return A->RelativeOrder < B->RelativeOrder;
				}

				return A->bPinned != 0;
			}

			return A->SortBias > B->SortBias;
		}
	);

	CurveViewsContainer->Clear();
	for (TSharedPtr<SCurveEditorView> View : SortedViews)
	{
		CurveViewsContainer->AddView(View.ToSharedRef());
	}

	OnPostRebuildCurveViewsDelegate.Broadcast();
}

void SCurveEditorPanel::OnCurveEditorToolChanged(FCurveEditorToolID InToolId)
{
	ToolPropertiesPanel->OnToolChanged(InToolId);
}

void SCurveEditorPanel::UpdateTime()
{
	const FCurveEditorSelection& Selection = CurveEditor->Selection;
	if (CachedSelectionSerialNumber == Selection.GetSerialNumber())
	{
		return;
	}

	if (CurveEditor->GetSettings()->GetSnapTimeToSelection())
	{
		CurveEditor->SnapToSelectedKey();
	}
}

void SCurveEditorPanel::UpdateEditBox()
{
	const FCurveEditorSelection& Selection = CurveEditor->Selection;
	for (auto& OuterPair : EditObjects->CurveIDToKeyProxies)
	{
		const FKeyHandleSet* SelectedKeys = Selection.FindForCurve(OuterPair.Key);
		if(SelectedKeys)
		{
			for (auto& InnerPair : OuterPair.Value)
			{
				if (ICurveEditorKeyProxy* Proxy = Cast<ICurveEditorKeyProxy>(InnerPair.Value))
				{
					Proxy->UpdateValuesFromRawData();
				}
			}
		}
	}

	if (CachedSelectionSerialNumber == Selection.GetSerialNumber())
	{
		return;
	}

	TArray<FKeyHandle> KeyHandleScratch;
	TArray<UObject*>   NewProxiesScratch;

	TArray<UObject*> AllEditObjects;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection.GetAll())
	{
		FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key);
		if (!Curve)
		{
			continue;
		}

		KeyHandleScratch.Reset();
		NewProxiesScratch.Reset();

		auto& KeyHandleToEditObject = EditObjects->CurveIDToKeyProxies.FindOrAdd(Pair.Key);
		for (FKeyHandle Handle : Pair.Value.AsArray())
		{
			if (UObject* Existing = KeyHandleToEditObject.FindRef(Handle))
			{
				AllEditObjects.Add(Existing);
			}
			else
			{
				KeyHandleScratch.Add(Handle);
			}
		}

		if (KeyHandleScratch.Num() > 0)
		{
			NewProxiesScratch.SetNum(KeyHandleScratch.Num());
			Curve->CreateKeyProxies(CurveEditor, Pair.Key, KeyHandleScratch, NewProxiesScratch);

			for (int32 Index = 0; Index < KeyHandleScratch.Num(); ++Index)
			{
				if (UObject* NewObject = NewProxiesScratch[Index])
				{
					KeyHandleToEditObject.Add(KeyHandleScratch[Index], NewObject);
					AllEditObjects.Add(NewObject);

					// Update the proxy immediately after adding it so that it doesn't have the wrong values for 1 tick.
					if (ICurveEditorKeyProxy* Proxy = Cast<ICurveEditorKeyProxy>(NewObject))
					{
						Proxy->UpdateValuesFromRawData();
					}
				}
			}
		}
	}

	KeyDetailsView->GetPropertyRowGenerator()->SetObjects(AllEditObjects);
}

TSharedRef<SWidget> SCurveEditorPanel::MakeTangentModeMenu()
{
	FMenuBuilder MenuBuilder(true, GetCommands());
	
	const int32 SupportedTangentTypes = CurveEditor->GetSupportedTangentTypes();
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicSmartAuto)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicSmartAuto);
	};
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicAuto)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicAuto);
	};
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicUser)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicUser);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicBreak)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicBreak);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationLinear)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationLinear);
	}
	if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationConstant)
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationConstant);
	}

	return MenuBuilder.MakeWidget();
}

FText SCurveEditorPanel::GetTangentModeLabel() const
{
	switch (DetermineTangentMode())
	{
	case ETangentModeComboState::Constant: return LOCTEXT("TangentMode.Constant.Label", "Constant");
	case ETangentModeComboState::Linear: return LOCTEXT("TangentMode.Linear.Label", "Linear");
	case ETangentModeComboState::CubicAuto: return LOCTEXT("TangentMode.CubicAuto.Label", "Cubic (auto)");
	case ETangentModeComboState::CubicSmartAuto: return LOCTEXT("TangentMode.CubicSmartAuto.Label", "Cubic (smart)");
	case ETangentModeComboState::CubicUser: return LOCTEXT("TangentMode.CubicUser.Label", "Cubic (flat)");
	case ETangentModeComboState::CubicBreak: return LOCTEXT("TangentMode.CubicBreak.Label", "Cubic (broken)");
	case ETangentModeComboState::Mixed: return LOCTEXT("TangentMode.Mixed.Label", "Mixed");
	case ETangentModeComboState::NoKeys: return LOCTEXT("TangentMode.NoKeys.Label", "No keys");
	default: checkNoEntry(); return FText::GetEmpty();
	}
}

FText SCurveEditorPanel::GetTangentModeTooltip() const
{
	switch (DetermineTangentMode())
	{
	case ETangentModeComboState::Constant: return FCurveEditorCommands::Get().InterpolationConstant->GetDescription();
	case ETangentModeComboState::Linear: return FCurveEditorCommands::Get().InterpolationLinear->GetDescription();
	case ETangentModeComboState::CubicAuto: return FCurveEditorCommands::Get().InterpolationCubicAuto->GetDescription();
	case ETangentModeComboState::CubicSmartAuto: return FCurveEditorCommands::Get().InterpolationCubicSmartAuto->GetDescription();
	case ETangentModeComboState::CubicUser: return FCurveEditorCommands::Get().InterpolationCubicUser->GetDescription();
	case ETangentModeComboState::CubicBreak: return FCurveEditorCommands::Get().InterpolationCubicBreak->GetDescription();
	case ETangentModeComboState::Mixed: return LOCTEXT("TangentMode.Mixed.Tooltip", "The selected keys have different tangent modes.");
	case ETangentModeComboState::NoKeys: return LOCTEXT("TangentMode.NoKeys.Tooltip", "Add some keys to the curve to set their tangent modes.");
	default: checkNoEntry(); return FText::GetEmpty();
	}
}

FSlateIcon SCurveEditorPanel::GetTangentModeIcon() const
{
	switch (DetermineTangentMode())
	{
	case ETangentModeComboState::Constant: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationConstant");
	case ETangentModeComboState::Linear: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationLinear");
	case ETangentModeComboState::CubicAuto: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationCubicAuto");
	case ETangentModeComboState::CubicSmartAuto: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationCubicSmartAuto");
	case ETangentModeComboState::CubicUser: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationCubicUser");
	case ETangentModeComboState::CubicBreak: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationCubicBreak");
	case ETangentModeComboState::Mixed: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationMixed");
	case ETangentModeComboState::NoKeys: return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.InterpolationMixed");
	default: checkNoEntry(); return FSlateIcon();
	}
}

SCurveEditorPanel::ETangentModeComboState SCurveEditorPanel::DetermineTangentMode() const
{
	if (!CanSetKeyInterpolation())
	{
		return ETangentModeComboState::NoKeys;
	}
	
	const bool bIsConstantCommon		= CompareCommonInterpolationMode(RCIM_Constant);
	const bool bIsLinearCommon			= CompareCommonInterpolationMode(RCIM_Linear);
	const bool bIsCubicAutoCommon		= CompareCommonTangentMode(RCIM_Cubic, RCTM_Auto);
	const bool bIsCubicSmartAutoCommon	= CompareCommonTangentMode(RCIM_Cubic, RCTM_SmartAuto);
	const bool bIsCubicUserCommon		= CompareCommonTangentMode(RCIM_Cubic, RCTM_User);
	const bool bIsCubicBreakCommon		= CompareCommonTangentMode(RCIM_Cubic, RCTM_Break);

	if (bIsConstantCommon)
	{
		return ETangentModeComboState::Constant;
	}
	if (bIsLinearCommon)
	{
		return ETangentModeComboState::Linear;
	}
	if (bIsCubicAutoCommon)
	{
		return ETangentModeComboState::CubicAuto;
	}
	if (bIsCubicSmartAutoCommon)
	{
		return ETangentModeComboState::CubicSmartAuto;
	}
	if (bIsCubicUserCommon)
	{
		return ETangentModeComboState::CubicUser;
	}
	if (bIsCubicBreakCommon)
	{
		return ETangentModeComboState::CubicBreak;
	}
	
	return ETangentModeComboState::Mixed;
}

EVisibility SCurveEditorPanel::GetSplitterVisibility() const
{
	return EVisibility::Visible;
}

void SCurveEditorPanel::HandleSetKeyAttributesCommand(FKeyAttributes InKeyAttributes, FText InDescription)
{
	if (CurveEditor->Selection.IsEmpty())
	{
		SetKeyAttributesOnAllCurves(InKeyAttributes, InDescription);
	}
	else
	{
		SetKeyAttributesOnSelection(InKeyAttributes, InDescription);
	}
}

void SCurveEditorPanel::SetKeyAttributesOnSelection(const FKeyAttributes& InKeyAttributes, const FText& InDescription) const
{
	using namespace UE::CurveEditor;
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(GetCurveEditor(), ECurveChangeFlags::KeyAttributes).TrackSelectedCurves(), InDescription
	);
	
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->Selection.GetAll())
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
		{
			Curve->SetKeyAttributes(Pair.Value.AsArray(), InKeyAttributes);
		}
	}
}

void SCurveEditorPanel::SetKeyAttributesOnAllCurves(const FKeyAttributes& InKeyAttributes, const FText& InDescription) const
{
	using namespace UE::CurveEditor;
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(GetCurveEditor(), ECurveChangeFlags::KeyAttributes).TrackAllCurves(), InDescription
	);
	
	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : CurveEditor->GetCurves())
	{
		if (FCurveModel* Curve = Pair.Value.Get())
		{
			const TArray<FKeyHandle> KeyHandles = Curve->GetAllKeys();
			Curve->SetKeyAttributes(KeyHandles, InKeyAttributes);
		}
	}
}

void SCurveEditorPanel::SetCurveAttributes(FCurveAttributes CurveAttributes, FText Description)
{
	using namespace UE::CurveEditor;
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(CurveEditor, ECurveChangeFlags::CurveAttributes).TrackSelectedCurves(),
		Description
		);

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->Selection.GetAll())
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
		{
			Curve->SetCurveAttributes(CurveAttributes);
		}
	}
}

bool SCurveEditorPanel::CompareCommonInterpolationMode(ERichCurveInterpMode InterpMode) const
{
	const FKeyAttributes& CachedCommonKeyAttributes = SharedCurveInfo->GetCachedCommonKeyAttributes();
	return CachedCommonKeyAttributes.HasInterpMode() && CachedCommonKeyAttributes.GetInterpMode() == InterpMode;
}

bool SCurveEditorPanel::CompareCommonTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode) const
{
	const FKeyAttributes& CachedCommonKeyAttributes = SharedCurveInfo->GetCachedCommonKeyAttributes();
	return CompareCommonInterpolationMode(InterpMode) && CachedCommonKeyAttributes.HasTangentMode() && CachedCommonKeyAttributes.GetTangentMode() == TangentMode;
}

bool SCurveEditorPanel::CompareCommonTangentWeightMode(ERichCurveInterpMode InterpMode, ERichCurveTangentWeightMode TangentWeightMode) const
{
	const FKeyAttributes& CachedCommonKeyAttributes = SharedCurveInfo->GetCachedCommonKeyAttributes();
	return CompareCommonInterpolationMode(InterpMode) && CachedCommonKeyAttributes.HasTangentWeightMode() && CachedCommonKeyAttributes.GetTangentWeightMode() == TangentWeightMode;
}

bool SCurveEditorPanel::CompareCommonPreExtrapolationMode(ERichCurveExtrapolation PreExtrapolationMode) const
{
	const FCurveAttributes& CachedCommonCurveAttributes = SharedCurveInfo->GetCachedCommonCurveAttributes();
	return CachedCommonCurveAttributes.HasPreExtrapolation()  && CachedCommonCurveAttributes.GetPreExtrapolation() == PreExtrapolationMode;
}

bool SCurveEditorPanel::CompareCommonPostExtrapolationMode(ERichCurveExtrapolation PostExtrapolationMode) const
{
	const FCurveAttributes& CachedCommonCurveAttributes = SharedCurveInfo->GetCachedCommonCurveAttributes();
	return CachedCommonCurveAttributes.HasPostExtrapolation() && CachedCommonCurveAttributes.GetPostExtrapolation() == PostExtrapolationMode;
}

void SCurveEditorPanel::ToggleWeightedTangents()
{
	using namespace UE::CurveEditor;
	const TMap<FCurveModelID, FKeyHandleSet>& Selection = CurveEditor->GetSelection().GetAll();
	const FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(CurveEditor, Selection, ECurveChangeFlags::KeyAttributes),
		LOCTEXT("ToggleWeightedTangents_Transaction", "Toggle Weighted Tangents")
		);

	// Disable weights unless we find something that doesn't have weights, then add them
	FKeyAttributes KeyAttributesToAssign = FKeyAttributes().SetTangentWeightMode(RCTWM_WeightedNone);
	// Gather current key attributes
	TMap<FCurveModelID, TArray<FKeyAttributes>> KeyAttributesPerCurve;
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : Selection)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key);
		if (Curve)
		{
			TArray<FKeyAttributes>& KeyAttributes = KeyAttributesPerCurve.Add(Pair.Key);
			KeyAttributes.SetNum(Pair.Value.Num());
			Curve->GetKeyAttributes(Pair.Value.AsArray(), KeyAttributes);

			// Check all the key attributes if they support tangent weights, but don't have any. If we find any such keys, we'll enable weights on all.
			if (KeyAttributesToAssign.GetTangentWeightMode() == RCTWM_WeightedNone)
			{
				for (const FKeyAttributes& Attributes : KeyAttributes)
				{
					if (Attributes.HasTangentWeightMode() && !(Attributes.HasArriveTangentWeight() || Attributes.HasLeaveTangentWeight()))
					{
						KeyAttributesToAssign.SetTangentWeightMode(RCTWM_WeightedBoth);
						break;
					}
				}
			}
		}
	}

	// Assign the new key attributes to all the selected curves
	for (TTuple<FCurveModelID, TArray<FKeyAttributes>>& Pair : KeyAttributesPerCurve)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key);
		if (Curve)
		{
			for (FKeyAttributes& Attributes : Pair.Value)
			{
				Attributes = KeyAttributesToAssign;
			}

			TArrayView<const FKeyHandle> KeyHandles = Selection.FindChecked(Pair.Key).AsArray();
			Curve->SetKeyAttributes(KeyHandles, Pair.Value);
		}
	}
}

bool SCurveEditorPanel::CanToggleWeightedTangents() const
{
	return SharedCurveInfo->DoesSelectionSupportWeightedTangents() && CanSetKeyInterpolation();
}

bool SCurveEditorPanel::CanSetKeyInterpolation() const
{
	// Basically, return false if there are no keys... if user has selected something, we can set.
	// If the user has nothing selected, we can set all the keys in view. But if there are no keys, we want to disable the combo button.
	return Algo::AnyOf(CurveEditor->GetCurves(), [](const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair)
	{
		return Pair.Value && Pair.Value->GetNumKeys() > 0;
	});
}

FReply SCurveEditorPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		const UE::CurveEditor::FScopedSelectionChange Transaction(GetCurveEditor(), LOCTEXT("EscapeKeys", "Press escape"));
		CurveEditor->Selection.Clear();
		return FReply::Handled();
	}
	else if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SCurveEditorPanel::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	PendingFocus.SetPendingFocusIfNeeded(AsWeak());
}

void SCurveEditorPanel::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	PendingFocus.ResetPendingFocus();
}

void SCurveEditorPanel::EnablePendingFocusOnHovering(const bool InEnabled)
{
	PendingFocus.Enable(InEnabled);
}

TSharedRef<SWidget> SCurveEditorPanel::MakeCurveEditorCurveViewOptionsMenu(TSharedPtr<FExtender> InExtender)
{
	FMenuBuilder MenuBuilder(true, CurveEditor->GetCommands(), InExtender);

	// Framing
	MenuBuilder.BeginSection("Framing", LOCTEXT("FrameHeader", "Frame"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ZoomToFit);
	}
	MenuBuilder.EndSection();
	
	// Viewing mode
	MenuBuilder.BeginSection("CurveViewMode", LOCTEXT("CurveViewModeHeader", "Curve View Mode"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeAbsolute);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeStacked);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeNormalized);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TangentVisibility", LOCTEXT("CurveEditorMenuTangentVisibilityHeader", "Tangent Visibility"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetAllTangentsVisibility);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetUserTangentsVisibility);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetSelectedKeysTangentVisibility);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetNoTangentsVisibility);
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleAutoFrameCurveEditor);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleSnapTimeToSelection);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowBufferedCurves);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowTransportControls);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowBars);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowCurveEditorCurveToolTips);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowValueIndicatorLines);

	MenuBuilder.BeginSection("CurveColors", LOCTEXT("CurveColorsHeader", "Curve Colors"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetRandomCurveColorsForSelected);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetCurveColorsForSelected);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCurveEditorPanel::MakeCurvesMenu(TSharedPtr<FExtender> InExtender)
{
	FMenuBuilder MenuBuilder(true, CurveEditor->GetCommands(), InExtender);
	AddPreInfinityToMenu(MenuBuilder);
	AddPostInfinityToMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

void SCurveEditorPanel::AddPreInfinityToMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("PreInfinity", LOCTEXT("CurveEditorMenuPreInfinityHeader", "Pre-Infinity"));
	{
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapConstant);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapCycle);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapLinear);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate);
	}
	InMenuBuilder.EndSection();
}

void SCurveEditorPanel::AddPostInfinityToMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("PostInfinity", LOCTEXT("CurveEditorMenuPostInfinityHeader", "Post-Infinity"));
	{
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapConstant);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapCycle);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapLinear);
		InMenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate);
	}
	InMenuBuilder.EndSection();
}

FSlateIcon SCurveEditorPanel::GetCurveExtrapolationPreIcon() const
{
	// We check to see if pre/post share a extrapolation mode and return a shared icon, otherwise mixed.
	if (CompareCommonPreExtrapolationMode(RCCE_Constant))
	{
		return FCurveEditorCommands::Get().SetPreInfinityExtrapConstant->GetIcon();
	}
	else if (CompareCommonPreExtrapolationMode(RCCE_Cycle))
	{
		return FCurveEditorCommands::Get().SetPreInfinityExtrapCycle->GetIcon();
	}
	else if (CompareCommonPreExtrapolationMode(RCCE_CycleWithOffset))
	{
		return FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset->GetIcon();
	}
	else if (CompareCommonPreExtrapolationMode(RCCE_Linear))
	{
		return FCurveEditorCommands::Get().SetPreInfinityExtrapLinear->GetIcon();
	}
	else if (CompareCommonPreExtrapolationMode(RCCE_Oscillate))
	{
		return FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate->GetIcon();
	}
	else
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.PreInfinityMixed");
	}
}

FSlateIcon SCurveEditorPanel::GetCurveExtrapolationPostIcon() const
{
	// We check to see if pre/post share a extrapolation mode and return a shared icon, otherwise mixed.
	if (CompareCommonPostExtrapolationMode(RCCE_Constant))
	{
		return FCurveEditorCommands::Get().SetPostInfinityExtrapConstant->GetIcon();
	}
	else if (CompareCommonPostExtrapolationMode(RCCE_Cycle))
	{
		return FCurveEditorCommands::Get().SetPostInfinityExtrapCycle->GetIcon();
	}
	else if (CompareCommonPostExtrapolationMode(RCCE_CycleWithOffset))
	{
		return FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset->GetIcon();
	}
	else if (CompareCommonPostExtrapolationMode(RCCE_Linear))
	{
		return FCurveEditorCommands::Get().SetPostInfinityExtrapLinear->GetIcon();
	}
	else if (CompareCommonPostExtrapolationMode(RCCE_Oscillate))
	{
		return FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate->GetIcon();
	}
	else
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.PostInfinityMixed");
	}
}


void SCurveEditorPanel::ShowCurveFilterUI(TSubclassOf<UCurveEditorFilterBase> FilterClass)
{
	TSharedPtr<FTabManager> TabManager = WeakTabManager.Pin();
	TSharedPtr<SDockTab> OwnerTab = TabManager.IsValid() ? TabManager->GetOwnerTab() : TSharedPtr<SDockTab>();
	TSharedPtr<SWindow> RootWindow = OwnerTab.IsValid() ? OwnerTab->GetParentWindow() : TSharedPtr<SWindow>();

	FilterPanel = SCurveEditorFilterPanel::OpenDialog(RootWindow, CurveEditor.ToSharedRef(), FilterClass);
	FilterPanel->OnFilterClassChanged.BindRaw(this, &SCurveEditorPanel::FilterClassChanged);

	FilterClassChanged();
}

void SCurveEditorPanel::FilterClassChanged()
{
	OnFilterClassChanged.ExecuteIfBound();
}


const FGeometry& SCurveEditorPanel::GetScrollPanelGeometry() const
{
	return ScrollBox->GetCachedGeometry();
}

const FGeometry& SCurveEditorPanel::GetViewContainerGeometry() const
{
	return CurveViewsContainer->GetCachedGeometry();
}

namespace UE::CurveEditor::PanelDetail
{
	
static void FillToolbar(
	FToolBarBuilder& ToolBarBuilder,
	TSharedRef<SCurveKeyDetailPanel> InKeyDetailsPanel,
	TSharedRef<SCurveEditorPanel> InEditorPanel,
	TSharedRef<FPromotedFilterContainer> InToolbarPromotedFilters,
	TSharedPtr<FExtender> InExtender
	)
{
	
} 
}

TSharedPtr<FExtender> SCurveEditorPanel::GetToolbarExtender()
{
	using namespace UE::CurveEditor::PanelDetail;
	
	// We're going to create a new Extender and add the main Curve Editor icons to it.
	// We combine this with the extender provided by the Curve Editor Module as that extender has been extended by tools
	TSharedPtr<FExtender> Extender = CombineEditorExtensions();

	Extender->AddToolBarExtension(
		"Asset", EExtensionHook::After, GetCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &SCurveEditorPanel::BuildToolbar, Extender)
	);

	return Extender;
}

TSharedPtr<FExtender> SCurveEditorPanel::CombineEditorExtensions() const
{
	const TSharedRef<FUICommandList>& CommandListRef = GetCommands().ToSharedRef();
	
	ICurveEditorModule& CurveEditorModule = FModuleManager::Get().LoadModuleChecked<ICurveEditorModule>("CurveEditor");
	TArray<TSharedPtr<FExtender>> ToolbarExtenders;
	for (ICurveEditorModule::FCurveEditorMenuExtender& ExtenderCallback : CurveEditorModule.GetAllToolBarMenuExtenders())
	{
		ToolbarExtenders.Add(ExtenderCallback.Execute(CommandListRef));
	}
	for (const TSharedRef<ICurveEditorExtension>& Extension : CurveEditor->GetEditorExtensions())
	{
		if (const TSharedPtr<FExtender> OptionalExtender = Extension->MakeToolbarExtender(CommandListRef))
		{
			ToolbarExtenders.Add(OptionalExtender);
		}
	}
	return FExtender::Combine(ToolbarExtenders);
}

void SCurveEditorPanel::BuildToolbar(FToolBarBuilder& InToolBarBuilder, TSharedPtr<FExtender> InBaseExtender)
{
	using namespace UE::CurveEditor;
	
	InToolBarBuilder.BeginSection("View");
	InToolBarBuilder.BeginStyleOverride("CurveEditorToolbar");
	{
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeCurveEditorCurveViewOptionsMenu, InBaseExtender),
			LOCTEXT("CurveEditor.ViewOptions.Label", "View"),
			LOCTEXT("CurveEditor.ViewOptions.ToolTip", "View Options"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visibility"),
			false, NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("View"))
			);
		InToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed); // ... but don't affect the other entries
	}
	InToolBarBuilder.EndSection();

	InToolBarBuilder.BeginSection("Tools");
	{
		InToolBarBuilder.BeginStyleOverride("CurveEditorToolBar.ToolsCombo"); // Min size for combo button so size stays consistent independent of label size
		InToolBarBuilder.SetLabelVisibility(EVisibility::Visible); // Show label..
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeToolsComboMenu, InBaseExtender),
			TAttribute<FText>(this, &SCurveEditorPanel::GetCurrentToolLabel),
			FText::GetEmpty(),
			TAttribute<FSlateIcon>(this, &SCurveEditorPanel::GetCurrentToolIcon),
			false, NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("Tools"))
			);
		InToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed); // ... but don't affect the other entries
		InToolBarBuilder.EndStyleOverride();
	}
	InToolBarBuilder.EndSection();

	InToolBarBuilder.BeginSection("Key Details");
	{
		FMenuEntryStyleParams StyleParams;
		StyleParams.VerticalAlignment = VAlign_Fill;

		InToolBarBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				KeyDetailsView.ToSharedRef()				
			]
			, StyleParams, NAME_None, true, {}, {}, MakeResizeParams(TEXT("KeyDetails")));
	}
	InToolBarBuilder.EndSection();

	InToolBarBuilder.BeginSection("Adjustment");
	{
		// Dropdown Menu for choosing your axis snapping for tool movement
		TAttribute<FSlateIcon> AxisSnappingModeIcon;
		AxisSnappingModeIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([this] {
			switch (this->GetCurveEditor()->GetAxisSnap().RestrictedAxisList)
			{
			case ECurveEditorSnapAxis::CESA_X:
				return FCurveEditorCommands::Get().SetAxisSnappingHorizontal->GetIcon();
			case ECurveEditorSnapAxis::CESA_Y:
				return FCurveEditorCommands::Get().SetAxisSnappingVertical->GetIcon();
			default: // ECurveEditorSnapAxis::CESA_None
				return FCurveEditorCommands::Get().SetAxisSnappingNone->GetIcon();
			}
		}));
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeAxisSnapMenu),
			LOCTEXT("AxisSnappingOptions", "Axis Snapping"),
			LOCTEXT("AxisSnappingOptionsToolTip", "Choose which axes movement tools are locked to."),
			AxisSnappingModeIcon,
			false, NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("AxisSnapping"))
			);
		
		// Toggle Button for Time Snapping
		InToolBarBuilder.AddToolBarButton(
			FCurveEditorCommands::Get().ToggleInputSnapping,
			NAME_None, {}, {}, {}, NAME_None, {}, {}, {},
			MakeResizeParams(TEXT("ToggleInputSnapping"))
			);

		// Dropdown Menu to choose the snapping scale.
		FUIAction TimeSnapMenuAction(FExecuteAction(), FCanExecuteAction::CreateLambda([this] { return !this->DisabledTimeSnapTooltipAttribute.IsSet(); }));
		InToolBarBuilder.AddComboButton(
			TimeSnapMenuAction,
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeTimeSnapMenu),
			LOCTEXT("TimeSnappingOptions", "Time Snapping"),
			TAttribute<FText>(this, &SCurveEditorPanel::GetTimeSnapMenuTooltip),
			TAttribute<FSlateIcon>(),
			true,
			NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("TimeSnapping"))
			);

		// Toggle Button for Value Snapping
		InToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleOutputSnapping,
			NAME_None, {}, {}, {}, NAME_None, {}, {}, {},
			MakeResizeParams(TEXT("ToggleOutputSnapping"))
			);

		// Dropdown Menu to choose the snapping scale.
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeGridSpacingMenu),
			LOCTEXT("GridSnappingOptions", "Grid Snapping"),
			LOCTEXT("GridSnappingOptionsToolTip", "Choose the spacing between horizontal grid lines."),
			TAttribute<FSlateIcon>(),
			true,
			NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("GridSnapping"))
			);
	}
	InToolBarBuilder.EndSection();

	InToolBarBuilder.BeginSection("Tangents");
	{
		InToolBarBuilder.BeginStyleOverride("CurveEditorToolBar.TangentMode"); // Min size for combo button so size stays consistent independent of label size
		InToolBarBuilder.SetLabelVisibility(EVisibility::Visible); // Show label...
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeTangentModeMenu),
			TAttribute<FText>::CreateSP(this, &SCurveEditorPanel::GetTangentModeLabel),
			TAttribute<FText>::CreateSP(this, &SCurveEditorPanel::GetTangentModeTooltip),
			TAttribute<FSlateIcon>::CreateSP(this, &SCurveEditorPanel::GetTangentModeIcon),
			false, NAME_None, {}, {}, {}, EUserInterfaceActionType::Button,
			MakeResizeParams(TEXT("TangentMenu"))
			);
		InToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed); // ... but don't affect the other entries
		InToolBarBuilder.EndStyleOverride();
		
		const int32 SupportedTangentTypes = this->CurveEditor->GetSupportedTangentTypes();
		if (SupportedTangentTypes & (int32)ECurveEditorTangentTypes::InterpolationCubicWeighted)
		{
			InToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationToggleWeighted,
				NAME_None, {}, {}, {}, NAME_None, {}, {}, {},
				MakeResizeParams(TEXT("InterpolationToggleWeighted"))
				);
		}
		InToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().FlattenTangents,
			NAME_None, {}, {}, {}, NAME_None, {}, {}, {},
			MakeResizeParams(TEXT("FlattenTangents"))
			);
		InToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().StraightenTangents,
			NAME_None, {}, {}, {}, NAME_None, {}, {}, {},
			MakeResizeParams(TEXT("StraightenTangents"))
			);
	}
	InToolBarBuilder.EndSection();

	InToolBarBuilder.BeginSection("Curves");
	{
		InToolBarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCurveEditorPanel::MakeCurvesMenu, InBaseExtender),
			LOCTEXT("CurveEditor.CurvesCombo.Label", "Curves"),
			LOCTEXT("CurveEditor.CurvesCombo.ToolTip", "Curves"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.Curves")
			);
		InToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed); // ... but don't affect the other entries
				
		GetCurveEditor()->GetToolbarPromotedFilters()->AppendToBuilder(InToolBarBuilder);
		InToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().OpenUserImplementableFilterWindow);
	}
	InToolBarBuilder.EndStyleOverride();
}

TSharedRef<SWidget> SCurveEditorPanel::MakeTimeSnapMenu()
{
	TSharedRef<SWidget> InputSnapWidget =
		SNew(SFrameRatePicker)
		.Value_Lambda([this] { return this->CurveEditor->InputSnapRateAttribute.Get(); })
		.OnValueChanged_Lambda([this](FFrameRate InFrameRate) { this->CurveEditor->InputSnapRateAttribute = InFrameRate; })
		.PresetValues({
		// We re-use the common frame rates but omit some of them.
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_12(),	LOCTEXT("Snap_Input_Twelve", "82ms (1/12s)"), LOCTEXT("Snap_Input_Description_Twelve", "Snap time values to one twelfth of a second (ie: 12fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_15(),	LOCTEXT("Snap_Input_Fifteen", "66ms (1/15s)"), LOCTEXT("Snap_Input_Description_Fifteen", "Snap time values to one fifteenth of a second (ie: 15fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_24(),	LOCTEXT("Snap_Input_TwentyFour", "42ms (1/24s)"), LOCTEXT("Snap_Input_Description_TwentyFour", "Snap time values to one twenty-fourth of a second (ie: 24fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_25(),	LOCTEXT("Snap_Input_TwentyFive", "40ms (1/25s)"), LOCTEXT("Snap_Input_Description_TwentyFive", "Snap time values to one twenty-fifth of a second (ie: 25fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_30(),	LOCTEXT("Snap_Input_Thirty", "33ms (1/30s)"), LOCTEXT("Snap_Input_Description_Thirty", "Snap time values to one thirtieth of a second (ie: 30fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_48(),	LOCTEXT("Snap_Input_FourtyEight", "21ms (1/48s)"), LOCTEXT("Snap_Input_Description_FourtyEight", "Snap time values to one fourth-eight of a second (ie: 48fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_50(),	LOCTEXT("Snap_Input_Fifty", "20ms (1/50s)"), LOCTEXT("Snap_Input_Description_Fifty", "Snap time values to one fiftieth of a second (ie: 50fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_60(),	LOCTEXT("Snap_Input_Sixty", "16ms (1/60s)"), LOCTEXT("Snap_Input_Description_Sixty", "Snap time values to one sixtieth of a second (ie: 60fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_100(),	LOCTEXT("Snap_Input_OneHundred", "10ms (1/100s)"), LOCTEXT("Snap_Input_Description_OneHundred", "Snap time values to one one-hundredth of a second (ie: 100fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_120(),	LOCTEXT("Snap_Input_OneHundredTwenty", "8ms (1/120s)"), LOCTEXT("Snap_Input_Description_OneHundredTwenty", "Snap time values to one one-hundred-twentieth of a second (ie: 120fps)") },
		FCommonFrameRateInfo{ FCommonFrameRates::FPS_240(), LOCTEXT("Snap_Input_TwoHundredFourty", "4ms (1/240s)"), LOCTEXT("Snap_Input_Description_TwoHundredFourty", "Snap time values to one two-hundred-fourtieth of a second (ie: 240fps)") }
			});

	return InputSnapWidget;
}

FText SCurveEditorPanel::GetTimeSnapMenuTooltip() const
{
	// If this is specified then the time snap menu is disabled
	if (DisabledTimeSnapTooltipAttribute.IsSet())
	{
		return DisabledTimeSnapTooltipAttribute.Get();
	}

	return LOCTEXT("TimeSnappingOptionsToolTip", "Choose what precision the Time axis is snapped to while moving keys.");
}

TSharedRef<SWidget> SCurveEditorPanel::MakeGridSpacingMenu()
{
	TArray<SGridLineSpacingList::FNamedValue> SpacingAmounts;
	// SnapValues.Add( SNumericDropDown<float>::FNamedValue( 0.001f, LOCTEXT( "Snap_OneThousandth", "0.001" ), LOCTEXT( "SnapDescription_OneThousandth", "Set snap to 1/1000th" ) ) );
	//SnapValues.Add( SNumericDropDown<float>::FNamedValue( 0.01f, LOCTEXT( "Snap_OneHundredth", "0.01" ), LOCTEXT( "SnapDescription_OneHundredth", "Set snap to 1/100th" ) ) );
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(0.1f, LOCTEXT("OneTenth", "0.1"), LOCTEXT("Description_OneTenth", "Set grid spacing to 1/10th")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(0.5f, LOCTEXT("OneHalf", "0.5"), LOCTEXT("Description_OneHalf", "Set grid spacing to 1/2")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(1.0f, LOCTEXT("One", "1"), LOCTEXT("Description_One", "Set grid spacing to 1")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(2.0f, LOCTEXT("Two", "2"), LOCTEXT("Description_Two", "Set grid spacing to 2")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(5.0f, LOCTEXT("Five", "5"), LOCTEXT("Description_Five", "Set grid spacing to 5")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(10.0f, LOCTEXT("Ten", "10"), LOCTEXT("Description_Ten", "Set grid spacing to 10")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(50.0f, LOCTEXT("Fifty", "50"), LOCTEXT("Description_50", "Set grid spacing to 50")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(100.0f, LOCTEXT("OneHundred", "100"), LOCTEXT("Description_OneHundred", "Set grid spacing to 100")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(TOptional<float>(), LOCTEXT("Automatic", "Automatic"), LOCTEXT("Description_Automatic", "Set grid spacing to automatic")));

	TSharedRef<SWidget> OutputSnapWidget =
		SNew(SGridLineSpacingList)
		.DropDownValues(SpacingAmounts)
		.MinDesiredValueWidth(60)
		.Value_Lambda([this]() -> TOptional<float> { return this->CurveEditor->FixedGridSpacingAttribute.Get(); })
		.OnValueChanged_Lambda([this](TOptional<float> InNewOutputSnap) { this->CurveEditor->FixedGridSpacingAttribute = InNewOutputSnap; })
		.HeaderText(LOCTEXT("CurveEditorMenuGridSpacingHeader", "Grid Spacing"));

	return OutputSnapWidget;
}

TSharedRef<SWidget> SCurveEditorPanel::MakeAxisSnapMenu()
{
	FMenuBuilder MenuBuilder(true, GetCommands());

	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetAxisSnappingNone);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetAxisSnappingHorizontal);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetAxisSnappingVertical);
	
	return MenuBuilder.MakeWidget();
}

bool SCurveEditorPanel::IsInlineEditPanelEditable() const
{
	return CurveEditor->GetSelection().Count() > 0;
}

EVisibility SCurveEditorPanel::ShouldInstructionOverlayBeVisible() const
{
	// The instruction overlay is visible if they have no selection in the tree.
	const bool bCurvesAreVisible = CurveEditor->GetTreeSelection().Num() > 0 || CurveEditor->GetPinnedCurves().Num() > 0;
	return bCurvesAreVisible ? EVisibility::Hidden : EVisibility::HitTestInvisible;
}

TSharedRef<SWidget> SCurveEditorPanel::MakeToolsComboMenu(TSharedPtr<FExtender> InExtender)
{
	FMenuBuilder MenuBuilder(true, CurveEditor->GetCommands(), InExtender);

	MenuBuilder.BeginSection(TEXT("Tools"));
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().DeactivateCurrentTool);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText SCurveEditorPanel::GetCurrentToolLabel() const
{
	const ICurveEditorToolExtension* CurrentTool = GetCurveEditor()->GetCurrentTool();
	return CurrentTool ? CurrentTool->GetLabel() : FCurveEditorCommands::Get().DeactivateCurrentTool->GetLabel();
}

FText SCurveEditorPanel::GetCurrentToolDescription() const
{
	const ICurveEditorToolExtension* CurrentTool = GetCurveEditor()->GetCurrentTool();
	return CurrentTool ? CurrentTool->GetDescription() : FCurveEditorCommands::Get().DeactivateCurrentTool->GetDescription();
}

FSlateIcon SCurveEditorPanel::GetCurrentToolIcon() const
{
	const ICurveEditorToolExtension* CurrentTool = GetCurveEditor()->GetCurrentTool();
	return CurrentTool ? CurrentTool->GetIcon() : FCurveEditorCommands::Get().DeactivateCurrentTool->GetIcon();
}

void SCurveEditorPanel::OnSplitterFinishedResizing()
{
	SSplitter::FSlot const& LeftSplitterSlot = TreeViewSplitter->SlotAt(0);
	SSplitter::FSlot const& RightSplitterSlot = TreeViewSplitter->SlotAt(1);

	OnColumnFillCoefficientChanged(LeftSplitterSlot.GetSizeValue(), 0);
	OnColumnFillCoefficientChanged(RightSplitterSlot.GetSizeValue(), 1);

	CurveEditor->GetSettings()->SetTreeViewWidth(LeftSplitterSlot.GetSizeValue());
}

void SCurveEditorPanel::OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex)
{
	ColumnFillCoefficients[ColumnIndex] = FillCoefficient;
}

#undef LOCTEXT_NAMESPACE