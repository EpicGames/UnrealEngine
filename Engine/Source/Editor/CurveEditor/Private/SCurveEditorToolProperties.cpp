// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorToolProperties.h"

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "IStructureDetailsView.h"
#include "Layout/Children.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"

class FStructOnScope;
struct FPropertyChangedEvent;


void SCurveEditorToolProperties::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor, FCurveEditorToolID InToolId)
{
	WeakCurveEditor = InCurveEditor;
	ToolId = InToolId;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	FStructureDetailsViewArgs StructureDetailsViewArgs;
	StructureDetailsViewArgs.bShowObjects = true;
	StructureDetailsViewArgs.bShowAssets = true;
	StructureDetailsViewArgs.bShowClasses = true;
	StructureDetailsViewArgs.bShowInterfaces = true;

	DetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, nullptr);

	DetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SCurveEditorToolProperties::OnFinishedChangingProperties);

	OnToolChanged(ToolId);

	ChildSlot
	[
		DetailsView->GetWidget().ToSharedRef()
	];
}

void SCurveEditorToolProperties::OnToolChanged(FCurveEditorToolID NewToolId)
{
	ToolId = NewToolId;

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();

	if (CurveEditor)
	{
		TSharedPtr<FOnOptionsRefresh> OnOptionsRefreshDelegate;
		if (CurveEditor->GetToolExtensions().Contains(ToolId))
		{
			CurveEditor->GetToolExtensions()[ToolId]->OnOptionsRefreshDelegate.AddSP(this, &SCurveEditorToolProperties::RebuildProperties);
		}
	}
	RebuildProperties();
}

void SCurveEditorToolProperties::RebuildProperties()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		TSharedPtr<FStructOnScope> ToolOptions;
		if (CurveEditor->GetToolExtensions().Contains(ToolId))
		{
			ToolOptions = CurveEditor->GetToolExtensions()[ToolId]->GetToolOptions();
			DetailsView->SetStructureData(ToolOptions);
		}
		else
		{
			ToolOptions = nullptr;
			DetailsView->SetStructureData(nullptr);
		}
	}
}

void SCurveEditorToolProperties::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		if (CurveEditor->GetCurrentTool())
		{
			CurveEditor->GetCurrentTool()->OnToolOptionsUpdated(PropertyChangedEvent);
		}
	}
}