// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorView.h"

#include "Containers/Map.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorAxis.h"
#include "CurveEditorCurveDrawParamsCache.h"
#include "CurveEditorHelpers.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSettings.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "ICurveEditorBounds.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "SCurveEditorPanel.h"
#include "Slate/SRetainerWidget.h"
#include "Templates/Greater.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

int32 GCurveEditorUseCurveCache = 1;
FAutoConsoleVariableRef CCurveEditorUseCurveCache(
	TEXT("CurveEditor.UseCurveCache"), 
	GCurveEditorUseCurveCache, 
	TEXT("When true we cache curve values, when false we always regenerate"));

SCurveEditorView::SCurveEditorView()
	: bPinned(0)
	, bInteractive(1)
	, bFixedOutputBounds(0)
	, bAutoSize(1)
	, bAllowEmpty(0)
	, bAllowModelViewTransforms(1)
	, bUpdateModelViewTransforms(0)
	, bNeedsDefaultGridLinesH(1)
	, bNeedsDefaultGridLinesV(1)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CurveCacheFlags = ECurveCacheFlags::All;
	CachedValues.CachedActiveCurvesSerialNumber = 0xFFFFFFFF;
	CachedValues.CachedSelectionSerialNumber = 0xFFFFFFFF;
	CachedValues.CachedGeometrySize.X = -1;
	CachedValues.CachedGeometrySize.Y = -1;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CurveDrawParamsCache = MakeShared<UE::CurveEditor::FCurveDrawParamsCache>();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow deprecated CachedCurveParams to destruct
SCurveEditorView::~SCurveEditorView()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FVector2D SCurveEditorView::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D ContentDesiredSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	return FVector2D(ContentDesiredSize.X, FixedHeight.Get(ContentDesiredSize.Y));
}

void SCurveEditorView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bAllowModelViewTransforms)
	{
		UpdateCurveViewTransformsFromModels();
	}
}

void SCurveEditorView::UpdateCurveViewTransformsFromModels()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();

	if (!CurveEditor)
	{
		return;
	}

	bool bTransformChanged = false;
	for (TPair<FCurveModelID, FCurveInfo>& CurveInfo : CurveInfoByID)
	{
		const FCurveModel* CurveModel = CurveEditor->FindCurve(CurveInfo.Key);
		if (CurveModel)
		{
			FTransform2d PreviousTransform = CurveInfo.Value.ViewToCurveTransform;

			CurveInfo.Value.ViewToCurveTransform = CurveModel->GetCurveTransform();

			bTransformChanged |= (CurveInfo.Value.ViewToCurveTransform != PreviousTransform);
		}
	}

	if (bTransformChanged)
	{
		CurveDrawParamsCache->Invalidate(SharedThis(this));

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CurveCacheFlags = ECurveCacheFlags::All;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void SCurveEditorView::GetInputBounds(double& OutInputMin, double& OutInputMax) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		CurveEditor->GetBounds().GetInputBounds(OutInputMin, OutInputMax);

		// This code assumes no scaling between the container and the view (which is a pretty safe assumption to make)
		const FGeometry& ViewGeometry      = GetCachedGeometry();
		const FGeometry ContainerGeometry = CurveEditor->GetPanel().IsValid() ? CurveEditor->GetPanel()->GetViewContainerGeometry() : ViewGeometry;

		const float ContainerWidth = ContainerGeometry.GetLocalSize().X;
		const float ViewWidth      = ViewGeometry.GetLocalSize().X;

		if (ViewWidth > 0.f)
		{
			const float LeftPixelCrop = ViewGeometry.LocalToAbsolute(FVector2D(0.f, 0.f)).X - ContainerGeometry.LocalToAbsolute(FVector2D(0.f, 0.f)).X;
			const float RightPixelCrop = ContainerGeometry.LocalToAbsolute(FVector2D(ContainerWidth, 0.f)).X - ViewGeometry.LocalToAbsolute(FVector2D(ViewWidth, 0.f)).X;

			const double ContainerInputPerPixel = (OutInputMax - OutInputMin) / ContainerWidth;

			// Offset by the total range first
			OutInputMin += ContainerInputPerPixel * LeftPixelCrop;
			OutInputMax -= ContainerInputPerPixel * RightPixelCrop;
		}
	}
}

FCurveEditorScreenSpace SCurveEditorView::GetViewSpace() const
{
	double InputMin = 0.0, InputMax = 1.0;
	GetInputBounds(InputMin, InputMax);

	return FCurveEditorScreenSpace(GetCachedGeometry().GetLocalSize(), InputMin, InputMax, OutputMin, OutputMax);
}

FCurveEditorScreenSpace SCurveEditorView::GetCurveSpace(FCurveModelID CurveID) const
{
	const_cast<SCurveEditorView*>(this)->UpdateCustomAxes();

	FCurveInfo CurveInfo = CurveInfoByID.FindRef(CurveID);

	FTransform2d CurveTransform = CurveInfo.ViewToCurveTransform;

	double InputMin = 0.0, InputMax = 1.0;
	GetInputBounds(InputMin, InputMax);

	double AxisOutputMin = OutputMin;
	double AxisOutputMax = OutputMax;
	if (CurveInfo.HorizontalAxis)
	{
		InputMin = CustomHorizontalAxes[CurveInfo.HorizontalAxis.Index].Min;
		InputMax = CustomHorizontalAxes[CurveInfo.HorizontalAxis.Index].Max;
	}
	if (CurveInfo.VerticalAxis)
	{
		AxisOutputMin = CustomVerticalAxes[CurveInfo.VerticalAxis.Index].Min;
		AxisOutputMax = CustomVerticalAxes[CurveInfo.VerticalAxis.Index].Max;
	}

	return FCurveEditorScreenSpace(GetCachedGeometry().GetLocalSize(), InputMin, InputMax, AxisOutputMin, AxisOutputMax).ToCurveSpace(CurveTransform);
}

FTransform2d SCurveEditorView::GetViewToCurveTransform(FCurveModelID CurveID) const
{
	FCurveInfo CurveInfo = CurveInfoByID.FindRef(CurveID);
	FTransform2d CurveTransform = CurveInfo.ViewToCurveTransform;
	return CurveTransform;
}

FCurveEditorScreenSpaceH SCurveEditorView::GetHorizontalAxisSpace(FCurveEditorViewAxisID ID) const
{
	double Min = 0.0;
	double Max = 1.0;

	if (ID)
	{
		Min = CustomHorizontalAxes[ID].Min;
		Max = CustomHorizontalAxes[ID].Max;
	}
	else
	{
		GetInputBounds(Min, Max);
	}

	return FCurveEditorScreenSpaceH(GetCachedGeometry().GetLocalSize().X, Min, Max);
}

FCurveEditorScreenSpaceV SCurveEditorView::GetVerticalAxisSpace(FCurveEditorViewAxisID ID) const
{
	double Min = OutputMin;
	double Max = OutputMax;

	if (ID)
	{
		Min = CustomVerticalAxes[ID].Min;
		Max = CustomVerticalAxes[ID].Max;
	}
	return FCurveEditorScreenSpaceV(GetCachedGeometry().GetLocalSize().X, Min, Max);
}

FCurveEditorViewAxisID SCurveEditorView::GetAxisForCurve(FCurveModelID CurveID, ECurveEditorAxisOrientation Axis) const
{
	const_cast<SCurveEditorView*>(this)->UpdateCustomAxes();

	FCurveInfo CurveInfo = CurveInfoByID.FindRef(CurveID);

	return (Axis == ECurveEditorAxisOrientation::Horizontal)
		? CurveInfo.HorizontalAxis
		: CurveInfo.VerticalAxis;
}

TSharedPtr<FCurveEditorAxis> SCurveEditorView::GetAxis(FCurveEditorViewAxisID ID, ECurveEditorAxisOrientation Axis) const
{
	if (!ID)
	{
		return nullptr;
	}

	return (Axis == ECurveEditorAxisOrientation::Horizontal)
		? CustomHorizontalAxes[ID.Index].Axis
		: CustomVerticalAxes[ID.Index].Axis;
}

void SCurveEditorView::AddCurve(FCurveModelID CurveID)
{
	CurveInfoByID.Add(CurveID, FCurveInfo{CurveInfoByID.Num()});
	OnCurveListChanged();
	if (bAllowModelViewTransforms)
	{
		bUpdateModelViewTransforms = true;
	}
}

void SCurveEditorView::RemoveCurve(FCurveModelID CurveID)
{
	if (FCurveInfo* InfoToRemove = CurveInfoByID.Find(CurveID))
	{
		const int32 CurveIndex = InfoToRemove->CurveIndex;
		InfoToRemove = nullptr;

		CurveInfoByID.Remove(CurveID);

		for (TTuple<FCurveModelID, FCurveInfo>& Info : CurveInfoByID)
		{
			if (Info.Value.CurveIndex > CurveIndex)
			{
				--Info.Value.CurveIndex;
			}
		}

		OnCurveListChanged();
		if (bAllowModelViewTransforms)
		{
			bUpdateModelViewTransforms = true;
		}
	}
}

void SCurveEditorView::UpdateCustomAxes()
{
	if (!bUpdateModelViewTransforms)
	{
		return;
	}

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	check(CurveEditor.IsValid());

	bUpdateModelViewTransforms = false;

	bool bHasAnyCustomAxesH = false;
	bool bHasAnyCustomAxesV = false;

	bool bHasAnyDefaultAxesH = false;
	bool bHasAnyDefaultAxesV = false;

	TSortedMap<int32, int32> OldHorizontalToNew;

	struct FWorkingAxisInfo
	{
		FAxisInfo Info;
		FCurveEditorViewAxisID ID;
	};
	TMap<TSharedPtr<FCurveEditorAxis>, FCurveEditorViewAxisID> NewCustomHorizontalAxes;
	TMap<TSharedPtr<FCurveEditorAxis>, FCurveEditorViewAxisID> NewCustomVerticalAxes;

	TArray<FWorkingAxisInfo> HorizontalAxisIdMap, VerticalAxisIdMap;


	// Populate existing axes with 0 use-counts to preserve scale
	for (const FAxisInfo& Info : CustomHorizontalAxes)
	{
		FWorkingAxisInfo WorkingInfo;
		WorkingInfo.Info = Info;
		WorkingInfo.ID = HorizontalAxisIdMap.Num();
		NewCustomHorizontalAxes.Add(Info.Axis, WorkingInfo.ID);
		HorizontalAxisIdMap.Add(WorkingInfo);
	}
	for (const FAxisInfo& Info : CustomVerticalAxes)
	{
		FWorkingAxisInfo WorkingInfo;
		WorkingInfo.Info = Info;
		WorkingInfo.ID = VerticalAxisIdMap.Num();
		NewCustomVerticalAxes.Add(Info.Axis, WorkingInfo.ID);
		VerticalAxisIdMap.Add(WorkingInfo);
	}


	// Iterate all curves and allocate axes
	for (TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
	{
		FCurveModel* CurveModel = CurveEditor->FindCurve(Pair.Key);
		FCurveEditorViewAxisID HorizontalAxisID, VerticalAxisID;

		if (ensureAlways(CurveModel))
		{
			TSharedPtr<FCurveEditorAxis> HorizontalAxis = nullptr;
			TSharedPtr<FCurveEditorAxis> VerticalAxis = nullptr;

			CurveModel->AllocateAxes(CurveEditor.Get(), HorizontalAxis, VerticalAxis);

			// Horizontal Axis
			if (HorizontalAxis)
			{
				HorizontalAxisID = NewCustomHorizontalAxes.FindRef(HorizontalAxis);
				if (!HorizontalAxisID)
				{
					HorizontalAxisID = HorizontalAxisIdMap.Num();
					HorizontalAxisIdMap.Add(FWorkingAxisInfo{ FAxisInfo{HorizontalAxis}, HorizontalAxisID });
					NewCustomHorizontalAxes.Add(HorizontalAxis, HorizontalAxisID);
				}

				HorizontalAxisIdMap[HorizontalAxisID.Index].Info.UseCount += 1;
				bHasAnyCustomAxesH = true;
			}
			else
			{
				bHasAnyDefaultAxesH = true;
			}

			// Vertical Axis
			if (VerticalAxis)
			{
				VerticalAxisID = NewCustomVerticalAxes.FindRef(VerticalAxis);
				if (!VerticalAxisID)
				{
					VerticalAxisID = VerticalAxisIdMap.Num();
					VerticalAxisIdMap.Add(FWorkingAxisInfo{ FAxisInfo{VerticalAxis}, VerticalAxisID });
					NewCustomVerticalAxes.Add(VerticalAxis, VerticalAxisID);
				}

				VerticalAxisIdMap[VerticalAxisID.Index].Info.UseCount += 1;
				bHasAnyCustomAxesV = true;
			}
			else
			{
				bHasAnyDefaultAxesV = true;
			}
		}

		Pair.Value.HorizontalAxis = HorizontalAxisID;
		Pair.Value.VerticalAxis   = VerticalAxisID;
	}

	CustomHorizontalAxes.SetNum(HorizontalAxisIdMap.Num());
	CustomVerticalAxes.SetNum(VerticalAxisIdMap.Num());

	// Sort the IDs by use-count descending
	Algo::SortBy(HorizontalAxisIdMap, [](const FWorkingAxisInfo& In){ return In.Info.UseCount; }, TGreater<>());
	Algo::SortBy(VerticalAxisIdMap,   [](const FWorkingAxisInfo& In){ return In.Info.UseCount; }, TGreater<>());

	// Assign new axis IDs
	TArray<FCurveEditorViewAxisID> ReverseHorizontalLookup, ReverseVerticalLookup;

	ReverseHorizontalLookup.SetNum(HorizontalAxisIdMap.Num());
	for (int32 Index = 0; Index < HorizontalAxisIdMap.Num(); ++Index)
	{
		FCurveEditorViewAxisID OldId = HorizontalAxisIdMap[Index].ID;
		ReverseHorizontalLookup[OldId.Index] = Index;
		CustomHorizontalAxes[Index] = HorizontalAxisIdMap[Index].Info;
	}

	ReverseVerticalLookup.SetNum(VerticalAxisIdMap.Num());
	for (int32 Index = 0; Index < VerticalAxisIdMap.Num(); ++Index)
	{
		FCurveEditorViewAxisID OldId = VerticalAxisIdMap[Index].ID;
		ReverseVerticalLookup[OldId.Index] = Index;
		CustomVerticalAxes[Index] = VerticalAxisIdMap[Index].Info;
	}

	for (TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
	{
		if (Pair.Value.HorizontalAxis)
		{
			Pair.Value.HorizontalAxis = ReverseHorizontalLookup[Pair.Value.HorizontalAxis.Index];
		}
		if (Pair.Value.VerticalAxis)
		{
			Pair.Value.VerticalAxis = ReverseVerticalLookup[Pair.Value.VerticalAxis.Index];
		}
	}


	bNeedsDefaultGridLinesH = bHasAnyDefaultAxesH;
	bNeedsDefaultGridLinesV = bHasAnyDefaultAxesV;
}

void SCurveEditorView::InitCurveEditorReference(const TSharedRef<FCurveEditor>& InCurveEditor)
{
	WeakCurveEditor = InCurveEditor;
	
	// When the curve colors are changed, issue a redraw.
	InCurveEditor->OnCurveColorsChangedDelegate.AddSPLambda(this, [this](TConstArrayView<FCurveModelID> Curves)
	{
		// The cache does not save any info about colors so it cannot detect them changing... we'll just set the dirty flag and trigger a refresh.
		CurveDrawParamsCache->CurveCacheFlags = UE::CurveEditor::FCurveDrawParamsCache::ECurveCacheFlags::UpdateAll;
		RefreshRetainer();
	});
}

void SCurveEditorView::FrameVertical(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID)
{
	if (!bFixedOutputBounds && InOutputMin < InOutputMax)
	{
		if (AxisID)
		{
			FAxisInfo& AxisInfo = CustomVerticalAxes[AxisID];
			AxisInfo.Min = InOutputMin;
			AxisInfo.Max = InOutputMax;
		}
		else if (TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
		{
			OutputMin = InOutputMin;
			OutputMax = InOutputMax;
		}
	}
}

void SCurveEditorView::FrameHorizontal(double InInputMin, double InInputMax, FCurveEditorViewAxisID AxisID)
{
	if (InInputMin < InInputMax)
	{
		if (AxisID)
		{
			FAxisInfo& AxisInfo = CustomHorizontalAxes[AxisID];
			AxisInfo.Min = InInputMin;
			AxisInfo.Max = InInputMax;
		}
		else if (TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
		{
			CurveEditor->GetBounds().SetInputBounds(InInputMin, InInputMax);
		}
	}
}

TSharedPtr<FCurveEditor> SCurveEditorView::GetCurveEditor() const
{
	return WeakCurveEditor.IsValid() ? WeakCurveEditor.Pin() : nullptr;
}

UE::CurveEditor::FOnCurveHasChangedExternally& SCurveEditorView::OnCurveHasChangedExternally()
{
	return CurveDrawParamsCache->OnCurveHasChangedExternallyDelegate;
}

void SCurveEditorView::SetOutputBounds(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID)
{
	if (!bFixedOutputBounds && InOutputMin < InOutputMax)
	{
		if (AxisID)
		{
			FAxisInfo& AxisInfo = CustomVerticalAxes[AxisID];
			AxisInfo.Min = InOutputMin;
			AxisInfo.Max = InOutputMax;
		}

		// When no axis ID is specified, we scale all axes based on the change
		else if (TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
		{
			double CurrentRange = (OutputMax - OutputMin);
			double OffsetFactor = (InOutputMin - OutputMin) / CurrentRange;
			double Scale        = (InOutputMax - InOutputMin) / CurrentRange;

			OutputMin = InOutputMin;
			OutputMax = InOutputMax;

			// Set all axes scale
			for (FAxisInfo& AxisInfo : CustomVerticalAxes)
			{
				double AxisRange = (AxisInfo.Max - AxisInfo.Min);

				AxisInfo.Min += AxisRange * OffsetFactor;
				AxisInfo.Max  = AxisInfo.Min + AxisRange * Scale;
			}
		}
	}
}

void SCurveEditorView::SetInputBounds(double InInputMin, double InInputMax, FCurveEditorViewAxisID AxisID)
{
	if (InInputMin < InInputMax)
	{
		if (AxisID)
		{
			FAxisInfo& AxisInfo = CustomHorizontalAxes[AxisID];
			AxisInfo.Min = InInputMin;
			AxisInfo.Max = InInputMax;
		}
		// When no axis ID is specified, we scale all axes based on the change
		else if (TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
		{
			double CurrentInputMin = 0.0, CurrentInputMax = 1.0;
			CurveEditor->GetBounds().GetInputBounds(CurrentInputMin, CurrentInputMax);

			double CurrentRange = (CurrentInputMax - CurrentInputMin);
			double OffsetFactor = (InInputMin - CurrentInputMin) / CurrentRange;
			double Scale        = (InInputMax - InInputMin) / CurrentRange;

			// Set global scale
			CurveEditor->GetBounds().SetInputBounds(InInputMin, InInputMax);

			// Set all axes scale
			for (FAxisInfo& AxisInfo : CustomHorizontalAxes)
			{
				double AxisRange = (AxisInfo.Max - AxisInfo.Min);

				AxisInfo.Min += AxisRange * OffsetFactor;
				AxisInfo.Max  = AxisInfo.Min + AxisRange * Scale;
			}
		}
	}
}

void SCurveEditorView::Zoom(const FVector2D& Amount)
{
	FCurveEditorScreenSpace ViewSpace = GetViewSpace();

	const double InputOrigin  = (ViewSpace.GetInputMax()  - ViewSpace.GetInputMin())  * 0.5;
	const double OutputOrigin = (ViewSpace.GetOutputMax() - ViewSpace.GetOutputMin()) * 0.5;

	ZoomAround(Amount, InputOrigin, OutputOrigin);
}

void SCurveEditorView::ZoomAround(const FVector2D& Amount, double InputOrigin, double OutputOrigin)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	check(CurveEditor.IsValid());

	if (Amount.X != 0.f && CurveEditor.IsValid())
	{
		double InputMin = 0.0, InputMax = 1.0;
		CurveEditor->GetBounds().GetInputBounds(InputMin, InputMax);

		const double OriginAlpha = (InputOrigin - InputMin) / (InputMax - InputMin);

		InputMin = InputOrigin - (InputOrigin - InputMin) * Amount.X;
		InputMax = InputOrigin + (InputMax - InputOrigin) * Amount.X;

		CurveEditor->GetBounds().SetInputBounds(InputMin, InputMax);

		for (FAxisInfo& AxisInfo : CustomHorizontalAxes)
		{
			const double AxisInputOrigin = AxisInfo.Min + (AxisInfo.Max - AxisInfo.Min)*OriginAlpha;

			AxisInfo.Min = AxisInputOrigin - (AxisInputOrigin - AxisInfo.Min) * Amount.Y;
			AxisInfo.Max = AxisInputOrigin + (AxisInfo.Max - AxisInputOrigin) * Amount.Y;
		}
	}

	if (Amount.Y != 0.f)
	{
		const double OriginAlpha = (OutputOrigin - OutputMin) / (OutputMax - OutputMin);

		OutputMin = OutputOrigin - (OutputOrigin - OutputMin) * Amount.Y;
		OutputMax = OutputOrigin + (OutputMax - OutputOrigin) * Amount.Y;

		for (FAxisInfo& AxisInfo : CustomVerticalAxes)
		{
			const double AxisOutputOrigin = AxisInfo.Min + (AxisInfo.Max - AxisInfo.Min)*OriginAlpha;

			AxisInfo.Min = AxisOutputOrigin - (AxisOutputOrigin - AxisInfo.Min) * Amount.Y;
			AxisInfo.Max = AxisOutputOrigin + (AxisInfo.Max - AxisOutputOrigin) * Amount.Y;
		}
	}
}

void SCurveEditorView::GetCurveDrawParams(TArray<FCurveDrawParams>& OutDrawParams)
{
	// DEPRECATED 5.6, instead use FCurveDrawParamsCache::GetCurvedDrawParamsSynchronous

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	// Get the Min/Max values on the X axis, for Time
	double InputMin = 0, InputMax = 1;
	GetInputBounds(InputMin, InputMax);

	//make sure the transform is set up
	UpdateViewToTransformCurves(InputMin, InputMax);

	OutDrawParams.Reset(CurveInfoByID.Num());

	for (const TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
	{
		FCurveDrawParams Params(Pair.Key);

		FCurveModel* CurveModel = CurveEditor->FindCurve(Pair.Key);
		if (!ensureAlways(CurveModel))
		{
			continue;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetCurveDrawParam(CurveEditor, Pair.Key, CurveModel, Params);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		OutDrawParams.Add(MoveTemp(Params));
	}
}

void SCurveEditorView::GetCurveDrawParam(TSharedPtr<FCurveEditor>& CurveEditor, const FCurveModelID& ModelID, FCurveModel* CurveModel, FCurveDrawParams& Params) const
{	
	// DEPRECATED 5.6, instead use FCurveDrawParamsCache::GetCurveDrawParamSynchronous

	FCurveEditorScreenSpace CurveSpace = GetCurveSpace(ModelID);

	const double InputMin = CurveSpace.GetInputMin();
	const double InputMax = CurveSpace.GetInputMax();

	const double DisplayRatio = (CurveSpace.PixelsPerOutput() / CurveSpace.PixelsPerInput());

	const FKeyHandleSet* SelectedKeys = CurveEditor->GetSelection().GetAll().Find(ModelID);

	// Create a new set of Curve Drawing Parameters to represent this particular Curve
	Params.Color = CurveModel->GetColor();
	Params.Thickness = CurveModel->GetThickness();
	Params.DashLengthPx = CurveModel->GetDashLength();
	Params.bKeyDrawEnabled = CurveModel->IsKeyDrawEnabled();

	// Gather the display metrics to use for each key type. This allows a Curve Model to override
	// whether or not the curve supports Keys, Arrive/Leave Tangents, etc. If the Curve Model doesn't
	// support a particular capability we can skip drawing them.
	CurveModel->GetKeyDrawInfo(ECurvePointType::ArriveTangent, FKeyHandle::Invalid(), Params.ArriveTangentDrawInfo);
	CurveModel->GetKeyDrawInfo(ECurvePointType::LeaveTangent, FKeyHandle::Invalid(), Params.LeaveTangentDrawInfo);

	// Gather the interpolating points in input/output space
	TArray<TTuple<double, double>> InterpolatingPoints;

	CurveModel->DrawCurve(*CurveEditor, CurveSpace, InterpolatingPoints);
	Params.InterpolatingPoints.Reset(InterpolatingPoints.Num());

	// An Input Offset allows for a fixed offset to all keys, such as displaying them in the middle of a frame instead of at the start.
	double InputOffset = CurveModel->GetInputDisplayOffset();

	// Convert the interpolating points to screen space
	for (TTuple<double, double> Point : InterpolatingPoints)
	{
		Params.InterpolatingPoints.Add(
			FVector2D(
				CurveSpace.SecondsToScreen(Point.Get<0>() + InputOffset),
				CurveSpace.ValueToScreen(Point.Get<1>())
			)
		);
	}

	TArray<FKeyHandle> VisibleKeys;
	CurveModel->GetKeys(InputMin, InputMax, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), VisibleKeys);

	// Always reset the points to cover case going from 1 to 0 keys
	Params.Points.Reset(VisibleKeys.Num());
	
	if (VisibleKeys.Num())
	{
		ECurveEditorTangentVisibility TangentVisibility = CurveEditor->GetSettings()->GetTangentVisibility();

		TArray<FKeyPosition> AllKeyPositions;
		TArray<FKeyAttributes> AllKeyAttributes;

		AllKeyPositions.SetNum(VisibleKeys.Num());
		AllKeyAttributes.SetNum(VisibleKeys.Num());

		CurveModel->GetKeyPositions(VisibleKeys, AllKeyPositions);
		CurveModel->GetKeyAttributes(VisibleKeys, AllKeyAttributes);
		for (int32 Index = 0; Index < VisibleKeys.Num(); ++Index)
		{
			const FKeyHandle      KeyHandle = VisibleKeys[Index];
			const FKeyPosition& KeyPosition = AllKeyPositions[Index];
			const FKeyAttributes& Attributes = AllKeyAttributes[Index];

			bool bShowTangents = TangentVisibility == ECurveEditorTangentVisibility::AllTangents ||
				(TangentVisibility == ECurveEditorTangentVisibility::SelectedKeys && SelectedKeys &&
					(SelectedKeys->Contains(VisibleKeys[Index], ECurvePointType::Any)));

			double TimeScreenPos = CurveSpace.SecondsToScreen(KeyPosition.InputValue + InputOffset);
			double ValueScreenPos = CurveSpace.ValueToScreen(KeyPosition.OutputValue);

			// Add this key
			FCurvePointInfo Key(KeyHandle);
			Key.ScreenPosition = FVector2D(TimeScreenPos, ValueScreenPos);
			Key.LayerBias = 2;

			// Add draw info for the specific key
			CurveModel->GetKeyDrawInfo(ECurvePointType::Key, KeyHandle, /*Out*/ Key.DrawInfo);
			Params.Points.Add(Key);

			if (bShowTangents && Attributes.HasArriveTangent())
			{
				float ArriveTangent = Attributes.GetArriveTangent();

				FCurvePointInfo ArriveTangentPoint(KeyHandle);
				ArriveTangentPoint.Type = ECurvePointType::ArriveTangent;


				if (Attributes.HasTangentWeightMode() && Attributes.HasArriveTangentWeight() &&
					(Attributes.GetTangentWeightMode() == RCTWM_WeightedBoth || Attributes.GetTangentWeightMode() == RCTWM_WeightedArrive))
				{
					FVector2D TangentOffset = CurveEditor::ComputeScreenSpaceTangentOffset(CurveSpace, ArriveTangent, -Attributes.GetArriveTangentWeight());
					ArriveTangentPoint.ScreenPosition = Key.ScreenPosition + TangentOffset;
				}
				else
				{
					float PixelLength = 60.0f;
					ArriveTangentPoint.ScreenPosition = Key.ScreenPosition + CurveEditor::GetVectorFromSlopeAndLength(ArriveTangent * -DisplayRatio, -PixelLength);
				}
				ArriveTangentPoint.LineDelta = Key.ScreenPosition - ArriveTangentPoint.ScreenPosition;
				ArriveTangentPoint.LayerBias = 1;

				// Add draw info for the specific tangent
				FKeyDrawInfo TangentDrawInfo;
				CurveModel->GetKeyDrawInfo(ECurvePointType::ArriveTangent, KeyHandle, /*Out*/ ArriveTangentPoint.DrawInfo);

				Params.Points.Add(ArriveTangentPoint);
			}

			if (bShowTangents && Attributes.HasLeaveTangent())
			{
				float LeaveTangent = Attributes.GetLeaveTangent();

				FCurvePointInfo LeaveTangentPoint(KeyHandle);
				LeaveTangentPoint.Type = ECurvePointType::LeaveTangent;

				if (Attributes.HasTangentWeightMode() && Attributes.HasLeaveTangentWeight() &&
					(Attributes.GetTangentWeightMode() == RCTWM_WeightedBoth || Attributes.GetTangentWeightMode() == RCTWM_WeightedLeave))
				{
					FVector2D TangentOffset = CurveEditor::ComputeScreenSpaceTangentOffset(CurveSpace, LeaveTangent, Attributes.GetLeaveTangentWeight());

					LeaveTangentPoint.ScreenPosition = Key.ScreenPosition + TangentOffset;
				}
				else
				{
					float PixelLength = 60.0f;
					LeaveTangentPoint.ScreenPosition = Key.ScreenPosition + CurveEditor::GetVectorFromSlopeAndLength(LeaveTangent * -DisplayRatio, PixelLength);
				}

				LeaveTangentPoint.LineDelta = Key.ScreenPosition - LeaveTangentPoint.ScreenPosition;
				LeaveTangentPoint.LayerBias = 1;

				// Add draw info for the specific tangent
				FKeyDrawInfo TangentDrawInfo;
				CurveModel->GetKeyDrawInfo(ECurvePointType::LeaveTangent, KeyHandle, /*Out*/ LeaveTangentPoint.DrawInfo);

				Params.Points.Add(LeaveTangentPoint);
			}
		}
	}
}

void SCurveEditorView::RefreshRetainer()
{
	if (RetainerWidget)
	{
		RetainerWidget->RequestRender();
	}
}

void SCurveEditorView::CheckCacheAndInvalidateIfNeeded()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid() == false)
	{
		return;
	}
	
	const bool bUseCurveCache = GCurveEditorUseCurveCache > 0;
	if (bUseCurveCache)
	{
		TArray<FCurveModelID> ModelIDs;
		CurveInfoByID.GetKeys(ModelIDs);

		CurveDrawParamsCache->Invalidate(SharedThis(this), ModelIDs);
	}
	else
	{
		// Using deprecated methods to allow testing if the deprecated API remains functional for the time being.
		// Can be updated to DrawParamsCache::GetDrawParamSynchronous when GetCurveDrawParams is removed.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CachedValues.CachedOutputMin = OutputMin;
		CachedValues.CachedOutputMax = OutputMax;
		GetInputBounds(CachedValues.CachedInputMin, CachedValues.CachedInputMax);
		CachedValues.CachedGeometrySize = GetCachedGeometry().GetLocalSize();
	
		CurveDrawParamsCache->CachedDrawParams.Reset();

		GetCurveDrawParams(CurveDrawParamsCache->CachedDrawParams);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		RefreshRetainer();
	}
}
