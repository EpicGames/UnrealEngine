// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorViewContainer.h"

#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "Delegates/Delegate.h"
#include "DragOperations/CurveEditorDragOperation_Marquee.h"
#include "DragOperations/CurveEditorDragOperation_Pan.h"
#include "DragOperations/CurveEditorDragOperation_Zoom.h"
#include "DragOperations/CurveEditorDragOperation_ScrubTime.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "ICurveEditorBounds.h"
#include "ICurveEditorToolExtension.h"
#include "ITimeSlider.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/InputState.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Slate/SRetainerWidget.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Views/SInteractiveCurveEditorView.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"
#include "CurveEditorCommands.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;

#define LOCTEXT_NAMESPACE "SCurveEditorViewContainer"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "EngineUtils.h"

extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR

void SCurveEditorViewContainer::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
{
	SetCanTick(true);
	CurveEditor = InCurveEditor;

	TimeSliderController = InArgs._ExternalTimeSliderController;
	MinimumPanelHeight = InArgs._MinimumPanelHeight;

	CurveEditor->OnActiveToolChangedDelegate.AddSP(this, &SCurveEditorViewContainer::OnCurveEditorToolChanged);

	SetClipping(EWidgetClipping::ClipToBounds);
}

bool SCurveEditorViewContainer::ComputeVolatility() const
{
	return true;
}

FVector2D SCurveEditorViewContainer::ComputeDesiredSize(float) const
{
	FVector2D MyDesiredSize(0,0);

	int32 NumStretchPanels = 0;
	for (int32 Index = 0; Index < Children.Num(); ++Index)
	{
		const SBoxPanel::FSlot& Child = Children[Index];
		if (Child.GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{
			const FVector2D& ChildDesiredSize = Child.GetWidget()->GetDesiredSize();

			FMargin SlotPadding = Child.GetPadding();
			MyDesiredSize.Y += SlotPadding.GetTotalSpaceAlong<Orient_Vertical>();

			// For a vertical panel, we want to find the maximum desired width (including margin).
			// That will be the desired width of the whole panel.
			MyDesiredSize.X = FMath::Max(MyDesiredSize.X, ChildDesiredSize.X + SlotPadding.GetTotalSpaceAlong<Orient_Horizontal>());

			if (Child.GetSizeRule() == FSizeParam::SizeRule_Stretch)
			{
				++NumStretchPanels;
			}
			else
			{
				MyDesiredSize.Y += ChildDesiredSize.Y;
			}
		}
	}

	const float PanelHeight = CurveEditor->GetPanel()->GetScrollPanelGeometry().GetLocalSize().Y - 1.f;

	if (NumStretchPanels > 0)
	{
		MyDesiredSize.Y += FMath::Max(MinimumPanelHeight, (PanelHeight - MyDesiredSize.Y) / NumStretchPanels) * NumStretchPanels;
	}

	MyDesiredSize.Y = FMath::Max(MyDesiredSize.Y, PanelHeight);
	return MyDesiredSize;
}

void SCurveEditorViewContainer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ExpandInputBounds(AllottedGeometry.GetLocalSize().X);

	//we manually check the child views since we can't rely on ::Tick being called since the RetainerWidget will stop them if not actually rendering
	for (TSharedPtr<SCurveEditorView>& View : Views)
	{
		if (View)
		{
			View->CheckCacheAndInvalidateIfNeeded();
		}
	}

	if (CurveEditor->GetCurrentTool())
	{
		CurveEditor->GetCurrentTool()->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

int32 SCurveEditorViewContainer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	static const FName BackgroundBrushName("Brushes.Panel");
	const FSlateBrush* Background = FAppStyle::GetBrush(BackgroundBrushName);
	FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), Background, DrawEffects, Background->GetTint(InWidgetStyle));

	SVerticalBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if (CurveEditor->GetCurrentTool())
	{
		CurveEditor->GetCurrentTool()->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + CurveViewConstants::ELayerOffset::Tools, InWidgetStyle, bParentEnabled);
	}

	if (DragOperation.IsSet() && DragOperation->IsDragging())
	{
		// We want this to be relative to the view pane for the curves and not the global editor.
		DragOperation->DragImpl->Paint(AllottedGeometry, OutDrawElements, LayerId + CurveViewConstants::ELayerOffset::DragOperations);
	}

	if (TimeSliderController)
	{
		FPaintViewAreaArgs PaintArgs;
		PaintArgs.bDisplayTickLines = false;
		PaintArgs.bDisplayScrubPosition = true;
		PaintArgs.bDisplayMarkedFrames = false;
		PaintArgs.PlaybackRangeArgs = FPaintPlaybackRangeArgs(
			FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L"),
			FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R"),
			6.f);

		TimeSliderController->OnPaintViewArea(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + CurveViewConstants::ELayerOffset::GridOverlays, bParentEnabled, PaintArgs);
	}

	return LayerId + CurveViewConstants::ELayerOffset::Last;
}

bool SCurveEditorViewContainer::IsScrubTimeKeyEvent(const FKeyEvent& InKeyEvent)
{
	const FCurveEditorCommands& Commands = FCurveEditorCommands::Get();
	// Need to iterate through primary and secondary to make sure they are all pressed.
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		const FInputChord& Chord = *Commands.ScrubTime->GetActiveChord(ChordIndex);
		const bool bIsMovingTimeSlider = Chord.IsValidChord() && InKeyEvent.GetKey() == Chord.Key &&
			(Chord.NeedsAlt() ? InKeyEvent.IsAltDown() : !InKeyEvent.IsAltDown()) &&
			(Chord.NeedsShift() ? InKeyEvent.IsShiftDown() : !InKeyEvent.IsShiftDown()) &&
			(Chord.NeedsControl() ? InKeyEvent.IsControlDown() : !InKeyEvent.IsControlDown());

		if (bIsMovingTimeSlider)
		{
			return true;
		}
	}
	return false;
}

void SCurveEditorViewContainer::BroadcastOnCurveHasChangedExternally(const FCurveModelID& InCurveModelId) const
{
	OnCurveHasChangedExternallyDelegate.Broadcast(InCurveModelId);
}

FReply SCurveEditorViewContainer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply ToolReply = FReply::Unhandled();
	if (CurveEditor->GetCurrentTool())
	{
		ToolReply = CurveEditor->GetCurrentTool()->OnKeyDown(AsShared(), MyGeometry, InKeyEvent);
	}

	if (!ToolReply.IsEventHandled() &&
		InKeyEvent.GetKey() == EKeys::Escape && DragOperation.IsSet())
	{
		DragOperation->Cancel();
		DragOperation.Reset();
		return FReply::Handled();
	}

	if (!ToolReply.IsEventHandled() && 
		IsScrubTimeKeyEvent(InKeyEvent))
	{
		UE::CurveEditor::InputState::SetKeyPressed(InKeyEvent.GetKey(), true);
		UE::CurveEditor::InputState::SetScrubTimeKeyPressed(InKeyEvent.GetKey(), true);
		bIsScrubbingTime = true;
		return FReply::Handled();
	}

	return ToolReply;
}

FReply SCurveEditorViewContainer::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply ToolReply = FReply::Unhandled();
	if (CurveEditor->GetCurrentTool())
	{
		ToolReply = CurveEditor->GetCurrentTool()->OnKeyUp(AsShared(), MyGeometry, InKeyEvent);
	}

	if (!ToolReply.IsEventHandled() && 
		IsScrubTimeKeyEvent(InKeyEvent) && bIsScrubbingTime )
	{
		UE::CurveEditor::InputState::SetKeyPressed(InKeyEvent.GetKey(), false);
		UE::CurveEditor::InputState::SetScrubTimeKeyPressed(InKeyEvent.GetKey(), false);
		bIsScrubbingTime = false;
		if (DragOperation.IsSet())
		{
			DragOperation->Cancel();
			DragOperation.Reset();
			return FReply::Handled();
		}
	}
	
	return ToolReply;
}

FReply SCurveEditorViewContainer::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bCaughtMouseDown = true;

	bIsScrubbingTime = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FCurveEditorCommands::Get().ScrubTime);

	if (bIsScrubbingTime && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	if (CurveEditor->GetCurrentTool())
	{
		return CurveEditor->GetCurrentTool()->OnMouseButtonDown(AsShared(), MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SCurveEditorViewContainer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsScrubbingTime = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FCurveEditorCommands::Get().ScrubTime);

	FVector2D MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	// Marquee Selection or time scrub
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsScrubbingTime)
		{
			UE::CurveEditor::InputState::SetCommandPressed(FCurveEditorCommands::Get().ScrubTime, MouseEvent.GetModifierKeys(), true);

			DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
			DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_ScrubTime>(CurveEditor.Get());
		}
		else
		{
			DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
			DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_Marquee>(CurveEditor.Get());
		}
		const FVector2D InitialPosition = DragOperation->GetInitialPosition();
		DragOperation->DragImpl->MouseButtonDown(InitialPosition, MyGeometry, MouseEvent);
		return FReply::Handled();
	}
	// Alt RMB always Zoom Timeline
	else if (MouseEvent.IsAltDown() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
		DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_Zoom>(CurveEditor.Get(), nullptr);
		const FVector2D InitialPosition = DragOperation->GetInitialPosition();
		DragOperation->DragImpl->MouseButtonDown(InitialPosition, MyGeometry, MouseEvent);
		return FReply::Handled();
	}
	// Pan Timeline
	else if (CurveEditor->GetSettings()->AllowMousePan(MouseEvent))
	{
		DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
		DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_PanInput>(CurveEditor.Get());
		const FVector2D InitialPosition = DragOperation->GetInitialPosition();
		DragOperation->DragImpl->MouseButtonDown(InitialPosition, MyGeometry, MouseEvent);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SCurveEditorViewContainer::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (CurveEditor->GetCurrentTool())
	{
		return CurveEditor->GetCurrentTool()->OnMouseButtonDoubleClick(AsShared(), InMyGeometry, InMouseEvent);
	}

	return FReply::Unhandled();
}

FReply SCurveEditorViewContainer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2D MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	FReply ToolReply = FReply::Unhandled();
	bHadMouseMovesThisTick = true;
	
	if (CurveEditor->GetCurrentTool())
	{
		ToolReply = CurveEditor->GetCurrentTool()->OnMouseMove(AsShared(), MyGeometry, MouseEvent);
	}

	if (!ToolReply.IsEventHandled() && DragOperation.IsSet())
	{
		FVector2D InitialPosition = DragOperation->GetInitialPosition();

		if (!DragOperation->IsDragging() && DragOperation->AttemptDragStart(MouseEvent))
		{
			DragOperation->DragImpl->BeginDrag(InitialPosition, MousePixel, MouseEvent);
			return FReply::Handled().CaptureMouse(AsShared());
		}
		else if (DragOperation->IsDragging())
		{
			DragOperation->DragImpl->Drag(InitialPosition, MousePixel, MouseEvent);
			return FReply::Handled();
		}
	}

	return ToolReply;
}

void SCurveEditorViewContainer::OnFinishedPointerInput()
{
	if (!bHadMouseMovesThisTick)
	{
		return;
	}
	// Some tools & operations defer processing Drag calls for performance reasons. Give them a chance to process the accumulated input.
	
	if (CurveEditor->GetCurrentTool())
	{
		CurveEditor->GetCurrentTool()->OnFinishedPointerInput();
	}
	if (DragOperation && DragOperation->IsDragging())
	{
		DragOperation->DragImpl->FinishedPointerInput();
	}
	
	bHadMouseMovesThisTick = false;
}

FReply SCurveEditorViewContainer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2D MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	FReply Reply = FReply::Unhandled();
	if (DragOperation.IsSet())
	{
		DragOperation->DragImpl->MouseButtonUp(MyGeometry, MouseEvent);
	}
	if (!Reply.IsEventHandled() && bIsScrubbingTime && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Reply = FReply::Handled();
	}
	if (DragOperation.IsSet() && DragOperation->IsDragging())
	{
		FVector2D InitialPosition = DragOperation->GetInitialPosition();
		DragOperation->DragImpl->EndDrag(InitialPosition, MousePosition, MouseEvent);
		Reply = FReply::Handled();
	}
	else if (CurveEditor->GetCurrentTool())
	{
		Reply = CurveEditor->GetCurrentTool()->OnMouseButtonUp(AsShared(), MyGeometry, MouseEvent);
	}

	if (!Reply.IsEventHandled() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bCaughtMouseDown)
	{
		if (!MouseEvent.IsShiftDown() && !MouseEvent.IsAltDown() && !MouseEvent.IsControlDown())
		{
			const UE::CurveEditor::FScopedSelectionChange Transaction(CurveEditor, LOCTEXT("ClearSelection", "Clear selection"));
			CurveEditor->GetSelection().Clear();
			Reply = FReply::Handled();
		}
	}

	bCaughtMouseDown = false;
	DragOperation.Reset();
	return Reply.ReleaseMouseCapture();
}

FReply SCurveEditorViewContainer::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TimeSliderController)
	{
		const FReply Reply = TimeSliderController->OnTimeSliderMouseWheel(*this, MyGeometry, MouseEvent);
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}
	return FReply::Unhandled();
}

void SCurveEditorViewContainer::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
#if WITH_EDITOR
	if (!DragOperation.IsSet())
	{
		return;
	}
	GEditor->GetTimerManager()->SetTimerForNextTick([this]()
		{
			if (!HasMouseCapture())
			{
				DragOperation->Cancel();
				DragOperation.Reset();
			}
		});
#endif
}
FCursorReply SCurveEditorViewContainer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	const bool bIsScrubCommand = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FCurveEditorCommands::Get().ScrubTime);
	if (bIsScrubCommand)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	// Override the cursor while the Marquee select is happening. Ideally we'd be able to do a "+" and "-" while holding
	// shift and alt, but there's not built in OS cursor to do this.
	if (DragOperation.IsSet() && DragOperation->IsDragging())
	{
		return FCursorReply::Cursor(EMouseCursor::Crosshairs);
	}

	return FCursorReply::Unhandled();
}


void SCurveEditorViewContainer::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	bIsScrubbingTime = false;

	if (DragOperation.IsSet())
	{
		DragOperation->Cancel();
		DragOperation.Reset();
	}

	if (CurveEditor->GetCurrentTool())
	{
		CurveEditor->GetCurrentTool()->OnFocusLost(InFocusEvent);
	}

	SVerticalBox::OnFocusLost(InFocusEvent);
}

void SCurveEditorViewContainer::OnCurveEditorToolChanged(FCurveEditorToolID InToolId)
{
	// We need to end drag-drop operations if they switch tools. Otherwise they can start
	// a marquee select, use the keyboard to switch to a diferent tool, and then the marquee
	// select finishes after the tool has had a chance to activate.
	if (DragOperation.IsSet())
	{
		// We have to cancel it instead of ending it because ending it needs mouse position and some other stuff.
		DragOperation->Cancel();
		DragOperation.Reset();
	}
}

void SCurveEditorViewContainer::ExpandInputBounds(float NewWidth)
{
	const float OldWidth = GetCachedGeometry().GetLocalSize().X;
	if (NewWidth != OldWidth && OldWidth > 0)
	{
		// Retrieve the current bounds and cache them
		double InputMin = 0.0, InputMax = 1.0;
		CurveEditor->GetBounds().GetInputBounds(InputMin, InputMax);

		// Increase the visible input/output ranges based on the new size of the panel
		const double PixelToInputRatio = (InputMax - InputMin) / OldWidth;
		InputMax += PixelToInputRatio * (NewWidth - OldWidth);

		CurveEditor->GetBounds().SetInputBounds(InputMin, InputMax);
	}
}

FMargin SCurveEditorViewContainer::GetSlotPadding(int32 SlotIndex) const
{
	const bool bIsFirstView = SlotIndex == 0;
	const bool bIsLastView  = SlotIndex == Views.Num()-1;

	return FMargin(0.f, bIsFirstView ? 0.f : 5.f, 0.f, bIsLastView ? 0.f : 5.f);
}

void SCurveEditorViewContainer::AddView(TSharedRef<SCurveEditorView> ViewToAdd)
{
	const int32 InsertIndex = Views.Num();
	ViewToAdd->RelativeOrder = Views.Num();

	Views.Add(ViewToAdd);
	SVerticalBox::FSlot* SlotPointer = nullptr;
	AddSlot()
		.Expose(SlotPointer)
		[
		SAssignNew(RetainerWidget, SRetainerWidget)
		.RenderOnPhase(false)
		.RenderOnInvalidation(false)
		.bWarnOnInvalidSize(false)
		[
			SNew(SBox)
			.Padding(MakeAttributeSP(this, &SCurveEditorViewContainer::GetSlotPadding, InsertIndex))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				ViewToAdd
			]
		]
	];

	if (ViewToAdd->ShouldAutoSize())
	{
		SlotPointer->SetAutoHeight();
	}

	ViewToAdd->SetRetainerWidget(RetainerWidget);

	ViewToAdd->OnCurveHasChangedExternally().AddRaw(this, &SCurveEditorViewContainer::BroadcastOnCurveHasChangedExternally);
}

void SCurveEditorViewContainer::Clear()
{
	for (const TSharedPtr<SCurveEditorView>& View : Views)
	{
		View->OnCurveHasChangedExternally().RemoveAll(this);
	}
	
	Views.Reset();
	ClearChildren();
}


#undef LOCTEXT_NAMESPACE
