// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorCurveDrawParamsCache.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "ICurveEditorDragOperation.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class ITimeSliderController;
class SCurveEditorView;
class SRetainerWidget;
struct FCurveEditorToolID;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

/**
 * Curve editor widget that reflects the state of an FCurveEditor
 */
class SCurveEditorViewContainer : public SVerticalBox
{
	SLATE_BEGIN_ARGS(SCurveEditorViewContainer)
		: _MinimumPanelHeight(300.0f)
	{}

		/** Optional Time Slider Controller which allows us to synchronize with an externally controlled Time Slider */
		SLATE_ARGUMENT(TSharedPtr<ITimeSliderController>, ExternalTimeSliderController)
		/** The minimum height for this container panel. */
		SLATE_ARGUMENT(float, MinimumPanelHeight)

	SLATE_END_ARGS()

	/**
	 * Construct a new curve editor panel widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor);

	TArrayView<const TSharedPtr<SCurveEditorView>> GetViews() const { return Views; }

	UE_API void AddView(TSharedRef<SCurveEditorView> ViewToAdd);

	UE_API void Clear();

	/** Invoked when any of the views has invoked FCurveModel::HasChangedAndResetTest, and it returned true. */
	UE::CurveEditor::FOnCurveHasChangedExternally& OnCurveHasChangedExternally() { return OnCurveHasChangedExternallyDelegate; }

private:

	// SWidget Interface
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const;
	UE_API virtual bool ComputeVolatility() const override;
	// ~SWidget Interface

	/*~ Mouse interaction */
	UE_API virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnFinishedPointerInput() override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	UE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	virtual bool SupportsKeyboardFocus() const override { return true; }
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	UE_API FMargin GetSlotPadding(int32 SlotIndex) const;

	UE_API void OnCurveEditorToolChanged(FCurveEditorToolID InToolId);

	UE_API void ExpandInputBounds(float NewWidth);

	UE_API bool IsScrubTimeKeyEvent(const FKeyEvent& InKeyEvent);

	void BroadcastOnCurveHasChangedExternally(const FCurveModelID& InCurveModelId) const;

private:

	/** The curve editor pointer */
	TSharedPtr<FCurveEditor> CurveEditor;

	/** Optional time slider controller */
	TSharedPtr<ITimeSliderController> TimeSliderController;

	/** (Optional) the current drag operation */
	TOptional<FCurveEditorDelayedDrag> DragOperation;
	/** Whether DragOperation needs to have OnFinishedPointerInput called this tick. */
	bool bHadMouseMovesThisTick = false;

	/** Array of views whose cache may need to be updated and may need their height updating on tick. */
	TArray<TSharedPtr<SCurveEditorView>> Views;

	/** Possible pointer to a retainer widget that we may need to force update*/
	TSharedPtr<SRetainerWidget> RetainerWidget;

	/** Invoked when any of the views has invoked FCurveModel::HasChangedAndResetTest, and it returned true. */
	UE::CurveEditor::FOnCurveHasChangedExternally OnCurveHasChangedExternallyDelegate;

	/** 
	 * Whether or not this widget caught an OnMouseDown notification 
	 * Used to check if the selection should be cleared
	 */
	bool bCaughtMouseDown;

	/** The minimum height for this container panel. **/
	float MinimumPanelHeight;

	/** Whether or not we are scrubbing time*/
	bool bIsScrubbingTime = false;
};

#undef UE_API
