// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveKeyDetailPanel.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"

class FCurveEditor;

#define LOCTEXT_NAMESPACE "SCurveEditorPanel"

void SCurveKeyDetailPanel::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
{

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FPropertyRowGeneratorArgs Args;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	// Args.NotifyHook = this;

	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->OnRowsRefreshed().AddSP(this, &SCurveKeyDetailPanel::PropertyRowsRefreshed);

	PropertyRowsRefreshed();
}

// A Dummy editable text box that is visible before property rows are generated.
class STempConstrainedBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STempConstrainedBox)
		: _MinWidth(125.f)
		, _MaxWidth(125.f)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		MinWidth = InArgs._MinWidth;
		MaxWidth = InArgs._MaxWidth;

		ChildSlot
			[
				SNew(SEditableTextBox)
			];
	};

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		const float MinWidthVal = MinWidth.Get().Get(0.0f);
		const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

		if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
		{
			return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
		}
		else
		{
			FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

			float XVal = FMath::Max(MinWidthVal, ChildSize.X);
			if (MaxWidthVal >= MinWidthVal)
			{
				XVal = FMath::Min(MaxWidthVal, XVal);
			}

			return FVector2D(XVal, ChildSize.Y);
		}
	}

private:
	TAttribute< TOptional<float> > MinWidth;
	TAttribute< TOptional<float> > MaxWidth;
};


void SCurveKeyDetailPanel::PropertyRowsRefreshed()
{
	TSharedPtr<SWidget> TimeWidget = nullptr;
	TSharedPtr<SWidget> ValueWidget = nullptr;

	for (TSharedRef<IDetailTreeNode> RootNode : PropertyRowGenerator->GetRootTreeNodes())
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		RootNode->GetChildren(Children);

		for (TSharedRef<IDetailTreeNode> Child : Children)
		{
			TArray<TSharedRef<IDetailTreeNode>> SubChildren;
			Child->GetChildren(SubChildren);

			if (!TimeWidget.IsValid() && Child->GetNodeName() == TEXT("Time"))
			{
				FNodeWidgets NodeWidgets = Child->CreateNodeWidgets();
				TimeWidget = NodeWidgets.ValueWidget;
			}
			else if (!ValueWidget.IsValid() && Child->GetNodeName() == TEXT("Value"))
			{
				FNodeWidgets NodeWidgets = Child->CreateNodeWidgets();
				ValueWidget = NodeWidgets.ValueWidget;
			}
		}
	}

	// If either Time or Value were not found, use this ugly temporary hack until PropertyRowGenerator returns names for customized properties. This uses the first
	// two fields on the object instead of looking for "Time" and "Value". :(
	if (!TimeWidget || !ValueWidget)
	{
		TimeWidget = nullptr;
		ValueWidget = nullptr;

		for (TSharedRef<IDetailTreeNode> RootNode : PropertyRowGenerator->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			RootNode->GetChildren(Children);

			for (TSharedRef<IDetailTreeNode> Child : Children)
			{
				if (!TimeWidget.IsValid())
				{
					FNodeWidgets NodeWidgets = Child->CreateNodeWidgets();
					TimeWidget = NodeWidgets.ValueWidget;
				}
				else if (!ValueWidget.IsValid())
				{
					FNodeWidgets NodeWidgets = Child->CreateNodeWidgets();
					ValueWidget = NodeWidgets.ValueWidget;
				}
			}
		}
	}

	if (!TimeWidget)
	{
		if (!TempTimeWidget.IsValid())
		{
			TempTimeWidget = SNew(STempConstrainedBox);
		}

		TimeWidget = TempTimeWidget;
	}

	if (!ValueWidget)
	{
		if (!TempValueWidget.IsValid())
		{
			TempValueWidget = SNew(STempConstrainedBox);
		}

		ValueWidget = TempValueWidget;
	}

	if (TimeWidget && ValueWidget)
	{
		ConstructChildLayout(TimeWidget, ValueWidget);
	}
}

void SCurveKeyDetailPanel::ConstructChildLayout(TSharedPtr<SWidget> TimeWidget, TSharedPtr<SWidget> ValueWidget)
{
	check(TimeWidget && ValueWidget);
	TimeWidget->SetToolTipText(LOCTEXT("TimeEditBoxTooltip", "The time of the selected key(s)"));
	ValueWidget->SetToolTipText(LOCTEXT("ValueEditBoxTooltip", "The value of the selected key(s)"));

	ChildSlot
	[
		SNew(SBox)
		.VAlign(VAlign_Fill)
		.MaxDesiredWidth(FAppStyle::Get().GetFloat("CurveEditor.KeyDetailWidth"))
		[
			SNew(SHorizontalBox)

			// "Time" Edit box
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.Padding(4.f, 0.f, 0.f, 0.f)
			.FillWidth(0.5f)
			[
				TimeWidget.ToSharedRef()
			]


			// "Value" Edit box
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.Padding(4.f, 0.f, 0.f, 0.f)
			.FillWidth(0.5f)
			[
				ValueWidget.ToSharedRef()
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE // "SCurveEditorPanel"