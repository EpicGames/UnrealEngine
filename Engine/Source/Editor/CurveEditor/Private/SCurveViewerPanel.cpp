// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveViewerPanel.h"

#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorCurveDrawParamsCache.h"
#include "CurveEditorTypes.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Rendering/DrawElements.h"
#include "Templates/Less.h"
#include "Templates/UniquePtr.h"
#include "Views/SInteractiveCurveEditorView.h"

class FCurveModel;
class FPaintArgs;
class FSlateRect;
class FWidgetStyle;

#define LOCTEXT_NAMESPACE "SCurveViewerPanel"

namespace CurveViewerConstants
{
	static bool  bAntiAliasCurves = true;
}

void SCurveViewerPanel::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
{
	InitCurveEditorReference(InCurveEditor);
	CurveThickness = InArgs._CurveThickness;

	InCurveEditor->SetView(SharedThis(this));

	CurveDrawParamsCache->Invalidate(SharedThis(this));

	for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : InCurveEditor->GetCurves())
	{
		CurveInfoByID.Add(CurvePair.Key);
	}

	SetClipping(EWidgetClipping::ClipToBounds);
}

void SCurveViewerPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CheckCacheAndInvalidateIfNeeded();
}

int32 SCurveViewerPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	DrawCurves(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, DrawEffects);

	return LayerId + CurveViewConstants::ELayerOffset::Last;
}

void SCurveViewerPanel::DrawCurves(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const
{
	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

	float DashOffset = static_cast<float>(GetViewSpace().PixelsPerInput() * GetViewSpace().GetInputMin());

	for (const FCurveDrawParams& Params : CurveDrawParamsCache->GetCurveDrawParams())
	{
		if (Params.DashLengthPx > 0.f)
		{
			TArray<FVector2f> NewVector;
			NewVector.Reserve(Params.InterpolatingPoints.Num());
			for (const FVector2d& Vect : Params.InterpolatingPoints)
			{
				NewVector.Add(UE::Slate::CastToVector2f(Vect));
			}

			FSlateDrawElement::MakeDashedLines(
				OutDrawElements,
				BaseLayerId + CurveViewConstants::ELayerOffset::Curves,
				PaintGeometry,
				MoveTemp(NewVector),
				DrawEffects,
				Params.Color,
				CurveThickness.Get(),
				Params.DashLengthPx,
				DashOffset
			);
		}
		else
		{
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				BaseLayerId + CurveViewConstants::ELayerOffset::Curves,
				PaintGeometry,
				Params.InterpolatingPoints,
				DrawEffects,
				Params.Color,
				CurveViewerConstants::bAntiAliasCurves,
				CurveThickness.Get()
			);
		}

		if (Params.bKeyDrawEnabled)
		{
			for (int32 PointIndex = 0; PointIndex < Params.Points.Num(); PointIndex++)
			{
				const FCurvePointInfo& Point = Params.Points[PointIndex];
				const FKeyDrawInfo& PointDrawInfo = Params.GetKeyDrawInfo(Point.Type, PointIndex);

				const int32 KeyLayerId = BaseLayerId + Point.LayerBias + CurveViewConstants::ELayerOffset::Keys;
				FLinearColor PointTint = PointDrawInfo.Tint.IsSet() ? PointDrawInfo.Tint.GetValue() : Params.Color;

				// Brighten and saturate the points a bit so they pop
				FLinearColor HSV = PointTint.LinearRGBToHSV();
				HSV.G = FMath::Clamp(HSV.G * 1.1f, 0.f, 255.f);
				HSV.B = FMath::Clamp(HSV.B * 2.f, 0.f, 255.f);
				PointTint = HSV.HSVToLinearRGB();

				FPaintGeometry PointGeometry = AllottedGeometry.ToPaintGeometry(
					PointDrawInfo.ScreenSize,
					FSlateLayoutTransform(Point.ScreenPosition - (PointDrawInfo.ScreenSize * 0.5f))
				);

				FSlateDrawElement::MakeBox(OutDrawElements, KeyLayerId, PointGeometry, PointDrawInfo.Brush, DrawEffects, PointTint );
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE