// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/ScopedSelectionChangeEventSuppression.h"

#include "CurveEditor.h"
#include "CurveEditorSelectionPrivate.h"

namespace UE::CurveEditor
{
FScopedSelectionChangeEventSuppression::FScopedSelectionChangeEventSuppression(const TSharedRef<FCurveEditor>& InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
{
	KeySelection::FIncrementOnSelectionChangedSuppressionCount(InCurveEditor->Selection);
}

FScopedSelectionChangeEventSuppression::~FScopedSelectionChangeEventSuppression()
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		KeySelection::FDecrementOnSelectionChangedSuppressionCount(CurveEditorPin->Selection);
	}
}
}
