// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionCleanser.h"

#include "CurveEditor.h"
#include "Editor.h"
#include "SCurveEditor.h"
#include "SelectionUtils.h"
#include "Modification/Utils/ScopedSelectionChange.h"

namespace UE::CurveEditor
{
FSelectionCleanser::FSelectionCleanser(const TSharedRef<FCurveEditor>& InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
{
	if (FTransactionManager* TransactionManager = InCurveEditor->GetTransactionManager(); ensure(TransactionManager))
	{
		TransactionManager->OnCurvesChanged().AddRaw(this, &FSelectionCleanser::OnCurvesChanged);
	}

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FSelectionCleanser::~FSelectionCleanser()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
	
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}
	
	if (FTransactionManager* TransactionManager = CurveEditorPin->GetTransactionManager())
	{
		TransactionManager->OnCurvesChanged().RemoveAll(this);
	}
}

void FSelectionCleanser::PostUndo(bool bSuccess)
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		// If you create keys and then undo them, they may remain in the selection set.
		// Code that mutates keys should handle this case by using FScopedCurveChange & FScopedSelectionChange, but we'll clean up the selection
		// anyway just in case.
		// Stale keys presents issues with context menus and other things that are activated when there is a selection set.
		// To fix this, we have to loop through all of our curve models, and re-select only the key handles that were
		// previously selected that still exist. Ugly, but reasonably functional.
		const bool bHadStaleKeys = CleanseSelection(CurveEditorPin.ToSharedRef(), CurveEditorPin->Selection) == ECleanseResult::HadStaleKeys;
		UE_CLOGF(bHadStaleKeys, LogCurveEditor, Warning, "Curve Editor had stale keys after undo & redo. Investigate.");
	}
}

void FSelectionCleanser::OnCurvesChanged(const FGenericCurveChangeData& InChange) const
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin || !InChange.RemoveKeysData.HasChanges())
	{
		return;
	}

	const FScopedSelectionChange SelectionChange(CurveEditorPin);
	for (const TPair<FCurveModelID, FCurveKeyData>& Pair : InChange.RemoveKeysData.SavedCurveState)
	{
		CleanseSelection(CurveEditorPin.ToSharedRef(), CurveEditorPin->Selection, MakeConstArrayView({ Pair.Key }));
	}
}
}
