// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Modification/Keys/Diff/AppendPerCurveGenericChangeVisitor.h"
#include "Templates/UnrealTemplate.h"

class FCurveEditor;

namespace UE::CurveEditor
{
/**
 * Makes sure that the FCurveEditor::Selection does not contain any stale keys, i.e. to prevent that the selection contains FKeyHandles and
 * FCurveModelIds that do not exist in the FCurveEditor.
 * 
 * Stale keys presents issues with context menus and other things that are activated when there is a selection set.
 *
 * Cleansing happens when:
 * - An undo / redo operation is performed (regardless whether it directly affects FCurveEditor - the data underlying the FCurveModel may change)
 * - An undoable action is recorded to FTransactionManager.
 * Effectively, this means that if you use FScopedCurveChange to make changes to curves, this manager will handle cleansing selection.
 */
class FSelectionCleanser
	: public FNoncopyable
	, public FEditorUndoClient
{
public:
	
	explicit FSelectionCleanser(const TSharedRef<FCurveEditor>& InCurveEditor);
	~FSelectionCleanser();

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	//~ End FEditorUndoClient Interface

private:

	/** Instance that owns us. Used to check which curves are currently present. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** Invoked when a (transactional) change happens to curves. */
	void OnCurvesChanged(const FGenericCurveChangeData& InChange) const;
};
}

