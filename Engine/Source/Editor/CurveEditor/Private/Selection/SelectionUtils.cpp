// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionUtils.h"

#include "CurveEditor.h"
#include "CurveEditorTrace.h"
#include "ITimeSlider.h"
#include "SCurveEditor.h"
#include "Selection/ScopedSelectionChangeEventSuppression.h"

namespace UE::CurveEditor
{
namespace SelectionUtilsPrivate
{
static bool CleanseSingle(
	const TSharedRef<FCurveEditor>& InCurveEditor, FCurveEditorSelection& InSelection,
	const FCurveModelID& InModelId, const FKeyHandleSet& InSet
	)
{
	SCOPED_CURVE_EDITOR_TRACE(CleanseSingle);
	
	FCurveModel* CurveModel = InCurveEditor->FindCurve(InModelId);

	// If the entire curve was removed, just dump that out of the selection set.
	if (!CurveModel)
	{
		InSelection.Remove(InModelId);
		return true;
	}
	// Get all of the key handles from this curve.
	const TArray<FKeyHandle> KeyHandles = CurveModel->GetAllKeys();

	// The set handles will be mutated as we remove things so we need a copy that we can iterate through.
	const TConstArrayView<FKeyHandle> SelectedHandles = InSet.AsArray();
	const TArray<FKeyHandle> NonMutableArray(SelectedHandles.GetData(), SelectedHandles.Num());

	bool bContainedStaleKeys = false;
	for (const FKeyHandle& Handle : NonMutableArray)
	{
		// Check to see if our curve model contains this handle still.
		if (!KeyHandles.Contains(Handle))
		{
			InSelection.Remove(InModelId, ECurvePointType::Key, Handle);
			bContainedStaleKeys = true;
		}
	}
	return bContainedStaleKeys;
}
}
	
ECleanseResult CleanseSelection(
	const TSharedRef<FCurveEditor>& InCurveEditor, FCurveEditorSelection& InSelection
	)
{
	SCOPED_CURVE_EDITOR_TRACE(CleanseSelection);
	// Minor optimization to reduce selection changed calls
	const FScopedSelectionChangeEventSuppression SuppressSelectionEvents(InCurveEditor);

	bool bHadStaleCurvesOrKeys = false;
	TMap<FCurveModelID, FKeyHandleSet> SelectionSet = InSelection.GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Set : SelectionSet)
	{
		bHadStaleCurvesOrKeys |= SelectionUtilsPrivate::CleanseSingle(InCurveEditor, InSelection, Set.Key, Set.Value);
	}

	return bHadStaleCurvesOrKeys ? ECleanseResult::HadStaleKeys : ECleanseResult::NoStaleKeys;
}

ECleanseResult CleanseSelection(
	const TSharedRef<FCurveEditor>& InCurveEditor, FCurveEditorSelection& InSelection, TConstArrayView<FCurveModelID> InOnlyTheseCurves
	)
{
	SCOPED_CURVE_EDITOR_TRACE(CleanseSelection);
	// Minor optimization to reduce selection changed calls
	const FScopedSelectionChangeEventSuppression SuppressSelectionEvents(InCurveEditor);
	
	bool bHadStaleCurvesOrKeys = false;
	for (const FCurveModelID& CurveId : InOnlyTheseCurves)
	{
		if (const FKeyHandleSet* Set = InSelection.FindForCurve(CurveId))
		{
			bHadStaleCurvesOrKeys |= SelectionUtilsPrivate::CleanseSingle(InCurveEditor, InSelection, CurveId, *Set);
		}
	}
	
	return bHadStaleCurvesOrKeys ? ECleanseResult::HadStaleKeys : ECleanseResult::NoStaleKeys;
}

TMap<FCurveModelID, FKeyHandleSet> ResolveSelectedOrRelativeKeys(const FCurveEditor& InCurveEditor)
{
	TMap<FCurveModelID, FKeyHandleSet> CurvesToKeys = InCurveEditor.GetSelection().GetAll();
	if (!CurvesToKeys.IsEmpty())
	{
		return CurvesToKeys;
	}

	const TSharedPtr<ITimeSliderController> TimeSliderController = InCurveEditor.GetTimeSliderController();
	if (!TimeSliderController.IsValid())
	{
		return CurvesToKeys;
	}

	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const double CurrentTime = TickResolution.AsSeconds(TimeSliderController->GetScrubPosition());

	for (const FCurveModelID CurveID : InCurveEditor.GetEditedCurves())
	{
		const FCurveModel* const Curve = InCurveEditor.FindCurve(CurveID);
		if (!Curve)
		{
			continue;
		}

		const TArray<FKeyHandle> AllKeyHandles = Curve->GetAllKeys();
		if (AllKeyHandles.IsEmpty())
		{
			continue;
		}

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNum(AllKeyHandles.Num());
		Curve->GetKeyPositions(AllKeyHandles, KeyPositions);

		TArray<FKeyHandle> RelativeKeyHandles;

		for (int32 KeyIndex = 0; KeyIndex < KeyPositions.Num(); ++KeyIndex)
		{
			if (KeyPositions[KeyIndex].InputValue > CurrentTime)
			{
				RelativeKeyHandles.Add(AllKeyHandles[KeyIndex]);
			}
		}

		if (!RelativeKeyHandles.IsEmpty())
		{
			FKeyHandleSet& KeyHandleSet = CurvesToKeys.Add(CurveID);
			KeyHandleSet.Replace(RelativeKeyHandles, ECurvePointType::Key);
		}
	}

	return CurvesToKeys;
}

} // namespace UE::CurveEditor
