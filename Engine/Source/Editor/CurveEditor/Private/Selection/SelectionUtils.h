// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "CurveEditorSelection.h"
#include "Templates/SharedPointerFwd.h"

struct FCurveModelID;
struct FCurveEditorSelection;
class FCurveEditor;

namespace UE::CurveEditor
{
enum class ECleanseResult : uint8 { NoStaleKeys, HadStaleKeys };
	
/** Removes all curve models and keys from InSelection that do not exist in InCurveEditor. */
ECleanseResult CleanseSelection(const TSharedRef<FCurveEditor>& InCurveEditor, FCurveEditorSelection& InSelection);

/** Overload that will only remove 1. those FKeyHandles belonging to curves in InOnlyTheseCurves, 2. the curves in InOnlyTheseCurves. */
ECleanseResult CleanseSelection(const TSharedRef<FCurveEditor>& InCurveEditor, FCurveEditorSelection& InSelection,
	TConstArrayView<FCurveModelID> InOnlyTheseCurves
	);

/** Returns the explicit key selection, or keys after the current scrub time if nothing is selected. */
TMap<FCurveModelID, FKeyHandleSet> ResolveSelectedOrRelativeKeys(const FCurveEditor& InCurveEditor);

}
