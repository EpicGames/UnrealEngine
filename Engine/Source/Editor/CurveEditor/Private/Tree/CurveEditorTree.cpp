// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/CurveEditorTree.h"

#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "CurveEditor.h"
#include "CurveModel.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Less.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Tree/ICurveEditorTreeItem.h"

TArrayView<const FCurveModelID> FCurveEditorTreeItem::GetOrCreateCurves(FCurveEditor* CurveEditor)
{
	if (Curves.Num() == 0)
	{
		TSharedPtr<ICurveEditorTreeItem> ItemPtr = GetItem();
		if (ItemPtr.IsValid())
		{
			TArray<TUniquePtr<FCurveModel>> NewCurveModels;
			ItemPtr->CreateCurveModels(NewCurveModels);

			for (TUniquePtr<FCurveModel>& NewCurve : NewCurveModels)
			{
				FCurveModelID NewModelID = CurveEditor->AddCurveForTreeItem(MoveTemp(NewCurve), ThisID);
				Curves.Add(NewModelID);
			}
		}
	}
	else
	{
		for (const FCurveModelID& ID : Curves)
		{
			FCurveModel* CurveModel = CurveEditor->FindCurve(ID);
			if (CurveModel)
			{
				CurveEditor->BroadcastCurveChanged(CurveModel);
			}
		}
	}
	return Curves;
}

void FCurveEditorTreeItem::DestroyCurves(FCurveEditor* CurveEditor)
{
	for (FCurveModelID CurveID : Curves)
	{
		// Remove the curve from the curve editor
		CurveEditor->RemoveCurve(CurveID);
	}
	Curves.Empty();
}

void FCurveEditorTreeItem::DestroyUnpinnedCurves(FCurveEditor* CurveEditor)
{
	for (int32 Index = Curves.Num()-1; Index >= 0; --Index)
	{
		// Remove the curve from the curve editor
		if (!CurveEditor->IsCurvePinned(Curves[Index]))
		{
			CurveEditor->RemoveCurve(Curves[Index]);
			Curves.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
}

void FCurveEditorTreeDelegate::Broadcast()
{
	if (ensureAlwaysMsgf(!bBroadcasting, TEXT("Attempting to broadcast changes while already broadcasting changes. This may be an infinite loop and is not allowed. Do not attempt to mutate the tree in response to it being mutated")))
	{
		TGuardValue<bool> ScopeGuard(bBroadcasting, true);
		FSimpleMulticastDelegate::Broadcast();
	}
}

FScopedCurveEditorTreeEventGuard::FScopedCurveEditorTreeEventGuard(FCurveEditorTree* InTree)
	: Tree(InTree)
{
	CachedItemSerialNumber = Tree->Events.OnItemsChanged.SerialNumber;
	CachedSelectionSerialNumber = Tree->Events.OnSelectionChanged.SerialNumber;
	CachedFiltersSerialNumber = Tree->Events.OnFiltersChanged.SerialNumber;
	Tree->Events.UpdateGuardCounter += 1;
}

FScopedCurveEditorTreeEventGuard::FScopedCurveEditorTreeEventGuard(FScopedCurveEditorTreeEventGuard&& RHS)
	: Tree(RHS.Tree)
{
	Tree->Events.UpdateGuardCounter += 1;
}

FScopedCurveEditorTreeEventGuard& FScopedCurveEditorTreeEventGuard::operator=(FScopedCurveEditorTreeEventGuard&& RHS)
{
	if (&RHS != this)
	{
		Tree = RHS.Tree;
		Tree->Events.UpdateGuardCounter += 1;
	}
	return *this;
}

FScopedCurveEditorTreeEventGuard::~FScopedCurveEditorTreeEventGuard()
{
	// Before we decrement the UpdateGuardCounter, we check to see whether a re-filter is required so that
	// the re-filter operation itself remains guarded from causing subsequent updates
	const bool bRequiresFilter = (Tree->Events.OnItemsChanged.SerialNumber != CachedItemSerialNumber) || (Tree->Events.OnFiltersChanged.SerialNumber != CachedFiltersSerialNumber);
	if (bRequiresFilter && Tree->Events.UpdateGuardCounter == 1)
	{
		Tree->RunFilters();
	}

	if (--Tree->Events.UpdateGuardCounter == 0)
	{
		if (Tree->Events.OnItemsChanged.SerialNumber != CachedItemSerialNumber)
		{
			Tree->SortTreeItems();
			Tree->Compact();
			Tree->Events.OnItemsChanged.Broadcast();
		}

		if (Tree->Events.OnSelectionChanged.SerialNumber != CachedSelectionSerialNumber)
		{
			Tree->Events.OnSelectionChanged.Broadcast();
		}

		if (Tree->Events.OnFiltersChanged.SerialNumber != CachedFiltersSerialNumber)
		{
			Tree->Events.OnFiltersChanged.Broadcast();
		}
	}
}

FCurveEditorTree::FCurveEditorTree()
{
	NextTreeItemID.Value = 1;
	bIsDoingDirectSelection = false;
}

FCurveEditorTreeItem& FCurveEditorTree::GetItem(FCurveEditorTreeItemID ItemID)
{
	return Items.FindChecked(ItemID);
}

const FCurveEditorTreeItem& FCurveEditorTree::GetItem(FCurveEditorTreeItemID ItemID) const
{
	return Items.FindChecked(ItemID);
}

FCurveEditorTreeItem* FCurveEditorTree::FindItem(FCurveEditorTreeItemID ItemID)
{
	return Items.Find(ItemID);
}

const FCurveEditorTreeItem* FCurveEditorTree::FindItem(FCurveEditorTreeItemID ItemID) const
{
	return Items.Find(ItemID);
}

const TArray<FCurveEditorTreeItemID>& FCurveEditorTree::GetRootItems() const
{
	return RootItems.ChildIDs;
}

const TMap<FCurveEditorTreeItemID, FCurveEditorTreeItem>& FCurveEditorTree::GetAllItems() const
{
	return Items;
}

FCurveEditorTreeItem* FCurveEditorTree::AddItem(FCurveEditorTreeItemID ParentID)
{
	const FCurveEditorTreeItemID NewItemID = NextTreeItemID;
	{
		const FScopedCurveEditorTreeEventGuard EventGuard(this);

		FCurveEditorTreeItem& NewItem = Items.Add(NewItemID);
		NewItem.ThisID   = NewItemID;
		NewItem.ParentID = ParentID;

		FSortedCurveEditorTreeItems& ParentContainer = ParentID.IsValid() ? Items.FindChecked(ParentID).Children : RootItems;
		ParentContainer.ChildIDs.Add(NewItemID);
		ParentContainer.bRequiresSort = true;

		++NextTreeItemID.Value;
		++Events.OnItemsChanged.SerialNumber;
	}

	// ~FScopedCurveEditorTreeEventGuard executes FCurveEditorTree::Compact, so NewItem may have been reallocated.
	FCurveEditorTreeItem* Result = Items.Find(NewItemID);
	ensure(Result);
	return Result;
}

void FCurveEditorTree::RemoveItem(FCurveEditorTreeItemID ItemID, FCurveEditor* CurveEditor)
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	FCurveEditorTreeItem* Item = Items.Find(ItemID);
	if (!Item)
	{
		return;
	}

	// Remove the item from its parent
	FSortedCurveEditorTreeItems& ParentContainer = Item->ParentID.IsValid() ? Items.FindChecked(Item->ParentID).Children : RootItems;
	ParentContainer.ChildIDs.Remove(ItemID);

	Item->DestroyCurves(CurveEditor);

	// Item is going away now (and may be reallocated) so move its children into the function
	RemoveChildrenRecursive(MoveTemp(Item->Children.ChildIDs), CurveEditor);

	// Item is no longer valid

	if (Items.Remove(ItemID) != 0)
	{
		++Events.OnItemsChanged.SerialNumber;
	}
	if (Selection.Remove(ItemID) != 0)
	{
		++Events.OnSelectionChanged.SerialNumber;
	}
}

void FCurveEditorTree::RemoveChildrenRecursive(TArray<FCurveEditorTreeItemID>&& LocalChildren, FCurveEditor* CurveEditor)
{
	for (FCurveEditorTreeItemID ChildID : LocalChildren)
	{
		if (FCurveEditorTreeItem* ChildItem = Items.Find(ChildID))
		{
			// Destroy its curves while we know ChildItem is still a valid ptr
			ChildItem->DestroyCurves(CurveEditor);

			RemoveChildrenRecursive(MoveTemp(ChildItem->Children.ChildIDs), CurveEditor);

			if (Items.Remove(ChildID) != 0)
			{
				++Events.OnItemsChanged.SerialNumber;
			}
			if (Selection.Remove(ChildID) != 0)
			{
				++Events.OnSelectionChanged.SerialNumber;
			}
		}
	}
}

void FCurveEditorTree::Compact()
{
	Items.Compact();
	ChildItemIDs.Compact();
}

void FCurveEditorTree::ToggleExpansionState(bool bRecursive)
{
	ToggleExpansionStateDelegate.Broadcast(bRecursive);
}

void FCurveEditorTree::ExpandAllNodes()
{
	ExpandAllNodesDelegate.Broadcast();
}

void FCurveEditorTree::CollapseAllNodes()
{
	CollapseAllNodesDelegate.Broadcast();
}

ECurveEditorTreeFilterState FCurveEditorTree::PerformFilterPass(TArrayView<const FCurveEditorTreeFilter* const> FilterPtrs, TArrayView<const FCurveEditorTreeItemID> ItemsToFilter, ECurveEditorTreeFilterState InheritedState)
{
	ECurveEditorTreeFilterState ReturnedParentState = ECurveEditorTreeFilterState::NoMatch;

	for (FCurveEditorTreeItemID ItemID : ItemsToFilter)
	{
		// If it failed a previous pass, don't consider it for this pass
		if (FilterStates.Get(ItemID) == ECurveEditorTreeFilterState::NoMatch)
		{
			continue;
		}

		const FCurveEditorTreeItem& TreeItem = GetItem(ItemID);

		ECurveEditorTreeFilterState FilterState = InheritedState;
		ECurveEditorTreeFilterState ChildInheritedState = InheritedState;

		TSharedPtr<ICurveEditorTreeItem> TreeItemImpl = TreeItem.GetItem();
		if (TreeItemImpl)
		{
			for (const FCurveEditorTreeFilter* Filter : FilterPtrs)
			{
				const bool bMatchesFilter = TreeItemImpl->PassesFilter(Filter);
				if (bMatchesFilter)
				{
					ReturnedParentState |= ECurveEditorTreeFilterState::ImplicitParent;
					FilterState |= ECurveEditorTreeFilterState::Match;
					ChildInheritedState |= ECurveEditorTreeFilterState::ImplicitChild;

					if (Filter->ShouldExpandOnMatch())
					{
						ReturnedParentState |= ECurveEditorTreeFilterState::Expand;

						// Once an item is matched AND set to expand there's no other flags that could be 
						// discernibly applied, therefore we can early out and skip needlessly testing other filters
						break;
					}
					// Else, we have to keep iterating incase one of the other filters is set to expand
				}
			}
		}

		// Run the filter on all child nodes
		ECurveEditorTreeFilterState AppliedStateFromChild = PerformFilterPass(FilterPtrs, TreeItem.GetChildren(), ChildInheritedState);
		ReturnedParentState |= AppliedStateFromChild;

		// If we matched children we become an implicit parent
		FilterState |= AppliedStateFromChild;

		FilterStates.SetFilterState(ItemID, FilterState);
	}

	return ReturnedParentState;
}

void FCurveEditorTree::RunFilters()
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	// Always reset filter states
	FilterStates.Reset();

	if (WeakFilters.Num() != 0)
	{
		// Gather all valid filters into a map sorted by pass
		TSortedMap<int32, TArray<const FCurveEditorTreeFilter*>> FiltersByPass;

		for (int32 Index = WeakFilters.Num() - 1; Index >= 0; --Index)
		{
			TSharedPtr<FCurveEditorTreeFilter> Filter = WeakFilters[Index].Pin();
			if (!Filter)
			{
				// Remove invalid filters
				WeakFilters.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				FiltersByPass.FindOrAdd(Filter->GetFilterPass()).Add(Filter.Get());
			}
		}

		// Deactivate the filter states for the first pass to ensure that _all_ items are considered for filtering in the very first pass
		FilterStates.Deactivate();

		for (const TTuple<int32, TArray<const FCurveEditorTreeFilter*>>& Pair : FiltersByPass)
		{
			PerformFilterPass(Pair.Value, RootItems.ChildIDs, ECurveEditorTreeFilterState::NoMatch);

			// Activate the filters after the first pass so that subsequent passes only consider items that have previously matched in some way.
			// This ensures that each pass works as a boolean AND.
			FilterStates.Activate();
		}
	}
	else
	{
		FilterStates.Deactivate();
	}

	++Events.OnItemsChanged.SerialNumber;
	++Events.OnFiltersChanged.SerialNumber;
}

void FCurveEditorTree::AddFilter(TWeakPtr<FCurveEditorTreeFilter> NewFilter)
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	WeakFilters.AddUnique(NewFilter);

	// Always broadcast the filter change even if the filter already existed since calling AddFilter indicates a clear intent to have the tree re-filtered
	++Events.OnFiltersChanged.SerialNumber;
}

void FCurveEditorTree::RemoveFilter(TWeakPtr<FCurveEditorTreeFilter> FilterToRemove)
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	WeakFilters.Remove(FilterToRemove);
	// Always broadcast the filter change even if the filter already existed since calling RemoveFilter indicates a clear intent to have the tree re-filtered
	++Events.OnFiltersChanged.SerialNumber;
}

const FCurveEditorTreeFilter* FCurveEditorTree::FindFilterByType(ECurveEditorTreeFilterType Type) const
{
	for (const TWeakPtr<FCurveEditorTreeFilter>& WeakFilter : WeakFilters)
	{
		TSharedPtr<FCurveEditorTreeFilter> Pinned = WeakFilter.Pin();
		if (Pinned && Pinned->GetType() == Type)
		{
			return Pinned.Get();
		}
	}

	return nullptr;
}

TArrayView<const TWeakPtr<FCurveEditorTreeFilter>> FCurveEditorTree::GetFilters() const
{
	return WeakFilters;
}

void FCurveEditorTree::ClearFilters()
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	WeakFilters.Empty();
	++Events.OnFiltersChanged.SerialNumber;
}

const FCurveEditorFilterStates& FCurveEditorTree::GetFilterStates() const
{
	return FilterStates;
}

ECurveEditorTreeFilterState FCurveEditorTree::GetFilterState(FCurveEditorTreeItemID InTreeItemID) const
{
	return FilterStates.Get(InTreeItemID);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FCurveEditorTree::SetSortPredicate(FTreeItemSortPredicate InSortPredicate)
{
	SortPredicate = FCurveTreeItemSortPredicate::CreateLambda(
		[InSortPredicate = MoveTemp(InSortPredicate)](const FCurveEditorTreeItem& ItemA, const FCurveEditorTreeItem& ItemB)
		{
			return InSortPredicate.Execute(ItemA.GetItem().Get(), ItemB.GetItem().Get());
		});
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FCurveEditorTree::SetSortPredicate(FCurveTreeItemSortPredicate InSortPredicate)
{
	SortPredicate = MoveTemp(InSortPredicate);
}

void FCurveEditorTree::SortTreeItems()
{
	if (RootItems.ChildIDs.Num() > 0 && SortPredicate.IsBound())
	{
		SortTreeItems(RootItems);
	}
}

void FCurveEditorTree::SortTreeItems(FSortedCurveEditorTreeItems& TreeItemIDsToSort)
{
	// First collect the items for the IDs.
	TArray<FCurveEditorTreeItem*> TreeItemsToSort;
	for (const FCurveEditorTreeItemID& TreeItemID : TreeItemIDsToSort.ChildIDs)
	{
		TreeItemsToSort.Add(&Items[TreeItemID]);
	}

	// If there is more than one item, sort the items and then repopulate the ChildIDs from the sorted list of items.
	if (TreeItemIDsToSort.bRequiresSort && TreeItemsToSort.Num() > 1)
	{
		TreeItemsToSort.Sort([this](const FCurveEditorTreeItem& ItemA, const FCurveEditorTreeItem& ItemB)
		{
			return SortPredicate.Execute(ItemA, ItemB);
		});
		for (int32 i = 0; i < TreeItemsToSort.Num(); ++i)
		{
			TreeItemIDsToSort.ChildIDs[i] = TreeItemsToSort[i]->GetID();
		}
	}
	TreeItemIDsToSort.bRequiresSort = false;

	// Sort the children of the child items.
	for (FCurveEditorTreeItem* TreeItemToSort : TreeItemsToSort)
	{
		if (TreeItemToSort != nullptr && TreeItemToSort->Children.ChildIDs.Num() != 0)
		{
			SortTreeItems(TreeItemToSort->Children);
		}
	}
}

void FCurveEditorTree::SetDirectSelection(TArray<FCurveEditorTreeItemID>&& TreeItems, FCurveEditor* InCurveEditor)
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);
	TGuardValue<bool> Guard(bIsDoingDirectSelection, true);

	TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> PreviousSelection = MoveTemp(Selection);
	Selection.Reset();

	// Recursively add child items
	int32 LastDirectlySelected = TreeItems.Num();
	for (int32 Index = 0; Index < TreeItems.Num(); ++Index)
	{
		FCurveEditorTreeItemID ItemID = TreeItems[Index];

		const FCurveEditorTreeItem* TreeItem = Items.Find(ItemID);
		if (!ensureAlwaysMsgf(TreeItem, TEXT("Selected tree item does not exist. This must have bee applied externally.")))
		{
			continue;
		}

		if (Index < LastDirectlySelected)
		{
			Selection.Add(ItemID, ECurveEditorTreeSelectionState::Explicit);
		}
		else
		{
			Selection.Add(ItemID, ECurveEditorTreeSelectionState::ImplicitChild);
		}

		for (FCurveEditorTreeItemID ChildID : TreeItem->GetChildren())
		{
			if ((FilterStates.Get(ChildID) & ECurveEditorTreeFilterState::MatchBitMask) != ECurveEditorTreeFilterState::NoMatch)
			{
				TreeItems.Add(ChildID);
			}
		}
	}
	InCurveEditor->ResetMinMaxes();
	for (TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> OldItem : PreviousSelection)
	{
		const ECurveEditorTreeSelectionState* NewState = Selection.Find(OldItem.Key);
		if (!NewState || *NewState == ECurveEditorTreeSelectionState::None)
		{
			GetItem(OldItem.Key).DestroyUnpinnedCurves(InCurveEditor);
		}
	}

	// Ensure the new selection has valid curve models
	for (TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> NewItem : Selection)
	{
		if (NewItem.Value != ECurveEditorTreeSelectionState::None)
		{
			GetItem(NewItem.Key).GetOrCreateCurves(InCurveEditor);
		}
	}


	if (!PreviousSelection.OrderIndependentCompareEqual(Selection))
	{
		++Events.OnSelectionChanged.SerialNumber;
	}
}

void FCurveEditorTree::RemoveFromSelection(TArrayView<const FCurveEditorTreeItemID> TreeItems, FCurveEditor* InCurveEditor)
{
	FScopedCurveEditorTreeEventGuard EventGuard(this);

	bool bAnyRemoved = false;
	for (const FCurveEditorTreeItemID& ItemID : TreeItems)
	{
		FCurveEditorTreeItem* TreeItem = Items.Find(ItemID);
		if (!ensureAlwaysMsgf(TreeItem, TEXT("Selected tree item does not exist. This must have bee applied externally.")))
		{
			continue;
		}

		if (Selection.Remove(ItemID) > 0)
		{
			bAnyRemoved = true;
			TreeItem->DestroyUnpinnedCurves(InCurveEditor);
		}
	}

	if (bAnyRemoved)
	{
		++Events.OnSelectionChanged.SerialNumber;
	}
}

 void  FCurveEditorTree::RecreateModelsFromExistingSelection(FCurveEditor* CurveEditor)
{
	// Ensure the new selection has valid curve models
	for (TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> NewItem : Selection)
	{
		if (NewItem.Value != ECurveEditorTreeSelectionState::None)
		{
			GetItem(NewItem.Key).DestroyCurves(CurveEditor);
			GetItem(NewItem.Key).GetOrCreateCurves(CurveEditor);
		}
	}
}

TArray<FCurveEditorTreeItemID> FCurveEditorTree::GetCachedExpandedItems() const
{
	TArray<FCurveEditorTreeItemID> ExpandedItems;

	const int32 NumCachedExpandedItems = CachedExpandedItems.Num();
	if (NumCachedExpandedItems > 0)
	{
		ExpandedItems.Reserve(NumCachedExpandedItems);
		for (const TPair<FCurveEditorTreeItemID, FCurveEditorTreeItem>& Item : Items)
		{
			const TOptional<FString>& PathName = Item.Value.GetUniquePathName();
			if (PathName.IsSet())
			{
				int32 Hash = GetTypeHash(PathName.GetValue());
				if (CachedExpandedItems.Contains(Hash))
				{
					ExpandedItems.Add(Item.Key);

					// early exit
					if (ExpandedItems.Num() == NumCachedExpandedItems)
					{
						return ExpandedItems;					
					}
				}
			}
		}
	}
	
	return ExpandedItems;
}

void FCurveEditorTree::SetItemExpansion(FCurveEditorTreeItemID InTreeItemID, bool bInExpansion)
{
	if (FCurveEditorTreeItem* TreeItem = FindItem(InTreeItemID))
	{
		const TOptional<FString>& OptString = TreeItem->GetUniquePathName();
		if (OptString.IsSet())
		{
			int32 Hash = GetTypeHash(OptString.GetValue());
			if (bInExpansion)
			{
				CachedExpandedItems.Add(Hash);
			}
			else
			{
				CachedExpandedItems.Remove(Hash);
			}
		}
	}
}

const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& FCurveEditorTree::GetSelection() const
{
	return Selection;
}

ECurveEditorTreeSelectionState FCurveEditorTree::GetSelectionState(FCurveEditorTreeItemID InTreeItemID) const
{
	const ECurveEditorTreeSelectionState* State = Selection.Find(InTreeItemID);
	return State ? *State : ECurveEditorTreeSelectionState::None;
}
