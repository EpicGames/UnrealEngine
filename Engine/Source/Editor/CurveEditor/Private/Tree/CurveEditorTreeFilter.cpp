// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/CurveEditorTreeFilter.h"

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"


FCurveEditorTreeTextFilterTerm::FMatchResult FCurveEditorTreeTextFilterTerm::FMatchResult::Match(FStringView CandidateString) const
{
	// No tokens left - already a total match
	if (RemainingTokens.Num() == 0)
	{
		return FMatchResult(RemainingTokens);
	}

	FStringView MatchString = RemainingTokens[0].Token;
	const int32 MatchStartIndex = CandidateString.Find(MatchString, 0, ESearchCase::IgnoreCase);

	if (MatchStartIndex == INDEX_NONE)
	{
		return FMatchResult();
	}

	FMatchResult Result(RemainingTokens.RightChop(1));

	// The token matched! Continue to match chains of tokens separated by a period (.) within the same string

	CandidateString = CandidateString.Left(MatchStartIndex);

	while (Result.RemainingTokens.Num() > 0 && CandidateString.EndsWith('.'))
	{
		MatchString = Result.RemainingTokens[0].Token;

		// Remove the period from the end of the candidate string
		CandidateString = CandidateString.LeftChop(1);

		// Compare the tail
		if (CandidateString.Right(MatchString.Len()).Compare(MatchString, ESearchCase::IgnoreCase) != 0)
		{
			break;
		}

		// This token matched as well - remove it and keep matching...
		Result.RemainingTokens = Result.RemainingTokens.RightChop(1);
	}

	// If there are no more tokens we must have matched them all
	return Result;
}

FCurveEditorTreeTextFilterTerm::FMatchResult FCurveEditorTreeTextFilterTerm::Match(FStringView InString) const
{
	return FMatchResult(ChildToParentTokens).Match(InString);
}


ECurveEditorTreeFilterType FCurveEditorTreeFilter::RegisterFilterType()
{
	static ECurveEditorTreeFilterType NextFilterType = ECurveEditorTreeFilterType::CUSTOM_START;
	ensureMsgf(NextFilterType != ECurveEditorTreeFilterType::First, TEXT("Maximum limit for registered curve tree filters (64) reached."));
	if (NextFilterType == ECurveEditorTreeFilterType::First)
	{
		return NextFilterType;
	}

	ECurveEditorTreeFilterType ThisFilterType = NextFilterType;

	// When the custom view ID reaches 0x80000000 the left shift will result in well-defined unsigned integer wraparound, resulting in 0 (None)
	NextFilterType = ECurveEditorTreeFilterType( ((__underlying_type(ECurveEditorTreeFilterType))NextFilterType) + 1 );

	return NextFilterType;
}

void FCurveEditorTreeTextFilter::AssignFromText(const FString& FilterString)
{
	ChildToParentFilterTerms.Reset();

	static const bool bCullEmpty = true;

	TArray<FString> FilterTerms;
	FilterString.ParseIntoArray(FilterTerms, TEXT(" "), bCullEmpty);

	TArray<FString> ParentToChildTerms;
	for (const FString& Term : FilterTerms)
	{
		ParentToChildTerms.Reset();
		Term.ParseIntoArray(ParentToChildTerms, TEXT("."), bCullEmpty);

		// Move the results into a new term in reverse order (so they are child -> parent)
		FCurveEditorTreeTextFilterTerm NewTerm;
		for (int32 Index = ParentToChildTerms.Num()-1; Index >= 0; --Index)
		{
			NewTerm.ChildToParentTokens.Emplace(FCurveEditorTreeTextFilterToken{ MoveTemp(ParentToChildTerms[Index]) });
		}
		ChildToParentFilterTerms.Emplace(MoveTemp(NewTerm));
	}
}