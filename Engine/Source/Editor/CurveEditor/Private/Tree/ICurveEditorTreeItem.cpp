// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/ICurveEditorTreeItem.h"

const ICurveEditorTreeItem::FColumnNames ICurveEditorTreeItem::ColumnNames;

ICurveEditorTreeItem::FColumnNames::FColumnNames()
	: Label("Label")
	, SelectHeader("Select")
	, PinHeader("Pin")
{}