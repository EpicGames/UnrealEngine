// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/SCurveEditorTree.h"

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CurveEditor.h"
#include "Framework/Views/ITypedTableView.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Tree/CurveEditorTree.h"
#include "Tree/CurveEditorTreeTraits.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "UObject/NameTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

class SWidget;
struct FGeometry;


struct SCurveEditorTableRow : SMultiColumnTableRow<FCurveEditorTreeItemID>
{
	FCurveEditorTreeItemID TreeItemID;
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID)
	{
		TreeItemID = InTreeItemID;
		WeakCurveEditor = InCurveEditor;

		SMultiColumnTableRow::Construct(InArgs, OwnerTableView);

		SetForegroundColor(MakeAttributeSP(this, &SCurveEditorTableRow::GetForegroundColorByFilterState));
	}

	FSlateColor GetForegroundColorByFilterState() const
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();

		const bool bIsMatch = CurveEditor.IsValid() && ((CurveEditor->GetTree()->GetFilterState(TreeItemID) & ECurveEditorTreeFilterState::Match) != ECurveEditorTreeFilterState::NoMatch);
		return bIsMatch ? GetForegroundBasedOnSelection() : FSlateColor::UseSubduedForeground();
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		TSharedPtr<ICurveEditorTreeItem> TreeItem = CurveEditor ? CurveEditor->GetTreeItem(TreeItemID).GetItem() : nullptr;

		TSharedRef<ITableRow> TableRow = SharedThis(this);

		TSharedPtr<SWidget> Widget;
		if (TreeItem.IsValid())
		{
			Widget = TreeItem->GenerateCurveEditorTreeWidget(InColumnName, WeakCurveEditor, TreeItemID, TableRow);
		}

		if (!Widget.IsValid())
		{
			Widget = SNullWidget::NullWidget;
		}

		if (InColumnName == ICurveEditorTreeItem::ColumnNames.Label)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(3.f, 0.f, 0.f, 0.f))
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
		}

		return Widget.ToSharedRef();
	}
};


void SCurveEditorTree::Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor)
{
	bFilterWasActive = false;
	bUpdatingTreeWidgetSelection = false;
	bUpdatingCurveEditorTreeSelection = false;
	HeaderRow = SNew(SHeaderRow)
		.Visibility(EVisibility::Collapsed)

		+ SHeaderRow::Column(ICurveEditorTreeItem::ColumnNames.Label)

		+ SHeaderRow::Column(ICurveEditorTreeItem::ColumnNames.SelectHeader)
		.FixedWidth(InArgs._SelectColumnWidth)

		+ SHeaderRow::Column(ICurveEditorTreeItem::ColumnNames.PinHeader)
		.FixedWidth(24.f);

	CurveEditor = InCurveEditor;

	STreeView<FCurveEditorTreeItemID>::Construct(
		STreeView<FCurveEditorTreeItemID>::FArguments()
		.SelectionMode(ESelectionMode::Multi)
		.HeaderRow(HeaderRow)
		.HighlightParentNodesForSelection(true)
		.TreeItemsSource(&RootItems)
		.OnGetChildren(this, &SCurveEditorTree::GetTreeItemChildren)
		.OnGenerateRow(this, &SCurveEditorTree::GenerateRow)
		.OnSetExpansionRecursive(this, &SCurveEditorTree::SetItemExpansionRecursive)
		.OnExpansionChanged(this, &SCurveEditorTree::OnExpansionChanged)
		.OnMouseButtonDoubleClick(InArgs._OnMouseButtonDoubleClick)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnTreeViewScrolled(InArgs._OnTreeViewScrolled)
		.OnSelectionChanged_Lambda(
			[this](TListTypeTraits<FCurveEditorTreeItemID>::NullableType InItemID, ESelectInfo::Type Type)
			{
				this->OnTreeSelectionChanged(InItemID, Type);
			}
		)
		.AllowInvisibleItemSelection(true)
	);

	CurveEditor->GetTree()->Events.OnItemsChanged.AddSP(this, &SCurveEditorTree::RefreshTree);
	CurveEditor->GetTree()->Events.OnSelectionChanged.AddSP(this, &SCurveEditorTree::RefreshTreeWidgetSelection);

	CurveEditor->GetTree()->GetToggleExpansionState().AddSP(this, &SCurveEditorTree::ToggleExpansionState);
	CurveEditor->GetTree()->GetOnExpandAllNodes().AddSP(this, &SCurveEditorTree::ExpandAllNodes);
	CurveEditor->GetTree()->GetOnCollapseAllNodes().AddSP(this, &SCurveEditorTree::CollapseAllNodes);
}

void SCurveEditorTree::RefreshTree()
{
	RootItems.Reset();

	const FCurveEditorTree* CurveEditorTree = CurveEditor->GetTree();
	const FCurveEditorFilterStates& FilterStates = CurveEditorTree->GetFilterStates();
	bool bExpansionWasCleared = false;
	const TArray<FCurveEditorTreeItemID>& CachedExpandedItems = CurveEditor->GetTree()->GetCachedExpandedItems();

	// When changing to/from a filtered state, we save and restore expansion states
	if (FilterStates.IsActive() && !bFilterWasActive)
	{
		// Save expansion states
		PreFilterExpandedItems.Reset();
		GetExpandedItems(PreFilterExpandedItems);
	}
	else if (!FilterStates.IsActive() && bFilterWasActive)
	{
		// Add any currently selected items' parents to the expanded items array.
		// This ensures that items that were selected during a filter operation remain expanded and selected when finished
		for (FCurveEditorTreeItemID SelectedItemID : GetSelectedItems())
		{
			// Add the selected item's parent and any grandparents to the list
			const FCurveEditorTreeItem* ParentItem = CurveEditorTree->FindItem(SelectedItemID);
			if (ParentItem)
			{
				ParentItem = CurveEditorTree->FindItem(ParentItem->GetParentID());
			}

			while (ParentItem)
			{
				const FCurveEditorTreeItemID ParentID = ParentItem->GetID();
				if (IsItemExpanded(ParentID))
				{
					PreFilterExpandedItems.Add(ParentID);
				}
				ParentItem = CurveEditorTree->FindItem(ParentItem->GetParentID());
			}
		}

		// Restore expansion states
		bExpansionWasCleared = true;
		ClearExpandedItems();
		for (FCurveEditorTreeItemID ExpandedItem : PreFilterExpandedItems)
		{
			SetItemExpansion(ExpandedItem, true);
		}
		PreFilterExpandedItems.Reset();
	}

	// Repopulate root tree items based on filters
	for (FCurveEditorTreeItemID RootItemID : CurveEditor->GetRootTreeItems())
	{
		if ((FilterStates.Get(RootItemID) & ECurveEditorTreeFilterState::MatchBitMask) != ECurveEditorTreeFilterState::NoMatch)
		{
			RootItems.Add(RootItemID);
		}
	}

	RootItems.Shrink();
	RequestTreeRefresh();
	if (FilterStates.IsActive())
	{
		TArray<FCurveEditorTreeItemID> ExpandedItems;
		ExpandedItems.Reserve(FilterStates.GetNumMatchedImplicitly());

		FilterStates.ForEachItemState([&ExpandedItems](const TTuple<FCurveEditorTreeItemID, ECurveEditorTreeFilterState>& FilterState)
			{
				if ((FilterState.Value & ECurveEditorTreeFilterState::Expand) != ECurveEditorTreeFilterState::NoMatch)
				{
					ExpandedItems.Add(FilterState.Key);
				}
			}
		);

		if (ExpandedItems.Num() > 0)
		{
			bExpansionWasCleared = true;
			ClearExpandedItems();
			for (const FCurveEditorTreeItemID& ItemID : ExpandedItems)
			{
				SetItemExpansion(ItemID, true);
			}
		}
	}

	// at end we reset any cached expansions we still have set
	if (bExpansionWasCleared == false)
	{
		ClearExpandedItems();
	}
	for (const FCurveEditorTreeItemID& ItemID : CachedExpandedItems)
	{
		SetItemExpansion(ItemID, true);
	}

	bFilterWasActive = FilterStates.IsActive();
}

void SCurveEditorTree::RefreshTreeWidgetSelection()
{	
	if(bUpdatingCurveEditorTreeSelection == false)
	{
		TGuardValue<bool> SelectionGuard(bUpdatingTreeWidgetSelection, true);

		TArray<FCurveEditorTreeItemID> CurrentTreeWidgetSelection;
		GetSelectedItems(CurrentTreeWidgetSelection);
		const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurrentCurveEditorTreeSelection = CurveEditor->GetTreeSelection();

		TArray<FCurveEditorTreeItemID> NewTreeWidgetSelection;
		for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurveEditorTreeSelectionEntry : CurrentCurveEditorTreeSelection)
		{
			if (CurveEditorTreeSelectionEntry.Value != ECurveEditorTreeSelectionState::None)
			{
				NewTreeWidgetSelection.Add(CurveEditorTreeSelectionEntry.Key);
				CurrentTreeWidgetSelection.RemoveSwap(CurveEditorTreeSelectionEntry.Key);
			}
		}

		SetItemSelection(CurrentTreeWidgetSelection, false, ESelectInfo::Direct);
		SetItemSelection(NewTreeWidgetSelection, true, ESelectInfo::Direct);
	}
}

void SCurveEditorTree::ToggleExpansionState(bool bRecursive)
{
	if (GetSelectedItems().Num() > 0)
	{
		const bool bExpand = !IsItemExpanded(GetSelectedItems()[0]);

		for (const FCurveEditorTreeItemID& SelectedItemID : GetSelectedItems())
		{
			if (bRecursive)
			{
				SetItemExpansionRecursive(SelectedItemID, bExpand);
			}
			else
			{
				SetItemExpansion(SelectedItemID, bExpand);
			}
		}
	}
	else if (CurveEditor->GetRootTreeItems().Num() > 0)
	{
		const bool bExpand = !IsItemExpanded(CurveEditor->GetRootTreeItems()[0]);

		for (const FCurveEditorTreeItemID& RootItemID : CurveEditor->GetRootTreeItems())
		{
			if (bRecursive)
			{
				SetItemExpansionRecursive(RootItemID, bExpand);
			}
			else
			{
				SetItemExpansion(RootItemID, bExpand);
			}
		}	
	}
}

void SCurveEditorTree::ExpandAllNodes()
{
	for (const FCurveEditorTreeItemID& RootItemID : CurveEditor->GetRootTreeItems())
	{
		SetItemExpansionRecursive(RootItemID, true);
	}
}

void SCurveEditorTree::CollapseAllNodes()
{
	for (const FCurveEditorTreeItemID& RootItemID : CurveEditor->GetRootTreeItems())
	{
		SetItemExpansionRecursive(RootItemID, false);
	}
}

FReply SCurveEditorTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		ClearSelection();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<ITableRow> SCurveEditorTree::GenerateRow(FCurveEditorTreeItemID ItemID, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCurveEditorTableRow, OwnerTable, CurveEditor, ItemID);
}

void SCurveEditorTree::GetTreeItemChildren(FCurveEditorTreeItemID Parent, TArray<FCurveEditorTreeItemID>& OutChildren)
{
	const FCurveEditorFilterStates& FilterStates = CurveEditor->GetTree()->GetFilterStates();

	for (FCurveEditorTreeItemID ChildID : CurveEditor->GetTreeItem(Parent).GetChildren())
	{
		if ((FilterStates.Get(ChildID) & ECurveEditorTreeFilterState::MatchBitMask) != ECurveEditorTreeFilterState::NoMatch)
		{
			OutChildren.Add(ChildID);
		}
	}
}

void SCurveEditorTree::OnTreeSelectionChanged(FCurveEditorTreeItemID, ESelectInfo::Type)
{
	if (bUpdatingTreeWidgetSelection == false)
	{
		TGuardValue<bool> SelecitonGuard(bUpdatingCurveEditorTreeSelection, true);
		CurveEditor->GetTree()->SetDirectSelection(GetSelectedItems(), CurveEditor.Get());
	}
}

void SCurveEditorTree::OnExpansionChanged(FCurveEditorTreeItemID Model, bool bInExpansionState)
{
	if (Model.IsValid())
	{
		CurveEditor->GetTree()->SetItemExpansion(Model, bInExpansionState);
	}
}

void SCurveEditorTree::SetItemExpansionRecursive(FCurveEditorTreeItemID Model, bool bInExpansionState)
{
	if (Model.IsValid())
	{
		SetItemExpansion(Model, bInExpansionState);

		TArray<FCurveEditorTreeItemID> Children;
		GetTreeItemChildren(Model, Children);

		for (FCurveEditorTreeItemID Child : Children)
		{
			if (Child.IsValid())
			{
				SetItemExpansionRecursive(Child, bInExpansionState);
			}
		}
	}
}

FCurveEditorTreeItemID SCurveEditorTree::FindVisibleSelectionProxy(const FCurveEditorTreeItemID InItemID) const
{
	const FCurveEditorTree& CurveEditorTree = *CurveEditor->GetTree();

	const FCurveEditorTreeItem* TreeItem = CurveEditorTree.FindItem(InItemID);
	while (TreeItem)
	{
		if (IsItemVisible(TreeItem->GetID()))
		{
			return TreeItem->GetID();
		}

		TreeItem = CurveEditorTree.FindItem(TreeItem->GetParentID());
	}

	return FCurveEditorTreeItemID::Invalid();
}