// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/SCurveEditorTreeFilterStatusBar.h"

#include "Algo/Count.h"
#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "Tree/CurveEditorTree.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCurveEditorTreeFilterStatusBar"

void SCurveEditorTreeFilterStatusBar::Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor)
{
	WeakCurveEditor = InCurveEditor;

	ChildSlot
	.Padding(FMargin(5.f, 0.f))
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(3.f, 0.f, 0.f, 0.f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(3.0f, 3.0f))
			[
				SNew(SHyperlink)
				.Visibility(this, &SCurveEditorTreeFilterStatusBar::GetVisibilityFromFilter)
				.Text(LOCTEXT("ClearFilters", "clear"))
				.OnNavigate(this, &SCurveEditorTreeFilterStatusBar::ClearFilters)
			]
		]

		+ SHorizontalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(3.0f, 3.0f))
			.Visibility(EVisibility::HitTestInvisible)
			[
				SAssignNew(TextBlock, STextBlock)
			]
		]
	];

	InCurveEditor->GetTree()->Events.OnItemsChanged.AddSP(this, &SCurveEditorTreeFilterStatusBar::UpdateText);
	InCurveEditor->GetTree()->Events.OnSelectionChanged.AddSP(this, &SCurveEditorTreeFilterStatusBar::UpdateText);
}

void SCurveEditorTreeFilterStatusBar::ClearFilters()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		CurveEditor->GetTree()->ClearFilters();
	}
}

EVisibility SCurveEditorTreeFilterStatusBar::GetVisibilityFromFilter() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	return (CurveEditor && CurveEditor->GetTree()->GetFilterStates().IsActive()) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SCurveEditorTreeFilterStatusBar::UpdateText()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!ensureAlways(CurveEditor))
	{
		return;
	}

	FText NewText;
	FLinearColor NewColor = FLinearColor::White;

	const FCurveEditorTree* Tree = CurveEditor->GetTree();
	const FCurveEditorFilterStates& FilterStates = Tree->GetFilterStates();
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Selection = Tree->GetSelection();

	FFormatNamedArguments NamedArgs;
	NamedArgs.Add("Total", Tree->GetAllItems().Num());

	const bool bHasSelection = Selection.Num() != 0;
	const bool bHasFilter    = FilterStates.IsActive();

	if (bHasSelection)
	{
		const int32 NumExplicitlySelected = Algo::CountIf(Selection, [](TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> Pair){ return Pair.Value == ECurveEditorTreeSelectionState::Explicit; });
		NamedArgs.Add("NumSelected", NumExplicitlySelected);
	}

	if (bHasFilter)
	{
		NamedArgs.Add("NumMatched", FilterStates.GetNumMatched() + FilterStates.GetNumMatchedImplicitly());
	}


	if (bHasFilter)
	{
		if (FilterStates.GetNumMatched() == 0)
		{
			// Red = no matched
			NewColor = FLinearColor( 1.0f, 0.4f, 0.4f );
		}
		else
		{
			// Green = matched filter
			NewColor = FLinearColor( 0.4f, 1.0f, 0.4f );
		}

		if (bHasSelection)
		{
			NewText = FText::Format(LOCTEXT("FilteredStatus_WithSelection", "Showing {NumMatched} of {Total} items ({NumSelected} selected)"), NamedArgs);
		}
		else 
		{
			NewText = FText::Format(LOCTEXT("FilteredStatus_NoSelection", "Showing {NumMatched} of {Total} items"), NamedArgs);
		}
	}
	else if (bHasSelection)
	{
		NewText = FText::Format(LOCTEXT("UnfilteredStatus_WithSelection", "{Total} items ({NumSelected} selected)"), NamedArgs);
	}
	else
	{
		NewText = FText::Format(LOCTEXT("UnfilteredStatus_NoSelection", "{Total} items"), NamedArgs);
	}

	TextBlock->SetColorAndOpacity(NewColor);
	TextBlock->SetText(NewText);
}

static double CurveEditorOpacityDurationSeconds = 2.0;

void SCurveEditorTreeFilterStatusBar::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GetVisibility() == EVisibility::Visible && OpacityThrobEndTime != FLT_MAX)
	{
		if (IsHovered())
		{
			// Make sure the status bar stays visible and keep it from fading until OnUnhovered
			ShowStatusBar();
			FadeOutStatusBar();
		}
		else
		{
			double CurrentTime = FPlatformTime::Seconds();

			float Opacity = 0.f;
			if (OpacityThrobEndTime > CurrentTime)
			{
				double Difference = OpacityThrobEndTime - CurrentTime;
				Opacity = Difference / CurveEditorOpacityDurationSeconds;
			}

			if (Opacity > 0.f)
			{
				FLinearColor BorderColor = GetColorAndOpacity();
				BorderColor.A = Opacity;
				SetColorAndOpacity(BorderColor);
			}
			else
			{
				SetVisibility(EVisibility::Hidden);
			}
		}
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SCurveEditorTreeFilterStatusBar::ShowStatusBar()
{
	OpacityThrobEndTime = FLT_MAX;
	FLinearColor BorderColor = GetColorAndOpacity();
	BorderColor.A = 1.f;
	SetColorAndOpacity(BorderColor);
	SetVisibility(EVisibility::Visible);
}

void SCurveEditorTreeFilterStatusBar::HideStatusBar()
{
	SetVisibility(EVisibility::Hidden);
}

void SCurveEditorTreeFilterStatusBar::FadeOutStatusBar()
{
	// Show and then start fading out the status bar
	OpacityThrobEndTime = FPlatformTime::Seconds() + CurveEditorOpacityDurationSeconds;
}

#undef LOCTEXT_NAMESPACE