// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/SCurveEditorTreePin.h"

#include "Algo/AllOf.h"
#include "Containers/ArrayView.h"
#include "CurveEditor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Tree/CurveEditorTree.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/ITableRow.h"

struct FSlateBrush;


void SCurveEditorTreePin::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow)
{
	WeakCurveEditor = InCurveEditor;
	WeakTableRow = InTableRow;
	TreeItemID = InTreeItemID;

	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.Visibility(this, &SCurveEditorTreePin::GetPinVisibility)
		.OnClicked(this, &SCurveEditorTreePin::TogglePinned)
		[
			SNew(SImage)
			.Image(this, &SCurveEditorTreePin::GetPinBrush)
		]
	];
}

FReply SCurveEditorTreePin::TogglePinned()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		if (IsPinnedRecursive(TreeItemID, CurveEditor.Get()))
		{
			TArray<FCurveEditorTreeItemID> UnpinnedItems;
			UnpinRecursive(TreeItemID, CurveEditor.Get(), UnpinnedItems);
			if (UnpinnedItems.Num() > 0)
			{
				CurveEditor->RemoveFromTreeSelection(UnpinnedItems);
			}
		}
		else
		{
			PinRecursive(TreeItemID, CurveEditor.Get());
		}
	}
	return FReply::Handled();
}

void SCurveEditorTreePin::PinRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor) const
{
	FCurveEditorTreeItem* Item = CurveEditor->FindTreeItem(InTreeItem);
	if (ensureMsgf(Item != nullptr, TEXT("Can't find curve editor tree item. Ignoring pinning request.")))
	{
		for (FCurveModelID CurveID : Item->GetOrCreateCurves(CurveEditor))
		{
			CurveEditor->PinCurve(CurveID);
		}

		for (FCurveEditorTreeItemID Child : Item->GetChildren())
		{
			PinRecursive(Child, CurveEditor);
		}
	}
}

void SCurveEditorTreePin::UnpinRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor, TArray<FCurveEditorTreeItemID>& OutUnpinnedItems) const
{
	const bool bIsSelected = CurveEditor->GetTreeSelectionState(InTreeItem) == ECurveEditorTreeSelectionState::Explicit;

	FCurveEditorTreeItem* Item = CurveEditor->FindTreeItem(InTreeItem);
	if (ensureMsgf(Item != nullptr, TEXT("Can't find curve editor tree item. Ignoring unpinning request.")))
	{
		for (FCurveModelID CurveID : Item->GetCurves())
		{
			if (bIsSelected)
			{
				CurveEditor->UnpinCurve(CurveID);
			}
			else
			{
				Item->DestroyCurves(CurveEditor);
			}
		}

		OutUnpinnedItems.Add(InTreeItem);
		for (FCurveEditorTreeItemID Child : Item->GetChildren())
		{
			UnpinRecursive(Child, CurveEditor, OutUnpinnedItems);
		}
	}
}

bool SCurveEditorTreePin::IsPinnedRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor) const
{
	const FCurveEditorTreeItem* Item = CurveEditor->FindTreeItem(InTreeItem);
	if (!Item)
	{
		return false;
	}

	TArrayView<const FCurveModelID>          Curves   = Item->GetCurves();
	TArrayView<const FCurveEditorTreeItemID> Children = Item->GetChildren();

	const bool bAllChildren = Algo::AllOf(Children, [this, CurveEditor](FCurveEditorTreeItemID In){ return this->IsPinnedRecursive(In, CurveEditor); });

	if (Curves.Num() == 0)
	{
		return Children.Num() > 0 && bAllChildren;
	}

	const bool bAllCurves = Algo::AllOf(Curves, [CurveEditor](FCurveModelID In) { return CurveEditor->IsCurvePinned(In); });
	return bAllCurves && bAllChildren;
}

EVisibility SCurveEditorTreePin::GetPinVisibility() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<ITableRow> Row = WeakTableRow.Pin();
	TSharedPtr<SWidget> RowWidget = Row ? TSharedPtr<SWidget>(Row->AsWidget()) : nullptr;

	if (RowWidget && RowWidget->IsHovered())
	{
		return EVisibility::Visible;
	}
	else if (CurveEditor && IsPinnedRecursive(TreeItemID, CurveEditor.Get()))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

const FSlateBrush* SCurveEditorTreePin::GetPinBrush() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		if (IsPinnedRecursive(TreeItemID, CurveEditor.Get()))
		{
			return FAppStyle::GetBrush("GenericCurveEditor.Pin_Active");
		}
	}

	return FAppStyle::GetBrush("GenericCurveEditor.Pin_Inactive");
}
