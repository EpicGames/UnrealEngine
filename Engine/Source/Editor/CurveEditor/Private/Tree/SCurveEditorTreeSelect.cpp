// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/SCurveEditorTreeSelect.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "Layout/Children.h"
#include "Math/NumericLimits.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Tree/CurveEditorTree.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SCurveEditorTreeSelect"

void SCurveEditorTreeSelect::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow)
{
	WeakCurveEditor = InCurveEditor;
	WeakTableRow = InTableRow;
	TreeItemID = InTreeItemID;

	ChildSlot
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("SelectAll", "Select all keys"))
		.Visibility(this, &SCurveEditorTreeSelect::GetSelectVisibility)
		.OnClicked(this, &SCurveEditorTreeSelect::SelectAll)
		[
			SNew(SImage)
			.Image(this, &SCurveEditorTreeSelect::GetSelectBrush)
		]
	];
}

FReply SCurveEditorTreeSelect::SelectAll()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return FReply::Handled();
	}
	
	FCurveEditorTreeItem* Item = CurveEditor->FindTreeItem(TreeItemID);
	if (!Item)
	{
		return FReply::Handled();
	}			
	
	const bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	const bool bIsControlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();

	const UE::CurveEditor::FScopedSelectionChange Transaction(CurveEditor, LOCTEXT("Tree.SelectAll", "Select all"));
	if (!bIsShiftDown && !bIsControlDown)
	{
		CurveEditor->GetSelection().Clear();
	}
		
	for (FCurveModelID CurveID : Item->GetCurves())
	{
		FCurveModel* CurveModel = CurveEditor.Get()->FindCurve(CurveID);
		if (CurveModel)
		{
			const TArray<FKeyHandle> KeyHandles = CurveModel->GetAllKeys();
		
			if (bIsControlDown)
			{
				CurveEditor.Get()->GetSelection().Toggle(CurveID, ECurvePointType::Key, KeyHandles);
			}
			else
			{
				CurveEditor.Get()->GetSelection().Add(CurveID, ECurvePointType::Key, KeyHandles);
			}
		}
	}

	return FReply::Handled();
}


EVisibility SCurveEditorTreeSelect::GetSelectVisibility() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return EVisibility::Collapsed;
	}
	
	FCurveEditorTreeItem* Item = CurveEditor->FindTreeItem(TreeItemID);
	if (!Item)
	{
		return EVisibility::Collapsed;
	}
		
	for (FCurveModelID CurveID : Item->GetCurves())
	{
		if (CurveEditor.Get()->GetSelection().GetAll().Contains(CurveID))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

const FSlateBrush* SCurveEditorTreeSelect::GetSelectBrush() const
{
	return FAppStyle::GetBrush("GenericCurveEditor.Select");
}

#undef LOCTEXT_NAMESPACE