// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tree/SCurveEditorTreeTextFilter.h"

#include "CurveEditor.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Tree/CurveEditorTree.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Widgets/Input/SSearchBox.h"

struct FGeometry;


void SCurveEditorTreeTextFilter::Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> CurveEditor)
{
	WeakCurveEditor = CurveEditor;

	CreateSearchBox();
	CurveEditor->GetTree()->Events.OnFiltersChanged.AddSP(this, &SCurveEditorTreeTextFilter::OnTreeFilterListChanged);
}

void SCurveEditorTreeTextFilter::CreateSearchBox()
{
	ChildSlot
	[
		SAssignNew(SearchBox, SSearchBox)
		.HintText(NSLOCTEXT("CurveEditor", "TextFilterHint", "Filter"))
		.OnTextChanged(this, &SCurveEditorTreeTextFilter::OnFilterTextChanged)
	];
}

void SCurveEditorTreeTextFilter::OnTreeFilterListChanged()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor && CurveEditor->GetTree()->FindFilterByType(ECurveEditorTreeFilterType::Text) == nullptr)
	{
		// If our filter has been removed externally, recreate the search box widget - this saves us having to manage a potentially
		// recursive loop setting the search text and responding to the filter list being changed etc
		Filter = nullptr;
		CreateSearchBox();
	}
}

void SCurveEditorTreeTextFilter::OnFilterTextChanged( const FText& FilterText )
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		// Remove our binding to the OnFiltersChanged event since we know this will be broadcast
		CurveEditor->GetTree()->Events.OnFiltersChanged.RemoveAll(this);

		if (!Filter)
		{
			Filter = MakeShared<FCurveEditorTreeTextFilter>();
		}

		Filter->AssignFromText(FilterText.ToString());

		if (!Filter->IsEmpty())
		{
			Filter->InputText = FilterText;
			CurveEditor->GetTree()->AddFilter(Filter);
		}
		else
		{
			CurveEditor->GetTree()->RemoveFilter(Filter);
		}

		// Re-add our binding to the OnFiltersChanged now that we know it has been broadcast
		CurveEditor->GetTree()->Events.OnFiltersChanged.AddSP(this, &SCurveEditorTreeTextFilter::OnTreeFilterListChanged);
	}
}

FReply SCurveEditorTreeTextFilter::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) 
{ 
	if(SearchBox)
	{
		FSlateApplication::Get().SetKeyboardFocus(SearchBox, InFocusEvent.GetCause());
	}
	
	return FReply::Handled();
}
