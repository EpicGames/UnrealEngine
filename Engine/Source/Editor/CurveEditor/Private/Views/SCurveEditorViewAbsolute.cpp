// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SCurveEditorViewAbsolute.h"

#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class FCurveEditor;

void SCurveEditorViewAbsolute::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	SInteractiveCurveEditorView::Construct(InArgs, InCurveEditor);

	ChildSlot
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Top)
	.Padding(FMargin(0.f, CurveViewConstants::CurveLabelOffsetY, CurveViewConstants::CurveLabelOffsetX, 0.f))
	[
		SNew(STextBlock)
		.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
		.ColorAndOpacity(this, &SCurveEditorViewAbsolute::GetCurveCaptionColor)
		.Text(this, &SCurveEditorViewAbsolute::GetCurveCaption)
	];
}