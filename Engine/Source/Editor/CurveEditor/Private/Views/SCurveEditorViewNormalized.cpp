// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SCurveEditorViewNormalized.h"

#include "Containers/SortedMap.h"
#include "CurveEditor.h"
#include "CurveEditorAxis.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSettings.h"
#include "CurveModel.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/TransformCalculus2D.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Rendering/DrawElements.h"
#include "SCurveEditorView.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class FPaintArgs;
class FSlateRect;
class FText;
class FWidgetStyle;

constexpr float NormalizedPadding = 10.f;

void SCurveEditorViewNormalized::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	bAllowModelViewTransforms = false;
	OutputMin = -0.1;
	OutputMax =  1.1;

	SInteractiveCurveEditorView::Construct(InArgs, InCurveEditor);

	ChildSlot
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Top)
	.Padding(FMargin(0.f, CurveViewConstants::CurveLabelOffsetY, CurveViewConstants::CurveLabelOffsetX, 0.f))
	[
		SNew(STextBlock)
		.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
		.ColorAndOpacity(this, &SCurveEditorViewNormalized::GetCurveCaptionColor)
		.Text(this, &SCurveEditorViewNormalized::GetCurveCaption)
	];
}

FTransform2d CalculateViewToCurveTransform(const double InCurveOutputMin, const double InCurveOutputMax)
{
	const double Scale = (InCurveOutputMax - InCurveOutputMin);
	if (Scale > KINDA_SMALL_NUMBER)
	{
		return FTransform2d(FScale2d(1.0, Scale), FVector2d(0.0, InCurveOutputMin));
	}
	else
	{
		return FTransform2d(FVector2d(0.f, InCurveOutputMin - 0.5));
	}
}

void SCurveEditorViewNormalized::DrawBufferedCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	if (!CurveEditor->GetSettings()->GetShowBufferedCurves())
	{
		return;
	}

	const float BufferedCurveThickness = 1.f;
	const bool  bAntiAliasCurves = true;
	const FLinearColor CurveColor = CurveViewConstants::BufferedCurveColor;
	const int32 CurveLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Curves;

	const double ValuePerPixel = 1.0 / AllottedGeometry.GetLocalSize().Y;
	const double ValueSpacePadding = NormalizedPadding * ValuePerPixel;

	// Calculate the normalized view to curve transform for each buffered curve, then draw
	const TArray<TUniquePtr<IBufferedCurveModel>>& BufferedCurves = CurveEditor->GetBufferedCurves();
	for (const TUniquePtr<IBufferedCurveModel>& BufferedCurve : BufferedCurves)
	{
		if (!CurveEditor->IsActiveBufferedCurve(BufferedCurve))
		{
			continue;
		}

		FTransform2d ViewToBufferedCurveTransform;
		double CurveOutputMin = BufferedCurve->GetValueMin(), CurveOutputMax = BufferedCurve->GetValueMax();

		ViewToBufferedCurveTransform = CalculateViewToCurveTransform(CurveOutputMin, CurveOutputMax);

		TArray<TTuple<double, double>> CurveSpaceInterpolatingPoints;
		FCurveEditorScreenSpace CurveSpace = GetViewSpace().ToCurveSpace(ViewToBufferedCurveTransform);

		BufferedCurve->DrawCurve(*CurveEditor, CurveSpace, CurveSpaceInterpolatingPoints);

		TArray<FVector2D> ScreenSpaceInterpolatingPoints;
		for (TTuple<double, double> Point : CurveSpaceInterpolatingPoints)
		{
			ScreenSpaceInterpolatingPoints.Add(FVector2D(
				CurveSpace.SecondsToScreen(Point.Get<0>()),
				CurveSpace.ValueToScreen(Point.Get<1>())
			));
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			CurveLayerId,
			AllottedGeometry.ToPaintGeometry(),
			ScreenSpaceInterpolatingPoints,
			DrawEffects,
			CurveColor,
			bAntiAliasCurves,
			BufferedCurveThickness
		);
	}
}

void SCurveEditorViewNormalized::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawGridLines(CurveEditor.ToSharedRef(), AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawBufferedCurves(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
		DrawCurves(CurveEditor.ToSharedRef(), AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
		DrawValueIndicatorLines(CurveEditor.ToSharedRef(), AllottedGeometry, OutDrawElements, BaseLayerId);
	}
}

void SCurveEditorViewNormalized::FrameVertical(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID)
{
	if (!bFixedOutputBounds && InOutputMin < InOutputMax)
	{
		double OutputPadding = 0.1;
		if (TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor())
		{
			const double PanelHeight = GetViewSpace().GetPhysicalHeight();
			OutputPadding = PanelHeight != 0.0 ? FMath::Min((double)CurveEditor->GetSettings()->GetFrameOutputPadding() / PanelHeight, 0.5) : 0.1;
		}

		const double NormalizedMin = -OutputPadding;
		const double NormalizedMax =  1.0 + OutputPadding;

		if (AxisID)
		{
			FAxisInfo& AxisInfo = GetVerticalAxisInfo(AxisID);
			AxisInfo.Min = NormalizedMin;
			AxisInfo.Max = NormalizedMax;
		}
		else
		{
			OutputMin = NormalizedMin;
			OutputMax = NormalizedMax;
		}
	}
}

void SCurveEditorViewNormalized::OnCurveListChanged()
{
	// Init only curves we haven't initialized yet, then drop entries for curves no longer in the view.
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		const TSharedRef<FCurveEditor> CurveEditorRef = CurveEditor.ToSharedRef();
		for (TPair<FCurveModelID, FCurveInfo>& Entry : CurveInfoByID)
		{
			if (!InitializedCurves.Contains(Entry.Key))
			{
				RecomputeViewToCurveTransform(CurveEditorRef, Entry.Key, Entry.Value);
				InitializedCurves.Add(Entry.Key);
			}
		}
	}

	for (auto It = InitializedCurves.CreateIterator(); It; ++It)
	{
		if (!CurveInfoByID.Contains(*It))
		{
			It.RemoveCurrent();
		}
	}
}

void SCurveEditorViewNormalized::PostUndo(bool bSuccess)
{	
	OnCurveListChanged();
}

void SCurveEditorViewNormalized::PostRedo(bool bSuccess)
{
	OnCurveListChanged();
}

void SCurveEditorViewNormalized::UpdateViewToTransformCurves(double InputMin, double InputMax)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const TSharedRef<FCurveEditor> CurveEditorRef = CurveEditor.ToSharedRef();
	for (TPair<FCurveModelID, FCurveInfo>& Entry : CurveInfoByID)
	{
		RecomputeViewToCurveTransform(CurveEditorRef, Entry.Key, Entry.Value);
	}
}

void SCurveEditorViewNormalized::RecomputeViewToCurveTransform(const TSharedRef<FCurveEditor>& CurveEditor, FCurveModelID CurveID, FCurveInfo& CurveInfo)
{
	FCurveModel* Curve = CurveEditor->FindCurve(CurveID);
	if (!ensureAlways(Curve))
	{
		return;
	}

	// Consider the whole time range when getting the value range...
	double MinTime, MaxTime;
	Curve->GetTimeRange(MinTime, MaxTime);
	// ... but clamp it to the time range the user has set with the time controller
	ClampToTimeController(MinTime, MaxTime);

	double CurveOutputMin = 0, CurveOutputMax = 1;
	Curve->GetValueRange(MinTime, MaxTime, CurveOutputMin, CurveOutputMax);

	CurveInfo.ViewToCurveTransform = CalculateViewToCurveTransform(CurveOutputMin, CurveOutputMax);
}


void SCurveEditorViewNormalized::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid() && !CurveEditor->AreBoundTransformUpdatesSuppressed())
	{
		InternalUpdateViewToTransformCurves();
	}

	SInteractiveCurveEditorView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

FReply SCurveEditorViewNormalized::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InternalUpdateViewToTransformCurves();

	return SInteractiveCurveEditorView::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SCurveEditorViewNormalized::InternalUpdateViewToTransformCurves()
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		double InputMin = 0.0;
		double InputMax = 1.0;
		GetInputBounds(InputMin, InputMax);
		UpdateViewToTransformCurves(InputMin, InputMax);
	}
}

void SCurveEditorViewNormalized::ClampToTimeController(double& OutMin, double& OutMax) const
{
	const TSharedPtr<ITimeSliderController> TimeController = GetCurveEditor()->GetTimeSliderController();
	if (!TimeController)
    {
		return;
    }
	
	const FFrameRate TickResolution = TimeController->GetTickResolution();
	const TRange<FFrameNumber> Range = TimeController->GetPlayRange();
	if (Range.HasLowerBound())
	{
		const double Seconds = TickResolution.AsSeconds(Range.GetLowerBoundValue().Value);
		OutMin = FMath::Max(OutMin, Seconds);
	}
	if (Range.HasUpperBound())
	{
		const double Seconds = TickResolution.AsSeconds(Range.GetUpperBoundValue().Value);
		OutMax = FMath::Min(OutMax, Seconds);
	}
}
