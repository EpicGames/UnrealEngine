// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SCurveEditorViewStacked.h"

#include "Containers/SortedMap.h"
#include "CurveEditor.h"
#include "CurveEditorHelpers.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSettings.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Layout/SlateRect.h"
#include "Math/Color.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"

class FPaintArgs;
class FWidgetStyle;
struct FSlateBrush;


void SCurveEditorViewStacked::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	bFixedOutputBounds = true;
	bAllowModelViewTransforms = false;
	OutputMin = 0.0;
	OutputMax = 1.0;

	StackedHeight = 150.0f;
	StackedPadding = 10.0f;

	SInteractiveCurveEditorView::Construct(InArgs, InCurveEditor);
}

FVector2D SCurveEditorViewStacked::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(100.f, StackedHeight*CurveInfoByID.Num() + StackedPadding*(CurveInfoByID.Num()+1));
}

void SCurveEditorViewStacked::GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{
	FCurveEditorScreenSpace ViewSpace = GetViewSpace();

	const double ValuePerPixel = 1.0 / StackedHeight;
	const double ValueSpacePadding = StackedPadding * ValuePerPixel;

	const TOptional<float> GridLineSpacing = CurveEditor->GetGridSpacing();
	// draw standard lines if fixed spacing is not set
	if (!GridLineSpacing)
	{
		for (int32 Index = 0; Index < CurveInfoByID.Num(); ++Index)
		{
			double Padding = (Index + 1)*ValueSpacePadding;
			double LowerValue = Index + Padding;

			// Lower Grid line
			MajorGridLines.Add(ViewSpace.ValueToScreen(LowerValue));
			// Center Grid line
			MajorGridLines.Add(ViewSpace.ValueToScreen(LowerValue + 0.5));
			// Upper Grid line
			MajorGridLines.Add(ViewSpace.ValueToScreen(LowerValue + 1.0));

			MinorGridLines.Add(ViewSpace.ValueToScreen(LowerValue + 0.25));
			MinorGridLines.Add(ViewSpace.ValueToScreen(LowerValue + 0.75));
		}
	}
	else
	{
		TArray<FCurveModelID> CurveIDs;
		CurveInfoByID.GetKeys(CurveIDs);
		for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
		{
			const int32 Index = CurveInfoByID.Num() - It->Value.CurveIndex - 1;

			double Padding = (Index + 1)*ValueSpacePadding;
			double LowerValue = Index + Padding;

			double ViewSpaceMax = GetViewSpace().ValueToScreen(LowerValue);
			double ViewSpaceMin = GetViewSpace().ValueToScreen(LowerValue + 1.0);

			FCurveEditorScreenSpace CurveSpace = GetCurveSpace(CurveIDs[Index]);
			TArray<float> TempMajorGridLines, TempMinorGridLines;
			CurveEditor::ConstructFixedYGridLines(CurveSpace, 4, GridLineSpacing.GetValue(), MajorGridLines, MinorGridLines, CurveEditor->GetGridLineLabelFormatYAttribute().Get(), 
				MajorGridLabels, CurveSpace.ScreenToValue(ViewSpaceMax), CurveSpace.ScreenToValue(ViewSpaceMin));
		}
	}
}

void SCurveEditorViewStacked::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawViewGrids(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
		DrawLabels(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
		DrawBufferedCurves(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
		DrawCurves(CurveEditor.ToSharedRef(), AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
		DrawValueIndicatorLines(CurveEditor.ToSharedRef(), AllottedGeometry, OutDrawElements, BaseLayerId);
	}
}

void SCurveEditorViewStacked::DrawViewGrids(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const int32 GridLineLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::GridLines;

	// Rendering info
	const float          Width = AllottedGeometry.GetLocalSize().X;
	const float          Height = AllottedGeometry.GetLocalSize().Y;
	const FLinearColor   MajorGridColor = CurveEditor->GetPanel()->GetGridLineTint();
	const FLinearColor   MinorGridColor = MajorGridColor.CopyWithNewOpacity(MajorGridColor.A * .5f);
	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();
	const FSlateBrush*   WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

	TArray<float> MajorGridLines, MinorGridLines;
	TArray<FText> MajorGridLabels;

	GetGridLinesX(CurveEditor.ToSharedRef(), MajorGridLines, MinorGridLines, &MajorGridLabels);

	// Pre-allocate an array of line points to draw our vertical lines. Each major grid line
	// will overwrite the X value of both points but leave the Y value untouched so they draw from the bottom to the top.
	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D(0.f, 0.f));
	LinePoints.Add(FVector2D(0.f, 0.f));

	const double ValuePerPixel = 1.0 / StackedHeight;
	const double ValueSpacePadding = StackedPadding * ValuePerPixel;

	FCurveEditorScreenSpace ViewSpace = GetViewSpace();
	for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		const int32  Index = CurveInfoByID.Num() - It->Value.CurveIndex - 1;
		double Padding = (Index + 1)*ValueSpacePadding;
		double LowerValue = Index + Padding;

		double PixelBottom = ViewSpace.ValueToScreen(LowerValue);
		double PixelTop    = ViewSpace.ValueToScreen(LowerValue + 1.0);
		if (!FSlateRect::DoRectanglesIntersect(MyCullingRect, TransformRect(AllottedGeometry.GetAccumulatedLayoutTransform(), FSlateRect(0, PixelTop, Width, PixelBottom))))
		{
			continue;
		}

		// Tint the views based on their curve color
		{
			FLinearColor CurveColorTint = Curve->GetColor().CopyWithNewOpacity(0.05f);
			const FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(
			FVector2D(Width, StackedHeight),
			FSlateLayoutTransform(FVector2D(0.f, PixelTop))
			);
			
			FSlateDrawElement::MakeBox(OutDrawElements, GridLineLayerId + 1, BoxGeometry, WhiteBrush, DrawEffects, CurveColorTint);
		}

		// Horizontal grid lines
		{
			TArray<float> MajorGridLinesH, MinorGridLinesH;
			GetGridLinesY(CurveEditor.ToSharedRef(), MajorGridLinesH, MinorGridLinesH);

			LinePoints[0].X = 0.0;
			LinePoints[1].X = Width;

			// draw major lines
			for (float GridLineVal : MajorGridLinesH)
			{
				LinePoints[0].Y = LinePoints[1].Y = GridLineVal;
				FSlateDrawElement::MakeLines(OutDrawElements, GridLineLayerId, PaintGeometry, LinePoints, DrawEffects, MajorGridColor, false);
			}

			// draw minor lines
			for (float GridLineVal : MinorGridLinesH)
			{
				LinePoints[0].Y = LinePoints[1].Y = GridLineVal;
				FSlateDrawElement::MakeLines(OutDrawElements, GridLineLayerId, PaintGeometry, LinePoints, DrawEffects, MinorGridColor, false);
			}
		}

		// Vertical grid lines
		{
			const float RoundedWidth = FMath::RoundToFloat(Width);

			LinePoints[0].Y = PixelTop;
			LinePoints[1].Y = PixelBottom;

			// Draw major vertical grid lines
			for (float VerticalLine : MajorGridLines)
			{
				VerticalLine = FMath::RoundToFloat(VerticalLine);
				if (VerticalLine >= 0 || VerticalLine <= RoundedWidth)
				{
					LinePoints[0].X = LinePoints[1].X = VerticalLine;
					FSlateDrawElement::MakeLines(OutDrawElements, GridLineLayerId, PaintGeometry, LinePoints, DrawEffects, MajorGridColor, false);
				}
			}

			// Now draw the minor vertical lines which are drawn with a lighter color.
			for (float VerticalLine : MinorGridLines)
			{
				VerticalLine = FMath::RoundToFloat(VerticalLine);
				if (VerticalLine >= 0 || VerticalLine <= RoundedWidth)
				{
					LinePoints[0].X = LinePoints[1].X = VerticalLine;
					FSlateDrawElement::MakeLines(OutDrawElements, GridLineLayerId, PaintGeometry, LinePoints, DrawEffects, MinorGridColor, false);
				}
			}
		}
	}
}

void SCurveEditorViewStacked::DrawLabels(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const int32 LabelLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Labels;

	const double ValuePerPixel = 1.0 / StackedHeight;
	const double ValueSpacePadding = StackedPadding * ValuePerPixel;

	const FSlateFontInfo FontInfo = FAppStyle::Get().GetFontStyle("NormalFont");
	const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
	const FCurveEditorScreenSpaceV ViewSpace = GetViewSpace();

	// Draw the curve labels for each view
	for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		const int32  CurveIndexFromBottom = CurveInfoByID.Num() - It->Value.CurveIndex - 1;
		const double PaddingToBottomOfView = (CurveIndexFromBottom + 1)*ValueSpacePadding;
		
		const double PixelBottom = ViewSpace.ValueToScreen(CurveIndexFromBottom + PaddingToBottomOfView);
		const double PixelTop = ViewSpace.ValueToScreen(CurveIndexFromBottom + PaddingToBottomOfView + 1.0);

		if (!FSlateRect::DoRectanglesIntersect(MyCullingRect, TransformRect(AllottedGeometry.GetAccumulatedLayoutTransform(), FSlateRect(0, PixelTop, LocalSize.X, PixelBottom))))
		{
			continue;
		}

		const FText Label = Curve->GetLongDisplayName();

		const FVector2D Position(CurveViewConstants::CurveLabelOffsetX, PixelTop + CurveViewConstants::CurveLabelOffsetY);

		const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(Position));
		const FPaintGeometry LabelDropshadowGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(Position + FVector2D(2, 2)));

		// Drop shadow
		FSlateDrawElement::MakeText(OutDrawElements, LabelLayerId, LabelDropshadowGeometry, Label, FontInfo, DrawEffects, FLinearColor::Black.CopyWithNewOpacity(0.80f));
		FSlateDrawElement::MakeText(OutDrawElements, LabelLayerId+1, LabelGeometry, Label, FontInfo, DrawEffects, Curve->GetColor());
	}
}

FTransform2d CalculateViewToCurveTransform(const double InCurveOutputMin, double const InCurveOutputMax, const double InValueOffset)
{
	if ((InCurveOutputMax - InCurveOutputMin) > KINDA_SMALL_NUMBER)
	{
		return Concatenate(FVector2d(0.f, InValueOffset), Concatenate(FScale2d(1.f, (InCurveOutputMax - InCurveOutputMin)), FVector2d(0.f, InCurveOutputMin)));
	}
	else
	{
		return Concatenate(FVector2d(0.f, InValueOffset - 0.5), FVector2d(0.f, InCurveOutputMin));
	}
}

void SCurveEditorViewStacked::DrawBufferedCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	if (!CurveEditor->GetSettings()->GetShowBufferedCurves())
	{
		return;
	}

	const TArray<TUniquePtr<IBufferedCurveModel>>& BufferedCurves = CurveEditor->GetBufferedCurves();

	const float BufferedCurveThickness = 1.f;
	const bool  bAntiAliasCurves = true;
	const FLinearColor CurveColor = CurveViewConstants::BufferedCurveColor;
	const int32 CurveLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Curves;

	const double ValuePerPixel = 1.0 / StackedHeight;
	const double ValueSpacePadding = StackedPadding * ValuePerPixel;


	// draw the buffered curves for each view
	for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		FTransform2d ViewToBufferedCurveTransform;

		const int32  CurveIndexFromBottom = CurveInfoByID.Num() - It->Value.CurveIndex - 1;
		const double PaddingToBottomOfView = (CurveIndexFromBottom + 1)*ValueSpacePadding;
		const double ValueOffset = -CurveIndexFromBottom - PaddingToBottomOfView;

		// Calculate the view to curve transform for each buffered curve, then draw
		for (const TUniquePtr<IBufferedCurveModel>& BufferedCurve : BufferedCurves)
		{
			if (!CurveEditor->IsActiveBufferedCurve(BufferedCurve))
			{
				continue;
			}

			double CurveOutputMin = BufferedCurve->GetValueMin(), CurveOutputMax = BufferedCurve->GetValueMax();

			ViewToBufferedCurveTransform = CalculateViewToCurveTransform(CurveOutputMin, CurveOutputMax, ValueOffset);

			TArray<TTuple<double, double>> CurveSpaceInterpolatingPoints;
			FCurveEditorScreenSpace CurveSpace = GetViewSpace().ToCurveSpace(ViewToBufferedCurveTransform);

			BufferedCurve->DrawCurve(*CurveEditor, CurveSpace, CurveSpaceInterpolatingPoints);

			TArray<FVector2D> ScreenSpaceInterpolatingPoints;
			for (TTuple<double, double> Point : CurveSpaceInterpolatingPoints)
			{
				ScreenSpaceInterpolatingPoints.Add(FVector2D(
					CurveSpace.SecondsToScreen(Point.Get<0>()),
					CurveSpace.ValueToScreen(Point.Get<1>())
				));
			}

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				CurveLayerId,
				AllottedGeometry.ToPaintGeometry(),
				ScreenSpaceInterpolatingPoints,
				DrawEffects,
				CurveColor,
				bAntiAliasCurves,
				BufferedCurveThickness
			);
		}

	}
}

void SCurveEditorViewStacked::UpdateViewToTransformCurves(double InputMin, double InputMax)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}
	double ValuePerPixel = 1.0 / StackedHeight;
	double ValueSpacePadding = StackedPadding * ValuePerPixel;

	for (auto It = CurveInfoByID.CreateIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		const int32  CurveIndexFromBottom = CurveInfoByID.Num() - It->Value.CurveIndex - 1;
		const double PaddingToBottomOfView = (CurveIndexFromBottom + 1) * ValueSpacePadding;
		const double ValueOffset = -CurveIndexFromBottom - PaddingToBottomOfView;

		double CurveOutputMin = 0, CurveOutputMax = 1;
		Curve->GetValueRange(InputMin, InputMax, CurveOutputMin, CurveOutputMax);

		It->Value.ViewToCurveTransform = CalculateViewToCurveTransform(CurveOutputMin, CurveOutputMax, ValueOffset);
	}

	OutputMax = FMath::Max(OutputMin + CurveInfoByID.Num() + ValueSpacePadding * (CurveInfoByID.Num() + 1), 1.0);
}


void SCurveEditorViewStacked::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	if (!CurveEditor->AreBoundTransformUpdatesSuppressed())
	{
		// Get the Min/Max values on the X axis, for Time
		double InputMin = 0, InputMax = 1;
		GetInputBounds(InputMin, InputMax);
		UpdateViewToTransformCurves(InputMin, InputMax);
	}

	SInteractiveCurveEditorView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}