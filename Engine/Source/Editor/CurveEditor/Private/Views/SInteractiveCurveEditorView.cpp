// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SInteractiveCurveEditorView.h"

#include "Algo/AnyOf.h"
#include "Algo/MaxElement.h"
#include "Algo/Sort.h"
#include "AnimatedRange.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorAxis.h"
#include "CurveEditorCommands.h"
#include "CurveEditorContextMenu.h"
#include "CurveEditorCurveDrawParamsCache.h"
#include "CurveEditorHelpers.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSettings.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveEditorZoomScaleConfig.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "DragOperations/CurveEditorDragOperation_MoveKeys.h"
#include "DragOperations/CurveEditorDragOperation_Pan.h"
#include "DragOperations/CurveEditorDragOperation_Tangent.h"
#include "DragOperations/CurveEditorDragOperation_Zoom.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "ICurveEditorToolExtension.h"
#include "ITimeSlider.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "Math/Box.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/InputState.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "SCurveEditorPanel.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "DragOperations/KeyDragOperationUtils.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/NumericTypeInterface.h"

class FPaintArgs;
class FWidgetStyle;

TAutoConsoleVariable<bool> CVarDrawCurveLines(TEXT("CurveEditor.DrawCurveLines"), true, TEXT("When true we draw curve lines, when false we do not."));
TAutoConsoleVariable<bool> CVarDrawCurveKeys(TEXT("CurveEditor.DrawCurveKeys"), true, TEXT("When true we draw curve keys, when false we do not."));

namespace CurveViewConstants
{
	/** The number of pixels to offset Labels from the Left/Right size. */
	constexpr float LabelOffsetPixels = 2.f;

	/** The number of pixels away the mouse can be and still be considering hovering over a curve. */
	constexpr float HoverProximityThresholdPx = 5.f;
}

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "EngineUtils.h"

extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR


#define LOCTEXT_NAMESPACE "SInteractiveCurveEditorView"

class SDynamicToolTip : public SToolTip
{
public:
	TAttribute<bool> bIsEnabled;
	virtual bool IsEmpty() const override { return !bIsEnabled.Get(); }
};

void SInteractiveCurveEditorView::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	FixedHeight = InArgs._FixedHeight;
	BackgroundTint = InArgs._BackgroundTint;
	MaximumCapacity = InArgs._MaximumCapacity;
	bAutoSize = InArgs._AutoSize;
	
	const TSharedPtr<FCurveEditor> CurveEditorPin = InCurveEditor.Pin();
	check(CurveEditorPin);
	InitCurveEditorReference(CurveEditorPin.ToSharedRef());
	{
		CurveDrawParamsCache->Invalidate(SharedThis(this));
		
		CurveEditorPin->OnActiveToolChangedDelegate.AddSP(this, &SInteractiveCurveEditorView::OnCurveEditorToolChanged);
		CurveEditorPin->GetSettings()->OnShowValueIndicatorsChanged().AddSP(this, &SInteractiveCurveEditorView::OnShowValueIndicatorsChanged);
	}

	TSharedRef<SDynamicToolTip> ToolTipWidget =
		SNew(SDynamicToolTip)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SInteractiveCurveEditorView::GetToolTipCurveName)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
		+ SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(this, &SInteractiveCurveEditorView::GetToolTipTimeText)
			.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
			.ColorAndOpacity(FLinearColor::Black)
		]
		+ SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(this, &SInteractiveCurveEditorView::GetToolTipValueText)
			.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
			.ColorAndOpacity(FLinearColor::Black)
		]
	];

	ToolTipWidget->bIsEnabled = MakeAttributeSP(this, &SInteractiveCurveEditorView::IsToolTipEnabled);
	SetToolTip(ToolTipWidget);
}

FText SInteractiveCurveEditorView::GetCurveCaption() const
{
	FText CurveCaption;

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor && CurveInfoByID.Num() == 1)
	{
		for (const TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
		{
			if (const FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
			{
				CurveCaption = Curve->GetLongDisplayName();
				break;
			}
		}
	}

	if (!CurveCaption.IdenticalTo(CachedCurveCaption))
	{
		CachedCurveCaption = CurveCaption;
		bNeedsRefresh = true;
	}

	return CurveCaption;
}

FSlateColor SInteractiveCurveEditorView::GetCurveCaptionColor() const
{
	FSlateColor CurveCaptionColor = BackgroundTint.CopyWithNewOpacity(1.f);

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor && CurveInfoByID.Num() == 1)
	{
		for (const TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
		{
			if (const FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
			{
				CurveCaptionColor = Curve->GetColor();
				break;
			}
		}
	}

	if (CurveCaptionColor != CachedCurveCaptionColor)
	{
		CachedCurveCaptionColor = CurveCaptionColor;
		bNeedsRefresh = true;
	}

	return CurveCaptionColor;
}

void SInteractiveCurveEditorView::GetGridLinesX(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{
	CurveEditor->GetGridLinesX(MajorGridLines, MinorGridLines, MajorGridLabels);

	FCurveEditorScreenSpaceH PanelSpace = CurveEditor->GetPanelInputSpace();
	FCurveEditorScreenSpaceH ViewSpace  = GetViewSpace();

	double InputOffset = ViewSpace.GetInputMin() - PanelSpace.GetInputMin();
	if (InputOffset != 0.0)
	{
		const float PixelDifference = InputOffset * PanelSpace.PixelsPerInput();
		for (float& Line : MajorGridLines)
		{
			Line -= PixelDifference;
		}
		for (float& Line : MinorGridLines)
		{
			Line -= PixelDifference;
		}
	}
}

void SInteractiveCurveEditorView::GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{
	const TOptional<float> GridLineSpacing = CurveEditor->GetGridSpacing();
	if (!GridLineSpacing)
	{
		CurveEditor::ConstructYGridLines(GetViewSpace(), 4, MajorGridLines, MinorGridLines, CurveEditor->GetGridLineLabelFormatYAttribute().Get(), MajorGridLabels);
	}
	else
	{
		CurveEditor::ConstructFixedYGridLines(GetViewSpace(), 4, GridLineSpacing.GetValue(), MajorGridLines, MinorGridLines, CurveEditor->GetGridLineLabelFormatYAttribute().Get(), MajorGridLabels, TOptional<double>(), TOptional<double>());
	}
}


void SInteractiveCurveEditorView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		RefreshRetainer();
	}

	SCurveEditorView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

int32 SInteractiveCurveEditorView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	PaintView(Args, AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, bParentEnabled);
	SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::WidgetContent, InWidgetStyle, bParentEnabled);

	return BaseLayerId;
}

void SInteractiveCurveEditorView::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		const TSharedRef<FCurveEditor> CurveEditorRef = CurveEditor.ToSharedRef();
		
		DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawGridLines(CurveEditorRef, AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawBufferedCurves(CurveEditorRef, AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
		DrawCurves(CurveEditorRef, AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
		DrawValueIndicatorLines(CurveEditorRef, AllottedGeometry, OutDrawElements, BaseLayerId);
	}
}

void SInteractiveCurveEditorView::DrawBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	if (BackgroundTint != FLinearColor::White)
	{
		FSlateDrawElement::MakeBox(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Background, AllottedGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush("ToolPanel.GroupBorder"), DrawEffects, BackgroundTint);
	}
}

void SInteractiveCurveEditorView::DrawGridLines(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	// Rendering info
	const float          Width = AllottedGeometry.GetLocalSize().X;
	const float          Height = AllottedGeometry.GetLocalSize().Y;
	const float          RoundedWidth = FMath::RoundToFloat(Width);
	const float          RoundedHeight = FMath::RoundToFloat(Height);
	const FLinearColor   MajorGridColor = CurveEditor->GetPanel()->GetGridLineTint();
	const FLinearColor   MinorGridColor = MajorGridColor.CopyWithNewOpacity(MajorGridColor.A * .5f);
	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();
	const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");

	FCurveEditorScreenSpace ViewSpace = GetViewSpace();

	FCurveEditorScreenSpaceH HorizontalGridSpace = ViewSpace;
	FCurveEditorScreenSpaceV VerticalGridSpace   = ViewSpace;

	TArray<double> MajorGridLines, MinorGridLines;

	struct FGridLine
	{
		double Value;
		FText Label;
	};

	struct FGridLineLabels
	{
		FGridLineLabels(FCurveEditorViewAxisID AxisID, TSet<FCurveEditorViewAxisID>& HighlightedAxes)
		{
			// Show default color if this axis is not highlighted, or its the only one
			if (HighlightedAxes.Num() == 0)
			{
				// Default
				Color = FLinearColor::White.CopyWithNewOpacity(0.65f);
			}
			else if (HighlightedAxes.Contains(AxisID))
			{
				// Highlighted
				Color = FLinearColor::White.CopyWithNewOpacity(0.95f);
			}
			else
			{
				// Subdued
				Color = FLinearColor::White.CopyWithNewOpacity(0.15f);
			}
		}
		TArray<FText> Labels;
		TArray<FVector2f> Sizes;
		FVector2f MaxSize;
		FLinearColor Color;
	};
	TArray<FGridLineLabels> MajorGridLabels;

	FCurveEditorSelection& Selection = CurveEditor->GetSelection();
	TSet<FCurveEditorViewAxisID> HighlightedHorizontalAxes, HighlightedVerticalAxes;

	TOptional<FCurveModelID> HoveredCurve = GetHoveredCurve();
	if (HoveredCurve)
	{
		if (const FCurveInfo* CurveInfo = CurveInfoByID.Find(HoveredCurve.GetValue()))
		{
			HighlightedHorizontalAxes.Add(CurveInfo->HorizontalAxis);
			HighlightedVerticalAxes.Add(CurveInfo->VerticalAxis);
		}
	}
	else for (const TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
	{
		if (const FKeyHandleSet* SelectionSet = Selection.FindForCurve(Pair.Key))
		{
			HighlightedHorizontalAxes.Add(Pair.Value.HorizontalAxis);
			HighlightedVerticalAxes.Add(Pair.Value.VerticalAxis);
		}
	}

	// Ask Custom axes to draw until we find something that does
	for (int32 Index = 0; Index < CustomHorizontalAxes.Num(); ++Index)
	{
		const FAxisInfo& AxisInfo = CustomHorizontalAxes[Index];
		AxisInfo.Axis->GetGridLines(*CurveEditor, *this, FCurveEditorViewAxisID(Index), MajorGridLines, MinorGridLines, ECurveEditorAxisOrientation::Horizontal);
		if (MajorGridLines.Num() || MinorGridLines.Num())
		{
			HorizontalGridSpace = FCurveEditorScreenSpaceH(HorizontalGridSpace.GetPhysicalWidth(), AxisInfo.Min, AxisInfo.Max);
			break;
		}
	}

	if (MajorGridLines.Num() == 0)
	{
		TArray<float> MajorGridLinesFloat, MinorGridLinesFloat;
		if (bNeedsDefaultGridLinesH)
		{
			// Auto populate the major grid labels
			FGridLineLabels& Labels = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(), HighlightedHorizontalAxes);
			GetGridLinesX(CurveEditor, MajorGridLinesFloat, MinorGridLinesFloat, &Labels.Labels);
		}
		else
		{
			GetGridLinesX(CurveEditor, MajorGridLinesFloat, MinorGridLinesFloat, nullptr);
		}

		// This legacy API defined grid lines in screen space
		if (MajorGridLinesFloat.Num())
		{
			MajorGridLines.SetNum(MajorGridLinesFloat.Num());
			for (int32 Index = 0; Index < MajorGridLines.Num(); ++Index)
			{
				MajorGridLines[Index] = ViewSpace.ScreenToSeconds(MajorGridLinesFloat[Index]);
			}
		}
		if (MinorGridLinesFloat.Num())
		{
			MinorGridLines.SetNum(MinorGridLinesFloat.Num());
			for (int32 Index = 0; Index < MinorGridLines.Num(); ++Index)
			{
				MinorGridLines[Index] = ViewSpace.ScreenToSeconds(MinorGridLinesFloat[Index]);
			}
		}
	}
	else if (bNeedsDefaultGridLinesH)
	{
		FText DefaultFormat = CurveEditor->GetGridLineLabelFormatXAttribute().Get();

		if (!DefaultFormat.IsEmpty())
		{
			FGridLineLabels& DefaultGridLabels = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(), HighlightedHorizontalAxes);

			const int32 Num = MajorGridLines.Num();
			DefaultGridLabels.Labels.SetNum(Num);
			for (int32 GridLineIndex = 0; GridLineIndex < Num; ++GridLineIndex)
			{
				// Put the grid line from HorizontalGridSpace into the default ViewSpace
				double GridLine = HorizontalGridSpace.SecondsToScreen(MajorGridLines[GridLineIndex]);
				GridLine = ViewSpace.ScreenToSeconds(GridLine);

				DefaultGridLabels.Labels[GridLineIndex] = FText::Format(DefaultFormat, GridLine);
			}
		}
	}

	// Populate grid labels for custom axes
	if (MajorGridLines.Num() > 0)
	{
		const int32 NumLabels = MajorGridLines.Num();

		for (int32 Index = 0; Index < CustomHorizontalAxes.Num(); ++Index)
		{
			const FAxisInfo& AxisInfo = CustomHorizontalAxes[Index];
			if (!AxisInfo.Axis->HasLabels())
			{
				continue;
			}

			FGridLineLabels& Entry = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(Index), HighlightedHorizontalAxes);

			Entry.Labels.SetNum(NumLabels);
			for (int32 GridLineIndex = 0; GridLineIndex < NumLabels; ++GridLineIndex)
			{
				Entry.Labels[GridLineIndex] = AxisInfo.Axis->MakeLabel(MajorGridLines[GridLineIndex]);
			}
		}
	}

	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Compute sizing
	for (FGridLineLabels& Entry : MajorGridLabels)
	{
		Entry.Sizes.SetNum(Entry.Labels.Num());

		FVector2f MaxSize(0.f, 0.f);
		for (int32 Index = 0; Index < Entry.Labels.Num(); ++Index)
		{
			const FVector2f LabelSize = FontMeasure->Measure(Entry.Labels[Index], FontInfo);

			Entry.Sizes[Index] = LabelSize;
			MaxSize.X = FMath::Max(MaxSize.X, LabelSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, LabelSize.Y);
		}
		Entry.MaxSize = MaxSize;
	}

	// Pre-allocate an array of line points to draw our vertical lines. Each major grid line
	// will overwrite the X value of both points but leave the Y value untouched so they draw from the bottom to the top.
	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D(0.f, 0.f));
	LinePoints.Add(FVector2D(0.f, Height));

	// Draw major vertical grid lines
	for (int32 i = 0; i < MajorGridLines.Num(); i++)
	{
		const float RoundedLine = FMath::RoundToFloat(HorizontalGridSpace.SecondsToScreen(MajorGridLines[i]));
		if (RoundedLine < 0 || RoundedLine > RoundedWidth)
		{
			continue;
		}

		// Vertical Grid Line
		LinePoints[0].X = LinePoints[1].X = RoundedLine;

		// Offset for all labels
		if (MajorGridLabels[0].Labels.Num() > 0)
		{
			FVector2f LabelOffset(0.f, 0.f);

			// Compute size of all labels
			for (FGridLineLabels& Entry : MajorGridLabels)
			{
				LabelOffset.Y += CurveViewConstants::LabelOffsetPixels;

				FVector2f LabelSize = Entry.Sizes[i];
				const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(
					FSlateLayoutTransform(
						FVector2f(
							LinePoints[0].X - LabelSize.X*.5f,                 // Center horizontally on the grid line
							LabelOffset.Y + (Entry.MaxSize.Y-LabelSize.Y)*.5f  // Center vertically within the axis
						)
					)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					BaseLayerId + CurveViewConstants::ELayerOffset::GridLabels,
					LabelGeometry,
					Entry.Labels[i],
					FontInfo,
					DrawEffects,
					Entry.Color
				);

				LabelOffset.Y += Entry.MaxSize.Y + CurveViewConstants::LabelOffsetPixels;
			}

			LinePoints[0].Y = LabelOffset.Y;
		}
		else
		{
			LinePoints[0].Y = 0.f;
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			BaseLayerId + CurveViewConstants::ELayerOffset::GridLines,
			PaintGeometry,
			LinePoints,
			DrawEffects,
			MajorGridColor,
			false
		);
	}

	LinePoints[0].Y = 0.f;

	// Now draw the minor vertical lines which are drawn with a lighter color.
	for (float PosX : MinorGridLines)
	{
		PosX = HorizontalGridSpace.SecondsToScreen(PosX);
		if (PosX < 0 || PosX > Width)
		{
			continue;
		}

		LinePoints[0].X = LinePoints[1].X = PosX;

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			BaseLayerId + CurveViewConstants::ELayerOffset::GridLines,
			PaintGeometry,
			LinePoints,
			DrawEffects,
			MinorGridColor,
			false
		);
	}







	MajorGridLines.Reset();
	MinorGridLines.Reset();
	MajorGridLabels.Reset();

	// Ask Custom axes to draw until we find something that does
	for (int32 Index = 0; Index < CustomVerticalAxes.Num(); ++Index)
	{
		const FAxisInfo& AxisInfo = CustomVerticalAxes[Index];
		AxisInfo.Axis->GetGridLines(*CurveEditor, *this, FCurveEditorViewAxisID(Index), MajorGridLines, MinorGridLines, ECurveEditorAxisOrientation::Vertical);
		if (MajorGridLines.Num() || MinorGridLines.Num())
		{
			VerticalGridSpace = FCurveEditorScreenSpaceV(ViewSpace.GetPhysicalHeight(), AxisInfo.Min, AxisInfo.Max);
			break;
		}
	}

	if (MajorGridLines.Num() == 0)
	{
		// Auto populate the major grid labels
		TArray<float> MajorGridLinesFloat, MinorGridLinesFloat;
		if (bNeedsDefaultGridLinesV)
		{
			// Auto populate the major grid labels
			FGridLineLabels& Labels = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(), HighlightedVerticalAxes);
			GetGridLinesY(CurveEditor, MajorGridLinesFloat, MinorGridLinesFloat, &Labels.Labels);
		}
		else
		{
			GetGridLinesY(CurveEditor, MajorGridLinesFloat, MinorGridLinesFloat, nullptr);
		}

		// This legacy API defined grid lines in screen space
		if (MajorGridLinesFloat.Num())
		{
			MajorGridLines.SetNum(MajorGridLinesFloat.Num());
			for (int32 Index = 0; Index < MajorGridLines.Num(); ++Index)
			{
				MajorGridLines[Index] = ViewSpace.ScreenToValue(MajorGridLinesFloat[Index]);
			}
		}
		if (MinorGridLinesFloat.Num())
		{
			MinorGridLines.SetNum(MinorGridLinesFloat.Num());
			for (int32 Index = 0; Index < MinorGridLines.Num(); ++Index)
			{
				MinorGridLines[Index] = ViewSpace.ScreenToValue(MinorGridLinesFloat[Index]);
			}
		}
	}
	else if (bNeedsDefaultGridLinesV)
	{
		FText DefaultFormat = CurveEditor->GetGridLineLabelFormatYAttribute().Get();
		if (!DefaultFormat.IsEmpty())
		{
			FGridLineLabels& DefaultGridLabels = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(), HighlightedVerticalAxes);

			const int32 Num = MajorGridLines.Num();
			DefaultGridLabels.Labels.SetNum(Num);
			for (int32 GridLineIndex = 0; GridLineIndex < Num; ++GridLineIndex)
			{
				double GridLine = VerticalGridSpace.ValueToScreen(MajorGridLines[GridLineIndex]);
				GridLine = ViewSpace.ScreenToValue(GridLine);

				DefaultGridLabels.Labels[GridLineIndex] = FText::Format(DefaultFormat, GridLine);
			}
		}
	}

	// Populate grid labels for custom axes
	if (MajorGridLines.Num() > 0)
	{
		const int32 NumLabels = MajorGridLines.Num();

		for (int32 Index = 0; Index < CustomVerticalAxes.Num(); ++Index)
		{
			const FAxisInfo& AxisInfo = CustomVerticalAxes[Index];
			if (!AxisInfo.Axis->HasLabels())
			{
				continue;
			}

			FGridLineLabels& Entry = MajorGridLabels.Emplace_GetRef(FCurveEditorViewAxisID(Index), HighlightedVerticalAxes);

			Entry.Labels.SetNum(NumLabels);
			for (int32 GridLineIndex = 0; GridLineIndex < NumLabels; ++GridLineIndex)
			{
				Entry.Labels[GridLineIndex] = AxisInfo.Axis->MakeLabel(MajorGridLines[GridLineIndex]);
			}
		}
	}











	// Compute sizing
	for (FGridLineLabels& Entry : MajorGridLabels)
	{
		Entry.Sizes.SetNum(Entry.Labels.Num());

		FVector2f MaxSize(0.f, 0.f);
		for (int32 Index = 0; Index < Entry.Labels.Num(); ++Index)
		{
			const FVector2f LabelSize = FontMeasure->Measure(Entry.Labels[Index], FontInfo);

			Entry.Sizes[Index] = LabelSize;
			MaxSize.X = FMath::Max(MaxSize.X, LabelSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, LabelSize.Y);
		}
		Entry.MaxSize = MaxSize;
	}


	// Reset our cached Line to draw from left to right
	LinePoints[0].X = 0.f;
	LinePoints[1].X = Width;

	// Draw our major horizontal lines
	for (int32 i = 0; i < MajorGridLines.Num(); i++)
	{
		const float RoundedLine = FMath::RoundToFloat(VerticalGridSpace.ValueToScreen(MajorGridLines[i]));
		if (RoundedLine < 0 || RoundedLine > RoundedHeight)
		{
			continue;
		}

		// Overwrite the height of the line we're drawing to draw the different grid lines.
		LinePoints[0].Y = LinePoints[1].Y = RoundedLine;

		// Offset for all labels
		if (MajorGridLabels[0].Labels.Num() > 0)
		{
			FVector2f LabelOffset(0.f, 0.f);

			// Compute size of all labels
			for (FGridLineLabels& Entry : MajorGridLabels)
			{
				LabelOffset.X += CurveViewConstants::LabelOffsetPixels;

				FVector2f LabelSize = Entry.Sizes[i];
				const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(
					FSlateLayoutTransform(
						FVector2D(
							LabelOffset.X + (Entry.MaxSize.X-LabelSize.X)*.5f,  // Center horizontally within the axis
							LinePoints[0].Y - LabelSize.Y*.5f                   // Center vertically on the grid line
						)
					)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					BaseLayerId + CurveViewConstants::ELayerOffset::GridLabels,
					LabelGeometry,
					Entry.Labels[i],
					FontInfo,
					DrawEffects,
					Entry.Color
				);

				LabelOffset.X += Entry.MaxSize.X + CurveViewConstants::LabelOffsetPixels;
			}

			LinePoints[0].X = LabelOffset.X;
		}
		else
		{
			LinePoints[0].X = 0.f;
		}

		// Draw the grid line
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			BaseLayerId + CurveViewConstants::ELayerOffset::GridLines,
			PaintGeometry,
			LinePoints,
			DrawEffects,
			MajorGridColor,
			false
		);
	}

	LinePoints[0].X = 0.f;

	// Draw our minor horizontal lines
	for (float PosY : MinorGridLines)
	{
		PosY = VerticalGridSpace.ValueToScreen(PosY);
		if (PosY < 0 || PosY > Height)
		{
			continue;
		}

		LinePoints[0].Y = LinePoints[1].Y = PosY;

		// Now draw the minor grid lines with a lighter color.
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			BaseLayerId + CurveViewConstants::ELayerOffset::GridLines,
			PaintGeometry,
			LinePoints,
			DrawEffects,
			MinorGridColor,
			false
		);
	}
}

void SInteractiveCurveEditorView::DrawCurves(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const
{
	FLinearColor SelectionColor = CurveEditor->GetSettings()->GetSelectionColor();

	const FVector2D      VisibleSize = AllottedGeometry.GetLocalSize();
	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

	const float HoverThicknessOffset = 1.5f;
	const bool  bAntiAliasCurves = true;

	const bool bDrawLines = CVarDrawCurveLines.GetValueOnGameThread();
	const bool bDrawKeys = CVarDrawCurveKeys.GetValueOnGameThread();

	const TArray<FCurveDrawParams>& DrawParamsArray = CurveDrawParamsCache->GetCurveDrawParams();

	TOptional<FCurveModelID> HoveredCurve = GetHoveredCurve();
	for (const FCurveDrawParams& DrawParams : DrawParamsArray)
	{
		const FCurveModelID& ModelID = DrawParams.GetID();

		const bool bIsCurveHovered = HoveredCurve.IsSet() && HoveredCurve.GetValue() == ModelID;
		const float Thickness = bIsCurveHovered ? DrawParams.Thickness+HoverThicknessOffset : DrawParams.Thickness;
		const int32 CurveLayerId = bIsCurveHovered ? BaseLayerId + CurveViewConstants::ELayerOffset::Curves : BaseLayerId + CurveViewConstants::ELayerOffset::HoveredCurves;

		if (bDrawLines && DrawParams.bDrawInterpolatingPoints)
		{
			if (DrawParams.DashLengthPx > 0.f)
			{
				float DashOffset = static_cast<float>(GetViewSpace().PixelsPerInput() * GetViewSpace().GetInputMin());

				TArray<FVector2f> NewVector;
				NewVector.Reserve(DrawParams.InterpolatingPoints.Num());
				for (const FVector2d& Vect : DrawParams.InterpolatingPoints)
				{
					NewVector.Add(UE::Slate::CastToVector2f(Vect));
				}

				FSlateDrawElement::MakeDashedLines(
					OutDrawElements,
					CurveLayerId,
					PaintGeometry,
					MoveTemp(NewVector),
					DrawEffects,
					DrawParams.Color,
					Thickness,
					DrawParams.DashLengthPx,
					DashOffset
				);
			}
			else
			{
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					CurveLayerId,
					PaintGeometry,
					DrawParams.InterpolatingPoints,
					DrawEffects,
					DrawParams.Color,
					bAntiAliasCurves,
					Thickness
				);
			}
		}
		
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		// Draw tangents
		if (bDrawKeys && DrawParams.bKeyDrawEnabled)
		{
			for (int32 PointIndex = 0; PointIndex < DrawParams.Points.Num(); PointIndex++)
			{
				const FCurvePointInfo& Point = DrawParams.Points[PointIndex];
				if (!Point.bDraw)
				{
					continue;
				}

				const FKeyDrawInfo& PointDrawInfo = DrawParams.GetKeyDrawInfo(Point.Type, PointIndex);
				const bool          bSelected = CurveEditor->GetSelection().IsSelected(FCurvePointHandle(DrawParams.GetID(), Point.Type, Point.KeyHandle));
				FLinearColor  PointTint = PointDrawInfo.Tint.IsSet() ? PointDrawInfo.Tint.GetValue() : DrawParams.Color;

				if (bSelected)
				{
					PointTint = SelectionColor;
				}
				else
				{
					// Brighten and saturate the points a bit so they pop
					FLinearColor HSV = PointTint.LinearRGBToHSV();
					HSV.G = FMath::Clamp(HSV.G * 1.1f, 0.f, 255.f);
					HSV.B = FMath::Clamp(HSV.B * 2.f, 0.f, 255.f);
					PointTint = HSV.HSVToLinearRGB();
				}

				const int32 KeyLayerId = BaseLayerId + Point.LayerBias + (bSelected ? CurveViewConstants::ELayerOffset::SelectedKeys : CurveViewConstants::ELayerOffset::Keys);

				if (Point.LineDelta.X != 0.f || Point.LineDelta.Y != 0.f)
				{
					LinePoints[0] = Point.ScreenPosition + Point.LineDelta.GetSafeNormal() * (PointDrawInfo.ScreenSize.X*.5f);
					LinePoints[1] = Point.ScreenPosition + Point.LineDelta;

					// Draw the connecting line - connecting lines are always drawn below everything else
					FSlateDrawElement::MakeLines(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Keys - 1, PaintGeometry, LinePoints, DrawEffects, PointTint, true);
				}

				FPaintGeometry PointGeometry = AllottedGeometry.ToPaintGeometry(
					PointDrawInfo.ScreenSize,
					FSlateLayoutTransform(Point.ScreenPosition - (PointDrawInfo.ScreenSize * 0.5f))
				);

				FSlateDrawElement::MakeBox(OutDrawElements, KeyLayerId, PointGeometry, PointDrawInfo.Brush, DrawEffects, PointTint);
			}
		}
	}
}

void SInteractiveCurveEditorView::DrawBufferedCurves(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const
{
	if (!CurveEditor->GetSettings()->GetShowBufferedCurves())
	{
		return;
	}

	const float BufferedCurveThickness = 1.f;
	const bool  bAntiAliasCurves = true;
	const FLinearColor CurveColor = CurveViewConstants::BufferedCurveColor;
	const TArray<TUniquePtr<IBufferedCurveModel>>& BufferedCurves = CurveEditor->GetBufferedCurves();

	const int32 CurveLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Curves;

	// Draw each buffered curve using the view space transform since the curve space for all curves is the same
	for (const TUniquePtr<IBufferedCurveModel>& BufferedCurve : BufferedCurves)
	{
		if (!CurveEditor->IsActiveBufferedCurve(BufferedCurve))
		{
			continue;
		}

		TArray<TTuple<double, double>> CurveSpaceInterpolatingPoints;
		FCurveEditorScreenSpace CurveSpace = GetViewSpace();

		BufferedCurve->DrawCurve(*CurveEditor, CurveSpace, CurveSpaceInterpolatingPoints);

		TArray<FVector2D> ScreenSpaceInterpolatingPoints;
		for (TTuple<double, double> Point : CurveSpaceInterpolatingPoints)
		{
			ScreenSpaceInterpolatingPoints.Add(FVector2D(
				CurveSpace.SecondsToScreen(Point.Get<0>()),
				CurveSpace.ValueToScreen(Point.Get<1>())
			));
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			CurveLayerId,
			AllottedGeometry.ToPaintGeometry(),
			ScreenSpaceInterpolatingPoints,
			DrawEffects,
			CurveColor,
			bAntiAliasCurves,
			BufferedCurveThickness
		);
	}
}

void SInteractiveCurveEditorView::DrawValueIndicatorLines(
	TSharedRef<FCurveEditor> InCurveEditor, const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, const int32 InBaseLayerId
	) const
{
	UpdatedKeysWithValueIndicatorLines(*InCurveEditor);
	
	const FCurveModel* CurveModel = ValueIndicatorLineDrawData ? InCurveEditor->FindCurve(ValueIndicatorLineDrawData->HighlightedCurve) : nullptr;
	if (!CurveModel)
	{
		return;
	}

	const FCurveEditorScreenSpace ViewSpace = GetCurveSpace(ValueIndicatorLineDrawData->HighlightedCurve);
	const auto DrawLine = [&](const FKeyHandle& Handle)
	{
		FKeyPosition Position;
		CurveModel->GetKeyPositions({ Handle }, TArrayView<FKeyPosition>(&Position, 1));
			
		const float PosY = ViewSpace.ValueToScreen(Position.OutputValue);
		FSlateDrawElement::MakeDashedLines(
			OutDrawElements,
			InBaseLayerId + CurveViewConstants::ELayerOffset::GridOverlays,
			InAllottedGeometry.ToPaintGeometry(),
			{ FVector2f{ 0, PosY }, FVector2f{ InAllottedGeometry.GetLocalSize().X, PosY } },
			ESlateDrawEffect::None,
			FLinearColor::White * 0.6f
			);
	};

	checkf(ValueIndicatorLineDrawData->MinKey, TEXT("ValueIndicatorLineDrawData should be unset."));
	DrawLine(ValueIndicatorLineDrawData->MinKey);

	// If only 1 key selected, max is unset.
	if (ValueIndicatorLineDrawData->MaxKey)
	{
		DrawLine(ValueIndicatorLineDrawData->MaxKey);
	}
}

void SInteractiveCurveEditorView::UpdatedKeysWithValueIndicatorLines(const FCurveEditor& InCurveEditor) const
{
	if (!InCurveEditor.GetSettings()->GetShowValueIndicators())
	{
		ValueIndicatorLineDrawData.Reset();
		return;
	}
	
	const TMap<FCurveModelID, FKeyHandleSet>& Selection = InCurveEditor.Selection.GetAll();
	const FCurveModel* Line_CurveModel = ValueIndicatorLineDrawData
		? InCurveEditor.FindCurve(ValueIndicatorLineDrawData->HighlightedCurve) : nullptr;
	const FKeyHandleSet* Line_CurveSelectedKeys = ValueIndicatorLineDrawData
		? Selection.Find(ValueIndicatorLineDrawData->HighlightedCurve) : nullptr;
	
	// Clear cached data if it references a curve that is no longer selected or was removed, or the selection contains more than 1 curve.
	const bool bClearData = !Line_CurveModel || !Line_CurveSelectedKeys || Selection.Num() > 1;
	if (ValueIndicatorLineDrawData && bClearData)
	{
		ValueIndicatorLineDrawData.Reset();
	}

	if (ValueIndicatorLineDrawData && !bClearData)
	{
		// Update existing data: Min and Max may have changed
		const bool bIsValid = PickPointsToPlaceValueIndicatorLinesOn(
			*Line_CurveModel, *Line_CurveSelectedKeys, ValueIndicatorLineDrawData->MinKey, ValueIndicatorLineDrawData->MaxKey
			);
		if (!bIsValid)
		{
			ValueIndicatorLineDrawData.Reset();
		}
	}
	// Only show indicators when exactly 1 curve is selected
	else if (Selection.Num() == 1)
	{
		// Pick the first applicable curve
		for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : Selection)
		{
			const FCurveModelID& CurveId = Pair.Key;
			const FCurveModel* CurveModel = InCurveEditor.FindCurve(CurveId);
			if (!CurveModel)
			{
				continue;
			}
			
			FKeyHandle Min = FKeyHandle::Invalid();
			FKeyHandle Max = FKeyHandle::Invalid();
			if (PickPointsToPlaceValueIndicatorLinesOn(*CurveModel, Pair.Value, Min, Max))
			{
				ValueIndicatorLineDrawData.Emplace(CurveId, Min, Max);
				return;
			}
		}
	}
}

bool SInteractiveCurveEditorView::PickPointsToPlaceValueIndicatorLinesOn(
	const FCurveModel& InCurveModel,
	const FKeyHandleSet& InUserSelectedKeys,
	FKeyHandle& OutMinKey,
	FKeyHandle& OutMaxKey
	) const
{
	const TConstArrayView<FKeyHandle> Handles = InUserSelectedKeys.AsArray();
	const bool bContainsNonKeys = Algo::AnyOf(Handles, [&InUserSelectedKeys](const FKeyHandle& Handle)
	{
		return InUserSelectedKeys.PointType(Handle) != ECurvePointType::Key;
	});
	// No lines if tangents are selected (or Handles is empty, which should not happen).
	if (bContainsNonKeys || Handles.Num() == 0)
	{
		return false;
	}

	if (Handles.Num() == 1)
	{
		OutMinKey = Handles[0];
		OutMaxKey = FKeyHandle::Invalid();
		return true;
	}
	
	const auto GetKeyValue = [&InCurveModel](const FKeyHandle& Handle)
	{
		FKeyPosition Position;
		InCurveModel.GetKeyPositions({ Handle }, TArrayView<FKeyPosition>(&Position, 1));
		return Position.OutputValue;
	};
	const FKeyHandle* MinPoint = Algo::MinElementBy(Handles, GetKeyValue);
	const FKeyHandle* MaxPoint = Algo::MaxElementBy(Handles, GetKeyValue);
	OutMinKey = *MinPoint;
	OutMaxKey = OutMinKey == *MaxPoint ? FKeyHandle::Invalid() : *MaxPoint;
	return true;
}

bool SInteractiveCurveEditorView::GetPointsWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return false;
	}

	FVector2D LinePoints[2];
	FVector Start, End, StartToEnd;
	FBox WidgetRectangleBox(FVector(WidgetRectangle.Left, WidgetRectangle.Top, 0), FVector(WidgetRectangle.Right, WidgetRectangle.Bottom, 0));

	const float PointOverlapSensitivity = CurveEditor->GetSettings()->GetMarqueePointSensitivity();

	const TArray<FCurveDrawParams>& DrawParamsArray = CurveDrawParamsCache->GetCurveDrawParams();

	// Iterate through all of our points and see which points the marquee overlaps. Both of these coordinate systems
	// are in screen space pixels.  Also check tangent lines
	bool bFound = false;
	for (const FCurveDrawParams& DrawParams : DrawParamsArray)
	{
		for (int32 PointIndex = 0; PointIndex < DrawParams.Points.Num(); PointIndex++)
		{
			const FCurvePointInfo& Point = DrawParams.Points[PointIndex];

			const FKeyDrawInfo& DrawInfo = DrawParams.GetKeyDrawInfo(Point.Type, PointIndex);
			const FVector2d CollisionScreenSize = DrawInfo.ScreenSize * PointOverlapSensitivity;
			const FSlateRect PointRect = FSlateRect::FromPointAndExtent(Point.ScreenPosition - CollisionScreenSize/2, CollisionScreenSize);

			if (FSlateRect::DoRectanglesIntersect(PointRect, WidgetRectangle))
			{
				OutPoints->Add(FCurvePointHandle(DrawParams.GetID(), Point.Type, Point.KeyHandle));
				bFound = true;
			}
			else if (Point.LineDelta.X != 0.f || Point.LineDelta.Y != 0.f) //if tangent hit test line
			{
				LinePoints[0] = Point.ScreenPosition + Point.LineDelta.GetSafeNormal() * (DrawInfo.ScreenSize.X * .5f);
				LinePoints[1] = Point.ScreenPosition + Point.LineDelta;

				Start = FVector(LinePoints[0].X, LinePoints[0].Y, 0);
				End = FVector(LinePoints[1].X, LinePoints[1].Y, 0);
				StartToEnd = End - Start;

				if (FMath::LineBoxIntersection(WidgetRectangleBox, Start, End, StartToEnd))
				{
					OutPoints->Add(FCurvePointHandle(DrawParams.GetID(), Point.Type, Point.KeyHandle));
					bFound = true;
				}
			}
		}
	}

	return bFound;
}

bool SInteractiveCurveEditorView::GetCurveWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return false;
	}

	FBox WidgetRectangleBox(FVector(WidgetRectangle.Left, WidgetRectangle.Top, 0), FVector(WidgetRectangle.Right, WidgetRectangle.Bottom, 0));

	const TArray<FCurveDrawParams>& DrawParamsArray = CurveDrawParamsCache->GetCurveDrawParams();

	// Iterate through all of our interpolating points and terminates if one overlaps the marquee. Both of these coordinate systems
	// are in screen space pixels.
	TSet<FCurveModelID> CurveIDs;
	for (const FCurveDrawParams& DrawParams : DrawParamsArray)
	{
		for (int32 InterpolatingPointIndex = 1; InterpolatingPointIndex < DrawParams.InterpolatingPoints.Num(); InterpolatingPointIndex++)
		{
			FVector2D InterpolatingPointPrev = DrawParams.InterpolatingPoints[InterpolatingPointIndex-1];
			FVector2D InterpolatingPointNext = DrawParams.InterpolatingPoints[InterpolatingPointIndex];
			FVector Start(InterpolatingPointPrev.X, InterpolatingPointPrev.Y, 0);
			FVector End(InterpolatingPointNext.X, InterpolatingPointNext.Y, 0);
			FVector StartToEnd = End - Start;

			if (FMath::LineBoxIntersection(WidgetRectangleBox, Start, End, StartToEnd))
			{
				CurveIDs.Add(DrawParams.GetID());
			}
		}
	}

	bool bPointsAdded = false;
	for (const FCurveModelID& CurveID : CurveIDs)
	{
		if (const FCurveModel* Curve = CurveEditor->FindCurve(CurveID))
		{
			for (const FKeyHandle& KeyHandle : Curve->GetAllKeys())
			{
				OutPoints->Add(FCurvePointHandle(CurveID, ECurvePointType::Key, KeyHandle));
				bPointsAdded = true;
			}
		}
	}

	return bPointsAdded;
}

void SInteractiveCurveEditorView::UpdateCurveProximities(FVector2D MousePixel)
{
	TOptional<FCurveModelID> PreviouslyHovered = GetHoveredCurve();

	if (UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FCurveEditorCommands::Get().ScrubTime))
	{
		CurveProximities.Reset();
		CachedToolTipData.Reset();

		if (PreviouslyHovered.IsSet())
		{
			RefreshRetainer();
		}
		return;
	}

	if (DragOperation.IsSet())
	{
		// Don't update while dragging, assume the mouse remains over the hovered curve.
		return;
	}

	CurveProximities.Reset();
	CachedToolTipData.Reset();

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	TOptional<FCurvePointHandle> MousePoint = HitPoint(MousePixel);
	if (MousePoint.IsSet())
	{
		// If the mouse is over a point, that curve is always the closest, so just add that directly and don't
		// bother adding the others
		CurveProximities.Add(MakeTuple(MousePoint->CurveID, 0.f));
	}
	else for (const TTuple<FCurveModelID, FCurveInfo>& Pair : CurveInfoByID)
	{
		const FCurveModel* CurveModel = CurveEditor->FindCurve(Pair.Key);
		if (!ensureAlways(CurveModel))
		{
			continue;
		}

		FCurveEditorScreenSpace CurveSpace = GetCurveSpace(Pair.Key);

		double MinMouseTime = CurveSpace.ScreenToSeconds(MousePixel.X - CurveViewConstants::HoverProximityThresholdPx);
		double MaxMouseTime = CurveSpace.ScreenToSeconds(MousePixel.X + CurveViewConstants::HoverProximityThresholdPx);
		double MouseValue = CurveSpace.ScreenToValue(MousePixel.Y);
		float  PixelsPerOutput = CurveSpace.PixelsPerOutput();

		FVector2D MinPos(MousePixel.X - CurveViewConstants::HoverProximityThresholdPx, 0.0f);
		FVector2D MaxPos(MousePixel.X + CurveViewConstants::HoverProximityThresholdPx, 0.0f);

		double InputOffset = CurveModel->GetInputDisplayOffset();
		double MinEvalTime = MinMouseTime - InputOffset;
		double MaxEvalTime = MaxMouseTime - InputOffset;

		double MinValue = 0.0, MaxValue = 0.0;
		if (CurveModel->Evaluate(MinEvalTime, MinValue) && CurveModel->Evaluate(MaxEvalTime, MaxValue))
		{
			MinPos.Y = CurveSpace.ValueToScreen(MinValue);
			MaxPos.Y = CurveSpace.ValueToScreen(MaxValue);

			const float Distance = (FMath::ClosestPointOnSegment2D(MousePixel, MinPos, MaxPos) - MousePixel).Size();
			if (Distance < CurveViewConstants::HoverProximityThresholdPx)
			{
				CurveProximities.Add(MakeTuple(Pair.Key, Distance));
			}
		}
	}

	Algo::SortBy(CurveProximities, [](TTuple<FCurveModelID, float> In) { return In.Get<1>(); });

	TOptional<FCurveModelID> NewHovered;
	if (CurveProximities.Num() > 0)
	{
		NewHovered = CurveProximities[0].Get<0>();

		// Update tooltips with data from the current mouse position if there is a hovered curve
		const FCurveModel* HoveredCurve = CurveEditor->FindCurve(CurveProximities[0].Get<0>());
		if (HoveredCurve)
		{
			FCurveEditorScreenSpace CurveSpace = GetCurveSpace(CurveProximities[0].Get<0>());
			double MouseTime = CurveSpace.ScreenToSeconds(MousePixel.X) - HoveredCurve->GetInputDisplayOffset();
			double EvaluatedTime = CurveEditor->GetCurveSnapMetrics(CurveProximities[0].Get<0>()).SnapInputSeconds(MouseTime);

			double EvaluatedValue = 0.0;
			HoveredCurve->Evaluate(EvaluatedTime, EvaluatedValue);

			FCachedToolTipData ToolTipData;
			ToolTipData.Text = FormatToolTipCurveName(*HoveredCurve);
			ToolTipData.EvaluatedTime = FormatToolTipTime(*HoveredCurve, EvaluatedTime);
			ToolTipData.EvaluatedValue = FormatToolTipValue(*HoveredCurve, EvaluatedValue);
			
			CachedToolTipData = ToolTipData;
		}
	}

	if (PreviouslyHovered != NewHovered)
	{
		RefreshRetainer();
	}
}

FText SInteractiveCurveEditorView::FormatToolTipCurveName(const FCurveModel& CurveModel) const
{
	return FText::Format(LOCTEXT("CurveEditorTooltipName", "Name: {0}"), CurveModel.GetLongDisplayName());
}

FText SInteractiveCurveEditorView::FormatToolTipTime(const FCurveModel& CurveModel, double EvaluatedTime) const
{
	return FText::Format(LOCTEXT("CurveEditorTime", "Time: {0}"), EvaluatedTime);
}

FText SInteractiveCurveEditorView::FormatToolTipValue(const FCurveModel& CurveModel, double EvaluatedValue) const
{
	return FText::Format(LOCTEXT("CurveEditorValue", "Value: {0}"), EvaluatedValue);
}

void SInteractiveCurveEditorView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{	
	// Unless a drag drop mouse move is reentering, make sure no curve is initially hovered
	if (!DragOperation.IsSet())
	{
		CurveProximities.Reset();
		RefreshRetainer();
	}

	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

void SInteractiveCurveEditorView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	// Unless a drag drop mouse move is leaving, make sure no curve shows hovered
	if (!DragOperation.IsSet())
	{
		CurveProximities.Reset();
		RefreshRetainer();
	}

	SCompoundWidget::OnMouseLeave(MouseEvent);
}

FReply SInteractiveCurveEditorView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<SCurveEditorPanel> EditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditor || !EditorPanel)
	{
		return FReply::Unhandled();
	}

	// Don't handle updating if we have a context menu open.
	if (ActiveContextMenu.Pin())
	{
		return FReply::Unhandled();
	}

	// Cache the mouse position so that commands such as add key can work from command bindings 
	CachedMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (DragOperation.IsSet())
	{
		FVector2D InitialPosition = DragOperation->GetInitialPosition();

		if (!DragOperation->IsDragging() && DragOperation->AttemptDragStart(MouseEvent))
		{
			DragOperation->DragImpl->BeginDrag(InitialPosition, CachedMousePosition, MouseEvent);
			return FReply::Handled().CaptureMouse(AsShared());
		}
		else if (DragOperation->IsDragging())
		{
			bHadMouseMovesThisTick = true;
			DragOperation->DragImpl->Drag(InitialPosition, CachedMousePosition, MouseEvent);
		}
		return FReply::Handled();
	}

	// We don't absorb this event as we're just updating hover states anyways.
	return FReply::Unhandled();
}

void SInteractiveCurveEditorView::OnFinishedPointerInput()
{
	// Update our Curve Proximities for hover states and context actions. This also updates our cached hovered curve.
	UpdateCurveProximities(CachedMousePosition);

	// Some operations defer processing Drag calls for performance reasons. Give them a chance to process the accumulated input.
	if (bHadMouseMovesThisTick && DragOperation && DragOperation->IsDragging())
	{
		DragOperation->DragImpl->FinishedPointerInput();
	}
	bHadMouseMovesThisTick = false;
}

FReply SInteractiveCurveEditorView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor || bFixedOutputBounds)
	{
		return FReply::Unhandled();
	}

	FCurveEditorScreenSpace ViewSpace = GetViewSpace();

	FVector2D MousePixel   = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	double    CurrentTime  = ViewSpace.ScreenToSeconds(MousePixel.X);
	double    CurrentValue = ViewSpace.ScreenToValue(MousePixel.Y);

	// If currently in a drag operation, allow it first chance at handling mouse wheel input
	if (DragOperation.IsSet())
	{
		FVector2D InitialPosition = DragOperation->GetInitialPosition();
		FReply Reply = DragOperation->DragImpl->MouseWheel(InitialPosition, MousePixel, MouseEvent);
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	// Attempt to zoom around the current time if settings specify it and there is a valid time.
	if (CurveEditor->GetSettings()->GetZoomPosition() == ECurveEditorZoomPosition::CurrentTime)
	{
		if (CurveEditor->GetTimeSliderController().IsValid())
		{
			FFrameTime ScrubPosition = CurveEditor->GetTimeSliderController()->GetScrubPosition();
			double PlaybackPosition = ScrubPosition / CurveEditor->GetTimeSliderController()->GetTickResolution();
			if (CurveEditor->GetTimeSliderController()->GetViewRange().Contains(PlaybackPosition))
			{
				CurrentTime = PlaybackPosition;
			}
		}
	}

	const double WheelMultiplier = CurveEditor->GetZoomScaleConfig().GetMouseWheelZoomMultiplierClamped();
	const double ZoomDelta = 1.0 - FMath::Clamp(0.1 * WheelMultiplier * MouseEvent.GetWheelDelta(), -0.9, 0.9);
	ZoomAround(FVector2D(ZoomDelta, ZoomDelta), CurrentTime, CurrentValue);

	return FReply::Handled();
}

TOptional<FCurveModelID> SInteractiveCurveEditorView::GetHoveredCurve() const
{
	if (CurveProximities.Num() > 0 && 
		CurveProximities[0].Get<0>().IsValid())
	{
		return CurveProximities[0].Get<0>();
	}

	return TOptional<FCurveModelID>();
}

bool SInteractiveCurveEditorView::IsToolTipEnabled() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		return (CachedToolTipData.IsSet() && CurveEditor->GetSettings()->GetShowCurveEditorCurveToolTips());
	}

	return false;
}

FText SInteractiveCurveEditorView::GetToolTipCurveName() const
{
	return CachedToolTipData.IsSet() ? CachedToolTipData->Text : FText();
}

FText SInteractiveCurveEditorView::GetToolTipTimeText() const
{
	return CachedToolTipData.IsSet() ? CachedToolTipData->EvaluatedTime : FText();
}

FText SInteractiveCurveEditorView::GetToolTipValueText() const
{
	return CachedToolTipData.IsSet() ? CachedToolTipData->EvaluatedValue : FText();
}

double SInteractiveCurveEditorView::GetTangentValue(const double InTime, const double InValue, FCurveModel* CurveToAddTo, double DeltaTime) const
{
	// Data
	double TargetTime = InTime + DeltaTime;				// The time to get tangent value. Could be left or right depending on is DeltaTime is negative or positive
	double TargetValue = 0.0;							// The helper value to get Tangent value
	CurveToAddTo->Evaluate(TargetTime, TargetValue);	// Initialize TargetValue by TargetTime
	double TangentValue = (TargetValue - InValue) / FMath::Abs(DeltaTime);	// The tangent value to return
	double PrevTangent = DBL_MAX;						// Used for determine whether the tangent is close to the limit
	int32 Count = 10;									// Preventing we stuck in this function for too long

	// Logic
	// While the tangents not close enough and we haven't reach the max iteration time
	while (!FMath::IsNearlyEqual(FMath::Abs(TangentValue), FMath::Abs(PrevTangent)) && Count > 0)
	{
		// Update previous tangent value and make delta time smaller
		PrevTangent = TangentValue;
		DeltaTime /= 2.0;
		TargetTime = InTime + DeltaTime;

		// Calculate a more precise tangent value
		CurveToAddTo->Evaluate(TargetTime, TargetValue);
		TangentValue = (TargetValue - InValue) / FMath::Abs(DeltaTime);

		--Count;
	}
	return TangentValue;
}

void SInteractiveCurveEditorView::HandleDirectKeySelectionByMouse(TSharedPtr<FCurveEditor> CurveEditor, const FPointerEvent& MouseEvent, TOptional<FCurvePointHandle> MouseDownPoint)
{
	if (!MouseDownPoint.IsSet())
	{
		CurveEditor->GetSelection().Clear();
		return;
	}

	const bool bIsShiftDown = MouseEvent.IsShiftDown();
	const bool bIsAltDown = MouseEvent.IsAltDown();
	const bool bIsControlDown = MouseEvent.IsControlDown();
	
	if (bIsShiftDown)
	{
		CurveEditor->GetSelection().Add(MouseDownPoint.GetValue());
	}
	else if (bIsAltDown)
	{
		CurveEditor->GetSelection().Remove(MouseDownPoint.GetValue());
	}
	else if (bIsControlDown)
	{
		CurveEditor->GetSelection().Toggle(MouseDownPoint.GetValue());
	}
	else
	{
		const bool bKeySelected = CurveEditor->GetSelection().Contains(MouseDownPoint->CurveID, MouseDownPoint->KeyHandle, ECurvePointType::Key);
		const bool bLeaveTangentSelected = CurveEditor->GetSelection().Contains(MouseDownPoint->CurveID, MouseDownPoint->KeyHandle, ECurvePointType::LeaveTangent);
		const bool bArriveTangentSelected = CurveEditor->GetSelection().Contains(MouseDownPoint->CurveID, MouseDownPoint->KeyHandle, ECurvePointType::ArriveTangent);

		if (bKeySelected || bLeaveTangentSelected || bArriveTangentSelected)
		{
			// If the picked key handle is already selected in any way, select all of the same point type for the selected points
			if (MouseDownPoint->PointType == ECurvePointType::LeaveTangent)
			{
				TArray<FCurvePointHandle> CurvePointHandles;
				for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
				{
					for (FKeyHandle Handle : Pair.Value.AsArray())
					{
						// If this isn't the opposite of the clicked on LeaveTangent, select the LeaveTangent so it can be moved as well
						if (Pair.Value.PointType(Handle) != ECurvePointType::ArriveTangent)
						{
							FCurvePointHandle CurvePointHandle(Pair.Key, MouseDownPoint->PointType, Handle);
							CurvePointHandles.Add(CurvePointHandle);
						}
					}
				}

				if (!bLeaveTangentSelected)
				{
					CurveEditor->GetSelection().Clear();
				}
				for (FCurvePointHandle CurvePointHandle : CurvePointHandles)
				{
					CurveEditor->GetSelection().Add(CurvePointHandle);
				}
				CurveEditor->GetSelection().Add(MouseDownPoint.GetValue());
			}
			else if (MouseDownPoint->PointType == ECurvePointType::ArriveTangent)
			{
				TArray<FCurvePointHandle> CurvePointHandles;
				for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
				{
					for (FKeyHandle Handle : Pair.Value.AsArray())
					{
						// If this isn't the opposite of the clicked on ArriveTangent, select the ArriveTangent so it can be moved as well
						if (Pair.Value.PointType(Handle) != ECurvePointType::LeaveTangent)
						{
							FCurvePointHandle CurvePointHandle(Pair.Key, MouseDownPoint->PointType, Handle);
							CurvePointHandles.Add(CurvePointHandle);
						}
					}
				}

				if (!bArriveTangentSelected)
				{
					CurveEditor->GetSelection().Clear();
				}
				for (FCurvePointHandle CurvePointHandle : CurvePointHandles)
				{
					CurveEditor->GetSelection().Add(CurvePointHandle);
				}
				CurveEditor->GetSelection().Add(MouseDownPoint.GetValue());
			}
			else if (MouseDownPoint->PointType == ECurvePointType::Key && !bKeySelected)
			{
				TArray<FCurvePointHandle> CurvePointHandles;
				for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
				{
					for (FKeyHandle Handle : Pair.Value.AsArray())
					{
						FCurvePointHandle CurvePointHandle(Pair.Key, MouseDownPoint->PointType, Handle);
						CurvePointHandles.Add(CurvePointHandle);
					}
				}

				CurveEditor->GetSelection().Clear();
				for (FCurvePointHandle CurvePointHandle : CurvePointHandles)
				{
					CurveEditor->GetSelection().Add(CurvePointHandle);
				}
				CurveEditor->GetSelection().Add(MouseDownPoint.GetValue());
			}
		}
			// If this isn't already selected, treat this as a new selection (clear selection)
		else 
		{
			CurveEditor->GetSelection().Clear();
			CurveEditor->GetSelection().Add(MouseDownPoint.GetValue());
		}
	}
}

FReply SInteractiveCurveEditorView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::CurveEditor;
	
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<SCurveEditorPanel> EditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditor || !EditorPanel)
	{
		return FReply::Unhandled();
	}
	
	FVector2D MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	// Cache the mouse position so that commands such as add key can work from command bindings 
	CachedMousePosition = MousePixel;

	// Rebind our context actions so that shift click commands use the right position.
	RebindContextualActions(MousePixel);


	if (CurveEditor->GetSettings()->AllowMouseEdit(MouseEvent))
	{
		// Middle Mouse can try to create keys on curves...
		TOptional<FCurvePointHandle> NewPoint;
		// ... and if MMB causes insertion of a key, the drag operation should append its changes to that same transaction.
		TUniquePtr<FScopedTransaction> RootTransaction;
		
		// Add a key to the closest curve to the mouse
		if (TOptional<FCurveModelID> HoveredCurve = GetHoveredCurve())
		{
			// Don't allow adding keys when shift is held down with selected keys since that is for dragging keys in a constrained axis
			const bool bDraggingKeys = MouseEvent.IsShiftDown() && !CurveEditor->GetSelection().GetAll().IsEmpty();

			// Don't create a new key if there is already a key or transform handle in place
			const bool bKeyAlreadyExists = HitPoint(MousePixel).IsSet();

			FCurveModel* CurveToAddTo = CurveEditor->FindCurve(HoveredCurve.GetValue());
			if (CurveToAddTo && !CurveToAddTo->IsReadOnly() && !bDraggingKeys && !bKeyAlreadyExists && CurveEditor->GetSettings()->GetMiddleMouseButtonAddsKey())
			{
				// Selection and key change should be part of same undo / redo op.
				RootTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("InsertAndMove", "Insert and Move Key"));
				const FScopedSelectionChange SelectionChange(CurveEditor);
				const FScopedCurveChange KeyChange(
					FCurvesSnapshotBuilder(CurveEditor, *HoveredCurve, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::AddKeys)
					);

				FCurveEditorScreenSpace CurveSpace = GetCurveSpace(HoveredCurve.GetValue());
				double MouseTime = CurveSpace.ScreenToSeconds(MousePixel.X);
				double MouseValue = CurveSpace.ScreenToValue(MousePixel.Y);

				FKeyAttributes KeyAttributes = GetDefaultKeyAttributesForCurveTime(*CurveEditor, *CurveToAddTo, MouseTime);

				FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(HoveredCurve.GetValue());
				MouseTime = SnapMetrics.SnapInputSeconds(MouseTime);
				MouseValue = SnapMetrics.SnapOutput(MouseValue);

				// If control is pressed. Keep the curve unchanged
				if (MouseEvent.IsControlDown())
				{
					KeyAttributes.SetTangentMode(RCTM_User);

					// Estimated delta time to compute right and left tangents
					double DeltaTime = 0.1;

					// Make mouse value more accurate 
					CurveToAddTo->Evaluate(MouseTime, MouseValue);

					// Compute right tangent
					double RightTangent = GetTangentValue(MouseTime, MouseValue, CurveToAddTo, DeltaTime);
					KeyAttributes.SetLeaveTangent(RightTangent);

					// Left
					double LeftTangent = GetTangentValue(MouseTime, MouseValue, CurveToAddTo, -DeltaTime);
					KeyAttributes.SetArriveTangent(LeftTangent);
				}

				// When adding to a curve with no variance, add it with the same value so that
				// curves don't pop wildly in normalized views due to a slight difference between the keys
				double CurveOutputMin = 0, CurveOutputMax = 1;
				CurveToAddTo->GetValueRange(CurveOutputMin, CurveOutputMax);
				if (CurveOutputMin == CurveOutputMax)
				{
					MouseValue = CurveOutputMin;
				}

				// Add a key on this curve
				TOptional<FKeyHandle> NewKey = CurveToAddTo->AddKey(FKeyPosition(MouseTime, MouseValue), KeyAttributes);
				if (NewKey.IsSet())
				{
					NewPoint = FCurvePointHandle(HoveredCurve.GetValue(), ECurvePointType::Key, NewKey.GetValue());

					CurveEditor->GetSelection().Clear();
					CurveEditor->GetSelection().Add(NewPoint.GetValue());
				}
				else
				{
					RootTransaction->Cancel();
					RootTransaction.Reset();
				}
			}
		}

		// If there are any tangent handles selected, prefer to drag those instead of keys
		ECurvePointType PointType = ECurvePointType::Key;
		if (!NewPoint.IsSet())
		{
			for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
			{
				for (FKeyHandle Handle : Pair.Value.AsArray())
				{
					if (Pair.Value.Contains(Handle, ECurvePointType::ArriveTangent) || Pair.Value.Contains(Handle, ECurvePointType::LeaveTangent))
					{
						PointType = ECurvePointType::ArriveTangent;
						break;
					}
				}
			}
		}
		
		DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
		DragOperation->DragImpl = CreateAndInitializeKeyDrag(*CurveEditor, PointType, NewPoint);
		// If MMB caused a key to be inserted above, append the drag operation changes to that transaction. 
		// Otherwise, we'll create a new transaction, whose title depends on whether a key or tangents are selected atm.
		DragOperation->Transaction = RootTransaction ? MoveTemp(RootTransaction) : CreateKeyOperationTransaction(*CurveEditor, PointType);

		return FReply::Handled().PreventThrottling();
	}
	// Alt RMB always Zoom Timeline
	else if (MouseEvent.IsAltDown() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
		DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_Zoom>(CurveEditor.Get(), SharedThis(this));
		return FReply::Handled();
	}
	else if (CurveEditor->GetSettings()->AllowMousePan(MouseEvent))
	{
		if (!bFixedOutputBounds)
		{
			DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
			DragOperation->DragImpl = MakeUnique<FCurveEditorDragOperation_PanView>(CurveEditor.Get(), SharedThis(this));
			return FReply::Handled();
		}
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// Key Selection Testing
		TOptional<FCurvePointHandle> MouseDownPoint = HitPoint(MousePixel);
		if (MouseDownPoint.IsSet())
		{
			if (FCurveModel* CurveModel = CurveEditor->FindCurve(MouseDownPoint->CurveID))
			{
				if (!CurveModel->IsReadOnly())
				{
					// Intention: The "Click Key" action must be part of the same transaction as the change made by the operation
					// First, we select the keys...
					const FScopedSelectionChange Transaction(WeakCurveEditor, LOCTEXT("ClickKey", "Click Key"));
					HandleDirectKeySelectionByMouse(CurveEditor, MouseEvent, MouseDownPoint);

					DragOperation = FCurveEditorDelayedDrag(MousePixel, MouseEvent.GetEffectingButton());
					DragOperation->DragImpl = CreateAndInitializeKeyDrag(*CurveEditor, MouseDownPoint->PointType, MouseDownPoint);
					// ... then we'll create the transaction, whose title depends on the number of keys selected...
					DragOperation->Transaction = CreateKeyOperationTransaction(*CurveEditor, MouseDownPoint->PointType);

					return FReply::Handled().PreventThrottling();
					// ... and finally, ~FScopedSelectionChange appends its command to the currently open transaction.
				}
			}
		}
	}

	return FReply::Unhandled();
}

void SInteractiveCurveEditorView::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	// Delaying nulling out until next tick because this could be invoked during OnMouseMove()
#if WITH_EDITOR
	GEditor->GetTimerManager()->SetTimerForNextTick([this]()
	{
		if (!HasMouseCapture())
		{
			DragOperation.Reset();
		}
	});
#endif
}

FReply SInteractiveCurveEditorView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<SCurveEditorPanel> EditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditor || !EditorPanel)
	{
		return FReply::Unhandled();
	}

	const bool bDragOperationWasSet = DragOperation.IsSet();
	const bool bWasDragging = DragOperation.IsSet() && DragOperation->IsDragging();
	FVector2D MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	if (bWasDragging)
	{
		FVector2D InitialPosition = DragOperation->GetInitialPosition();
		DragOperation->DragImpl->EndDrag(InitialPosition, MousePosition, MouseEvent);

		DragOperation.Reset();
		return FReply::Handled().ReleaseMouseCapture();
	}

	DragOperation.Reset();

	// Select the curve on mouse release if no key or tangent was clicked on
	if (!bWasDragging && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const bool bIsShiftDown = MouseEvent.IsShiftDown();
		const bool bIsAltDown = MouseEvent.IsAltDown();
		const bool bIsControlDown = MouseEvent.IsControlDown();

		// Curve Selection Testing.
		TOptional<FCurveModelID> HitCurve = GetHoveredCurve();
		if (!HitPoint(MousePosition).IsSet() && HitCurve.IsSet())
		{
			FCurveModel* CurveModel = CurveEditor->FindCurve(HitCurve.GetValue());
			const TArray<FKeyHandle> KeyHandles = CurveModel->GetAllKeys();

			// Only open a transaction if this click would actually change the selection.
			const FCurveEditorSelection& CurrentSelection = CurveEditor->GetSelection();
			const FKeyHandleSet* CurveKeys = CurrentSelection.FindForCurve(HitCurve.GetValue());
			const int32 NumCurveKeysSelected = CurveKeys ? CurveKeys->Num() : 0;

			bool bWouldChangeSelection;
			if (bIsShiftDown)
			{
				bWouldChangeSelection = NumCurveKeysSelected < KeyHandles.Num();
			}
			else if (bIsAltDown)
			{
				bWouldChangeSelection = NumCurveKeysSelected > 0;
			}
			else if (bIsControlDown)
			{
				bWouldChangeSelection = KeyHandles.Num() > 0;
			}
			else
			{
				bWouldChangeSelection = CurrentSelection.Count() != KeyHandles.Num() || NumCurveKeysSelected != KeyHandles.Num();
			}

			if (bWouldChangeSelection)
			{
				const UE::CurveEditor::FScopedSelectionChange Transaction(WeakCurveEditor, LOCTEXT("ClickCurve", "Click Curve"));

				// Add or remove all keys from the curve.
				if (bIsShiftDown)
				{
					CurveEditor->GetSelection().Add(HitCurve.GetValue(), ECurvePointType::Key, KeyHandles);
				}
				else if (bIsAltDown)
				{
					CurveEditor->GetSelection().Remove(HitCurve.GetValue(), ECurvePointType::Key, KeyHandles);
				}
				else if (bIsControlDown)
				{
					CurveEditor->GetSelection().Toggle(HitCurve.GetValue(), ECurvePointType::Key, KeyHandles);
				}
				else
				{
					CurveEditor->GetSelection().Clear();
					CurveEditor->GetSelection().Add(HitCurve.GetValue(), ECurvePointType::Key, KeyHandles);
				}
			}

			return FReply::Handled();
		}
	}

	// Pop up a menu if not dragging
	if (!bWasDragging && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FVector2D MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		TOptional<FCurvePointHandle> MouseDownPoint = HitPoint(MousePixel);

		if(MouseDownPoint.IsSet())
		{
			if (FCurveModel* CurveModel = CurveEditor->FindCurve(MouseDownPoint->CurveID))
			{
				if (!CurveModel->IsReadOnly())
				{
					const UE::CurveEditor::FScopedSelectionChange Transaction(WeakCurveEditor, LOCTEXT("ClickKey", "Click Key"));
					HandleDirectKeySelectionByMouse(CurveEditor, MouseEvent, MouseDownPoint);
				}
			}
		}
		
		CreateContextMenu(MyGeometry, MouseEvent);
		return FReply::Handled();
	}

	// If we hit a curve or another UI element, do not allow mouse input to bubble
	if (HitPoint(MousePosition) || GetHoveredCurve())
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SInteractiveCurveEditorView::CreateContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<SCurveEditorPanel> EditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditor || !EditorPanel)
	{
		return;
	}

	FVector2D MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	TOptional<FCurvePointHandle> MouseUpPoint = HitPoint(MousePosition);
	
	// We need to update our curve proximities (again) because OnMouseLeave is called (which clears them) 
	// before this menu is created due to the parent widget capturing mouse focus. The context menu needs
	// to know which curve you have highlighted for buffering curves.
	UpdateCurveProximities(MousePosition);

	// Rebind our context menu actions based on the results of hit-testing
	RebindContextualActions(MousePosition);

	// if (!MouseUpPoint.IsSet())
	// {
	// 	CurveEditor->Selection.Clear();
	// }

	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, EditorPanel->GetCommands());
	
	BuildContextMenu(MenuBuilder, MouseUpPoint, GetHoveredCurve());

	// Push the context menu
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	ActiveContextMenu = FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

void SInteractiveCurveEditorView::BuildContextMenu(FMenuBuilder& MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurveID)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		FCurveEditorContextMenu::BuildMenu(MenuBuilder, CurveEditor.ToSharedRef(), ClickedPoint, HoveredCurveID);
	}
}

TOptional<FCurvePointHandle> SInteractiveCurveEditorView::HitPoint(FVector2D MousePixel) const
{
	if (!WeakCurveEditor.IsValid())
	{
		return TOptional<FCurvePointHandle>();
	}
	const TSharedRef<FCurveEditor> CurveEditor = WeakCurveEditor.Pin().ToSharedRef();


	TOptional<FCurvePointHandle> HitPoint;
	TOptional<float> ClosestDistance;

	TOptional<FCurveModelID> HoveredCurve = GetHoveredCurve();

	const TArray<FCurveDrawParams>& DrawParamsArray = CurveDrawParamsCache->GetCurveDrawParams();

	// Find all keys within the current hit test time
	for (const FCurveDrawParams& DrawParams : DrawParamsArray)
	{
		const FCurveModelID& ModelID = DrawParams.GetID();

		// If we have a hovered curve, only hit a point within that curve
		if (HoveredCurve.IsSet() && ModelID != HoveredCurve.GetValue())
		{
			continue;
		}
	
		for (int32 PointIndex = 0; PointIndex < DrawParams.Points.Num(); PointIndex++)
		{
			const FCurvePointInfo& Point = DrawParams.Points[PointIndex];
			const FKeyDrawInfo& PointDrawInfo = DrawParams.GetKeyDrawInfo(Point.Type, PointIndex);

			// We artificially inflate the hit testing region for keys by a few pixels to make them easier to hit. The PointDrawInfo.ScreenSize specifies their drawn size,
			// so we need to inflate here when doing the actual hit testing. We subtract by half the extent to center it on the drawing.
			FVector2D HitTestSize = PointDrawInfo.ScreenSize + FVector2D(4.f, 4.f);

			FSlateRect KeyRect = FSlateRect::FromPointAndExtent(Point.ScreenPosition - (HitTestSize / 2.f), HitTestSize);

			if (KeyRect.ContainsPoint(MousePixel))
			{
				float DistanceSquared = (KeyRect.GetCenter() - MousePixel).SizeSquared();
				if (DistanceSquared <= ClosestDistance.Get(DistanceSquared))
				{
					ClosestDistance = DistanceSquared;
					HitPoint = FCurvePointHandle(DrawParams.GetID(), Point.Type, Point.KeyHandle);
				}
			}
		}
	}

	return HitPoint;
}

void SInteractiveCurveEditorView::RebindContextualActions(FVector2D InMousePosition)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	TSharedPtr<SCurveEditorPanel> CurveEditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditorPanel)
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = CurveEditorPanel->GetCommands();

	CommandList->UnmapAction(FCurveEditorCommands::Get().AddKeyHovered);
	CommandList->UnmapAction(FCurveEditorCommands::Get().PasteKeysHovered);
	CommandList->UnmapAction(FCurveEditorCommands::Get().AddKeyToAllCurves);
	CommandList->UnmapAction(FCurveEditorCommands::Get().AddKeyToSelectedCurves);
	CommandList->UnmapAction(FCurveEditorCommands::Get().ResetKeysToDefaultValues);
	CommandList->UnmapAction(FCurveEditorCommands::Get().BufferVisibleCurves);
	CommandList->UnmapAction(FCurveEditorCommands::Get().SwapBufferedCurves);
	CommandList->UnmapAction(FCurveEditorCommands::Get().ApplyBufferedCurves);
	

	TOptional<FCurveModelID> HoveredCurve = GetHoveredCurve();
	if (HoveredCurve.IsSet())
	{
		TSet<FCurveModelID> HoveredCurveSet;
		HoveredCurveSet.Add(HoveredCurve.GetValue());

		CommandList->MapAction(FCurveEditorCommands::Get().AddKeyHovered, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::AddKeyAtMousePosition, HoveredCurveSet));
		CommandList->MapAction(FCurveEditorCommands::Get().PasteKeysHovered, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::PasteKeys, HoveredCurveSet));
	}

	CommandList->MapAction(FCurveEditorCommands::Get().AddKeyToAllCurves, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::AddKeyAtScrubTime, false));

	CommandList->MapAction(
		FCurveEditorCommands::Get().AddKeyToSelectedCurves,
		FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::AddKeyAtScrubTime, true /*use selection*/));

	CommandList->MapAction(
		FCurveEditorCommands::Get().ResetKeysToDefaultValues,
		FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::ResetKeysToDefaultValues, true /*use selection*/));
		

	// Buffer Curves. Can only act on buffered curves if curves are selected in the tree or the curve has selected keys.
	CommandList->MapAction(FCurveEditorCommands::Get().BufferVisibleCurves, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::BufferCurves), FCanExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::CanBufferedCurves));
	CommandList->MapAction(FCurveEditorCommands::Get().SwapBufferedCurves, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::ApplyBufferCurves, true), FCanExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::CanApplyBufferedCurves));
	CommandList->MapAction(FCurveEditorCommands::Get().ApplyBufferedCurves, FExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::ApplyBufferCurves, false), FCanExecuteAction::CreateSP(this, &SInteractiveCurveEditorView::CanApplyBufferedCurves));
}

void SInteractiveCurveEditorView::BufferCurves()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		CurveEditor->AddBufferedCurves(CurveEditor->GetSelectionFromTreeAndKeys());

		// If the user had previously buffered the curve, moved it, and now re-buffers it again, we need to trigger a re-draw.
		if (CurveEditor->GetSettings()->GetShowBufferedCurves())
		{
			RefreshRetainer(); 
		}
	}
}

void SInteractiveCurveEditorView::ApplyBufferCurves(const bool bSwapBufferCurves)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		CurveEditor->ApplyBufferedCurves(CurveEditor->GetSelectionFromTreeAndKeys(), bSwapBufferCurves);
	}
}

bool SInteractiveCurveEditorView::CanBufferedCurves() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		return CurveEditor->GetSelectionFromTreeAndKeys().Num() > 0;
	}

	return false;
}

bool SInteractiveCurveEditorView::CanApplyBufferedCurves() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		return CurveEditor->GetSelectionFromTreeAndKeys().Num() > 0 && CurveEditor->GetBufferedCurves().Num() > 0;
	}

	return false;
}

FKeyAttributes SInteractiveCurveEditorView::GetDefaultKeyAttributesForCurveTime(const FCurveEditor& CurveEditor, const FCurveModel& CurveModel, double EvalTime) const
{
	FKeyAttributes KeyAttributes = CurveEditor.GetDefaultKeyAttribute().Get();

	// Give the model a chance to override the default interpolation modes
    TPair<ERichCurveInterpMode, ERichCurveTangentMode> Modes = CurveModel.GetInterpolationMode(
    	EvalTime,
    	KeyAttributes.HasInterpMode() ? KeyAttributes.GetInterpMode() : RCIM_Linear,
    	KeyAttributes.HasTangentMode() ? KeyAttributes.GetTangentMode() : RCTM_Auto
    	);

	if (const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>())
	{
		if (Settings->GetCubicAddKeyPrefersUserTangent())
		{
			if (Modes.Key == RCIM_Cubic)
			{
				Modes.Value = RCTM_User;
			}
		}
	}
    KeyAttributes.SetInterpMode(Modes.Key);
    KeyAttributes.SetTangentMode(Modes.Value);
	
    return KeyAttributes;
}

void SInteractiveCurveEditorView::ResetKeysToDefaultValues(bool bUseSelection)
{
	using namespace UE::CurveEditor;

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	TSet<FCurveModelID> CurvesToReset;
	if (bUseSelection == false)
	{
		CurvesToReset = CurveEditor->GetEditedCurves();
	}
	else
	{
		for (const auto& Pair : CurveEditor->GetSelection().GetAll())
		{
			CurvesToReset.Add(Pair.Key);
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ResetKeysToDefaultValues", "Set Default Value")); 
	FScopedSelectionChange SelectionChange(WeakCurveEditor);
	FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(WeakCurveEditor, CurvesToReset, ECurveChangeFlags::MoveKeys)
	);

	FCurveEditorSelection& Selection = CurveEditor->GetSelection();
	TArray<FKeyHandle> KeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;
	for (FCurveModelID CurveID : CurvesToReset)
	{
		if (FCurveModel* CurveModel = CurveEditor->FindCurve(CurveID))
		{
			if (CurveModel->HasDefaultValue())
			{
				double DefaultValue = CurveModel->GetDefaultValue();
				if (const FKeyHandleSet* SelectionSet = Selection.FindForCurve(CurveID))
				{
					if (SelectionSet->Num() > 0)
					{
						KeyHandles = SelectionSet->AsArray();
						SelectedKeyPositions.SetNum(KeyHandles.Num());
						CurveModel->GetKeyPositions(KeyHandles, SelectedKeyPositions);
						for (FKeyPosition &Key : SelectedKeyPositions)
						{
							Key.OutputValue = DefaultValue;
						}
						CurveModel->SetKeyPositions(KeyHandles, SelectedKeyPositions);
					}
				}
			}
		}
	}
}


void SInteractiveCurveEditorView::AddKeyAtScrubTime(bool bUseSelection)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	TSet<FCurveModelID> CurvesToAddTo;
	if (bUseSelection == false)
	{
		CurvesToAddTo = CurveEditor->GetEditedCurves();
	}
	else
	{
		for (const auto& Pair : CurveEditor->GetSelection().GetAll())
		{
			CurvesToAddTo.Add(Pair.Key);
		}
	}

	// If they don't have a time slider controller then we fall back to using mouse position.
	TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (!TimeSliderController)
	{
		AddKeyAtMousePosition(CurvesToAddTo);
		return;
	}

	// Snapping of the time will be done inside AddKeyAtTime.
	double ScrubTime = TimeSliderController->GetScrubPosition() / TimeSliderController->GetTickResolution();
	AddKeyAtTime(CurvesToAddTo, ScrubTime);
}

void SInteractiveCurveEditorView::AddKeyAtMousePosition(TSet<FCurveModelID> ForCurves)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	// Snapping will be done inside AddKeyAtTime
	double MouseTime = GetViewSpace().ScreenToSeconds(CachedMousePosition.X);
	AddKeyAtTime(ForCurves, MouseTime);
}

void SInteractiveCurveEditorView::AddKeyAtTime(const TSet<FCurveModelID>& ToCurves, double InTime)
{
	using namespace UE::CurveEditor;
	
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddKeyAtTime", "Add Key")); // Selection and key change should be part of same undo / redo op.
	FScopedSelectionChange SelectionChange(WeakCurveEditor);
	FScopedCurveChange KeyChange(
		FCurvesSnapshotBuilder(WeakCurveEditor, ToCurves, ECurveChangeFlags::AddKeys)
		);
	
	bool bAddedKey = false;

	// Clear the selection set as we will be selecting all the new keys created.
	CurveEditor->GetSelection().Clear();

	for (const FCurveModelID& CurveModelID : ToCurves)
	{
		FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID);
		check(CurveModel);

		if (CurveModel->IsReadOnly())
		{
			continue;
		}

		// Ensure the time is snapped if needed
		FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(CurveModelID);
		double SnappedTime = SnapMetrics.SnapInputSeconds(InTime);

		// Support optional input display offsets 
		double EvalTime = SnappedTime - CurveModel->GetInputDisplayOffset();

		double CurveValue = 0.0;
		if (CurveModel->Evaluate(EvalTime, CurveValue))
		{
			CurveValue = SnapMetrics.SnapOutput(CurveValue);

			// Curve Models allow us to create new keys ontop of existing keys which works, but causes some user confusion
			// Before we create a key, we instead check to see if there is already a key at this time, and if there is, we
			// add that key to the selection set instead. This solves issues with snapping causing keys to be created adjacent
			// to the mouse cursor (sometimes by a large amount).
			TArray<FKeyHandle> ExistingKeys;
			CurveModel->GetKeys(EvalTime - KINDA_SMALL_NUMBER, EvalTime + KINDA_SMALL_NUMBER, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), ExistingKeys);
			
			TOptional<FKeyHandle> NewKey;

			if (ExistingKeys.Num() > 0)
			{
				NewKey = ExistingKeys[0];
			}
			else
			{
				FKeyAttributes KeyAttributes = GetDefaultKeyAttributesForCurveTime(*CurveEditor, *CurveModel, EvalTime);
				if (KeyAttributes.HasInterpMode() && 
					KeyAttributes.GetInterpMode() == ERichCurveInterpMode::RCIM_Cubic&& 
					KeyAttributes.HasTangentMode() && 
					(KeyAttributes.GetTangentMode() == RCTM_User || KeyAttributes.GetTangentMode() == RCTM_Break))
				{
					//if we are within the range of existing keys set the tangent to be that of the slope of the curve, otherwise
					//just set it as flat
					double MinTime = 0., MaxTime = 0.;
					CurveModel->GetTimeRange(MinTime, MaxTime);
					if (EvalTime > MinTime && EvalTime < MaxTime)
					{
						const double DeltaTime = 0.1;

						// Compute right tangent
						double RightTangent = GetTangentValue(EvalTime, CurveValue, CurveModel, DeltaTime);
						KeyAttributes.SetLeaveTangent(RightTangent);

						// Left
						double LeftTangent = GetTangentValue(EvalTime, CurveValue, CurveModel, -DeltaTime);
						KeyAttributes.SetArriveTangent(LeftTangent);
					}
					else
					{
						KeyAttributes.SetLeaveTangent(0.0);
						KeyAttributes.SetArriveTangent(0.0);
					}
				}
				// Add a key on this curve
				NewKey = CurveModel->AddKey(FKeyPosition(EvalTime, CurveValue), KeyAttributes);
			}

			// Add the key to the selection set.
			if (NewKey.IsSet())
			{
				bAddedKey = true;
				CurveEditor->GetSelection().Add(FCurvePointHandle(CurveModelID, ECurvePointType::Key, NewKey.GetValue()));
			}
		}
	}

	if (!bAddedKey)
	{
		Transaction.Cancel();
	}
}

void SInteractiveCurveEditorView::PasteKeys(TSet<FCurveModelID> ToCurves)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	CurveEditor->PasteKeys(UE::CurveEditor::FKeyPasteArgs{ ToCurves });
}

void SInteractiveCurveEditorView::OnCurveEditorToolChanged(FCurveEditorToolID InToolId)
{
	// We need to end drag-drop operations if they switch tools. Otherwise they can start
	// a marquee select, use the keyboard to switch to a diferent tool, and then the marquee
	// select finishes after the tool has had a chance to activate.
	if (DragOperation.IsSet())
	{
		// We have to cancel it instead of ending it because ending it needs mouse position and some other stuff.
		DragOperation->Cancel();
		DragOperation.Reset();
	}
}

void SInteractiveCurveEditorView::OnShowValueIndicatorsChanged()
{
	// Setting bNeedsRefresh = true is not enough because we may not get ticked.
	RefreshRetainer();
}

#undef LOCTEXT_NAMESPACE // "SInteractiveCurveEditorView"
