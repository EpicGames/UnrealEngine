// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "Curves/RichCurve.h"
#include "Math/TransformCalculus2D.h"
#include "CurveDataAbstraction.generated.h"

struct FSlateBrush;

/**
 * Generic key position information for a key on a curve
 */
USTRUCT()
struct FKeyPosition
{
	GENERATED_BODY()

	FKeyPosition()
		: InputValue(0.0), OutputValue(0.0)
	{}

	FKeyPosition(double Input, double Output)
		: InputValue(Input), OutputValue(Output)
	{}

	FKeyPosition Transform(const FTransform2d& InTransform) const
	{
		FVector2d Point(InputValue, OutputValue);
		Point = InTransform.TransformPoint(Point);
		return FKeyPosition(Point.X, Point.Y);
	}

	/** The key's input (x-axis) position (i.e. it's time) */
	UPROPERTY()
	double InputValue;

	/** The key's output (t-axis) position (i.e. it's value) */
	UPROPERTY()
	double OutputValue;
	
	bool operator==(const FKeyPosition& Other) const = default;
	bool operator!=(const FKeyPosition& Other) const = default;
};

/**
 * Extended attributes that the curve editor understands
 */
USTRUCT()
struct FKeyAttributes
{
	GENERATED_BODY()
		
	FKeyAttributes()
	{
		bHasArriveTangent = 0;
		bHasLeaveTangent  = 0;
		bHasInterpMode    = 0;
		bHasTangentMode   = 0;
		bHasTangentWeightMode = 0;
		bHasArriveTangentWeight = 0;
		bHasLeaveTangentWeight = 0;
		ArriveTangent = 0.0f;
		LeaveTangent = 0.0f;
		InterpMode = RCIM_Linear;
		TangentMode = RCTM_Auto;
		TangentWeightMode = RCTWM_WeightedNone;
		ArriveTangentWeight = 0.0f;
		LeaveTangentWeight = 0.0f;
	}

	/**
	 * Check whether this key has the specified attributes
	 */
	bool HasArriveTangent() const		{ return bHasArriveTangent;		}
	bool HasLeaveTangent() const		{ return bHasLeaveTangent;		}
	bool HasInterpMode() const			{ return bHasInterpMode;		}
	bool HasTangentMode() const			{ return bHasTangentMode;		}
	bool HasTangentWeightMode() const	{ return bHasTangentWeightMode; }
	bool HasArriveTangentWeight() const { return bHasArriveTangentWeight; }
	bool HasLeaveTangentWeight() const  { return bHasLeaveTangentWeight; }
	
	/**
	 * Retrieve specific attributes for this key. Must check such attributes exist.
	 */
	float GetArriveTangent() const							{ check(bHasArriveTangent); return ArriveTangent; }	
	float GetLeaveTangent() const							{ check(bHasLeaveTangent);  return LeaveTangent;  }
	ERichCurveInterpMode GetInterpMode() const				{ check(bHasInterpMode);    return InterpMode;    }
	ERichCurveTangentMode GetTangentMode() const			{ check(bHasTangentMode);   return TangentMode;   }
	ERichCurveTangentWeightMode GetTangentWeightMode()const { check(bHasTangentWeightMode);   return TangentWeightMode; }
	float GetArriveTangentWeight() const { check(bHasArriveTangentWeight); return ArriveTangentWeight; }
	float GetLeaveTangentWeight() const { check(bHasLeaveTangentWeight);  return LeaveTangentWeight; }

	/**
	 * Set the attributes for this key
	 */
	FKeyAttributes& SetArriveTangent(float InArriveTangent)             { bHasArriveTangent = 1;    ArriveTangent = InArriveTangent; return *this; }
	FKeyAttributes& SetLeaveTangent(float InLeaveTangent)               { bHasLeaveTangent = 1;     LeaveTangent = InLeaveTangent;   return *this; }
	FKeyAttributes& SetInterpMode(ERichCurveInterpMode InInterpMode)    { bHasInterpMode = 1;       InterpMode = InInterpMode;       return *this; }
	FKeyAttributes& SetTangentMode(ERichCurveTangentMode InTangentMode) { bHasTangentMode = 1;      TangentMode = InTangentMode;     return *this; }
	FKeyAttributes& SetTangentWeightMode(ERichCurveTangentWeightMode InTangentWeightMode) { bHasTangentWeightMode = 1;      TangentWeightMode = InTangentWeightMode;     return *this; }
	FKeyAttributes& SetArriveTangentWeight(float InArriveTangentWeight) { bHasArriveTangentWeight = 1;    ArriveTangentWeight = InArriveTangentWeight; return *this; }
	FKeyAttributes& SetLeaveTangentWeight(float InLeaveTangentWeight) { bHasLeaveTangentWeight = 1;     LeaveTangentWeight = InLeaveTangentWeight;   return *this; }

	/**
	 * Reset specific attributes of this key, implying such attributes are not supported
	 */
	void UnsetArriveTangent() { bHasArriveTangent = 0; }
	void UnsetLeaveTangent()  { bHasLeaveTangent = 0;  }
	void UnsetInterpMode()    { bHasInterpMode = 0;    }
	void UnsetTangentMode()   { bHasTangentMode = 0;   }
	void UnsetTangentWeightMode() { bHasTangentWeightMode = 0; }
	void UnsetArriveTangentWeight() { bHasArriveTangentWeight = 0; }
	void UnsetLeaveTangentWeight() { bHasLeaveTangentWeight = 0; }

	/**
	 * Generate a new set of attributes that contains only those attributes common to both A and B
	 */
	static FKeyAttributes MaskCommon(const FKeyAttributes& A, const FKeyAttributes& B)
	{
		FKeyAttributes NewAttributes;
		if (A.bHasArriveTangent && B.bHasArriveTangent && A.ArriveTangent == B.ArriveTangent)
		{
			NewAttributes.SetArriveTangent(A.ArriveTangent);
		}

		if (A.bHasLeaveTangent && B.bHasLeaveTangent && A.LeaveTangent == B.LeaveTangent)
		{
			NewAttributes.SetLeaveTangent(A.LeaveTangent);
		}

		if (A.bHasInterpMode && B.bHasInterpMode && A.InterpMode == B.InterpMode)
		{
			NewAttributes.SetInterpMode(A.InterpMode);
		}

		if (A.bHasTangentMode && B.bHasTangentMode && A.TangentMode == B.TangentMode)
		{
			NewAttributes.SetTangentMode(A.TangentMode);
		}

		if (A.bHasTangentWeightMode && B.bHasTangentWeightMode && A.TangentWeightMode == B.TangentWeightMode)
		{
			NewAttributes.SetTangentWeightMode(A.TangentWeightMode);
		}

		if (A.bHasArriveTangentWeight && B.bHasArriveTangentWeight && A.ArriveTangentWeight == B.ArriveTangentWeight)
		{
			NewAttributes.SetArriveTangentWeight(A.ArriveTangentWeight);
		}

		if (A.bHasLeaveTangentWeight && B.bHasLeaveTangentWeight && A.LeaveTangentWeight == B.LeaveTangentWeight)
		{
			NewAttributes.SetLeaveTangentWeight(A.LeaveTangentWeight);
		}
		return NewAttributes;
	}

	bool operator==(const FKeyAttributes& Other) const = default;
	bool operator!=(const FKeyAttributes& Other) const = default;

private:

	/** True if this key supports entry tangents */
	UPROPERTY()
	uint8 bHasArriveTangent : 1;
	/** True if this key supports exit tangents */
	UPROPERTY()
	uint8 bHasLeaveTangent : 1;
	/** True if this key supports interpolation modes */
	UPROPERTY()
	uint8 bHasInterpMode : 1;
	/** True if this key supports tangent modes */
	UPROPERTY()
	uint8 bHasTangentMode : 1;
	/** True if this key supports tangent modes */
	UPROPERTY()
	uint8 bHasTangentWeightMode : 1;
	/** True if this key supports entry tangents weights*/
	UPROPERTY()
	uint8 bHasArriveTangentWeight : 1;
	/** True if this key supports exit tangents weights*/
	UPROPERTY()
	uint8 bHasLeaveTangentWeight : 1;

	/** This key's entry tangent, if bHasArriveTangent */
	UPROPERTY()
	float ArriveTangent{};
	/** This key's exit tangent, if bHasLeaveTangent */
	UPROPERTY()
	float LeaveTangent{};
	/** This key's interpolation mode, if bHasInterpMode */
	UPROPERTY()
	TEnumAsByte<ERichCurveInterpMode> InterpMode;
	/** This key's tangent mode, if bHasTangentMode */
	UPROPERTY()
	TEnumAsByte<ERichCurveTangentMode> TangentMode;
	/** This key's tangent weight mode, if bHasTangentWeightMode */
	UPROPERTY()
	TEnumAsByte<ERichCurveTangentWeightMode> TangentWeightMode;
	/** This key's entry tangent weight, if bHasArriveTangentWeight */
	UPROPERTY()
	float ArriveTangentWeight{};
	/** This key's exit tangent weight, if bHasLeaveTangentWeight */
	UPROPERTY()
	float LeaveTangentWeight{};

};
