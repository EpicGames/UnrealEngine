// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Curves/KeyHandle.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "CurveEditorTypes.h"

struct FSlateBrush;

/**
 * Structure that defines how to draw a particular key
 */
struct FKeyDrawInfo
{
	FKeyDrawInfo()
		: ScreenSize(0.f, 0.f)
		, Brush(nullptr)
	{}

	/** The size of the key on screen in slate units */
	FVector2D ScreenSize;

	/** The brush to use to draw the key */
	const FSlateBrush* Brush;

	/** A tint to apply to the brush */
	TOptional<FLinearColor> Tint;
};



/**
 * Structure that defines the necessary data for painting a given curve point
 */
struct FCurvePointInfo
{
	FCurvePointInfo(FKeyHandle InHandle)
		: KeyHandle(InHandle)
		, ScreenPosition(0.f, 0.f)
		, LineDelta(0.f, 0.f)
		, Type(ECurvePointType::Key)
		, LayerBias(0)
	{}

	/** This point's key handle */
	FKeyHandle KeyHandle;
	/** The position of the point on screen */
	FVector2D ScreenPosition;
	/** A screen space delta position that defines where to draw a line connected to this point. No line is drawn if zero. */
	FVector2D LineDelta;
	/** The type of the point */
	ECurvePointType Type;
	/** A layer bias to draw the point with (higher integers draw on top) */
	int32 LayerBias;
	/** DrawInfo to use when rendering this point. Allows customization of points on a per-point basis instead of per-curve. */
	FKeyDrawInfo DrawInfo;
	/** If false, the key will not be painted. */
	bool bDraw = true;
};


/**
 * Structure that defines the necessary data for painting a whole curve
 */
struct FCurveDrawParams
{
	/** The color to draw this curve */
	FLinearColor Color;

	/** An array of screen-space points that define this curve's shape. Rendered as a continuous line. */
	TArray<FVector2D> InterpolatingPoints;

	/** An array of distinct curve points for the visible range. */
	TArray<FCurvePointInfo> Points;

	/** Value defining how to draw keys of type ECurvePointType::ArriveTangent. */
	FKeyDrawInfo ArriveTangentDrawInfo;

	/** Value defining how to draw keys of type ECurvePointType::LeaveTangent. */
	FKeyDrawInfo LeaveTangentDrawInfo;

	/** Thickness of this curve */
	float Thickness = 2.f;

	/** The color to draw this curve */
	float DashLengthPx = 0.f;

	/** Whether or not to draw keys on the curve. */
	uint8 bKeyDrawEnabled : 1;

	/** If false, the interpolating points will not be painted. */
	uint8 bDrawInterpolatingPoints : 1 = true;

	/**
	 * Construct new draw parameters for the specified curve ID
	 */
	FCurveDrawParams(FCurveModelID InID)
		: Color(0.2f, 0.2f, 0.2f)
		, bKeyDrawEnabled(1)
		, ID(InID)
	{}

	/**
	 * Get the curve ID that these draw parameters relate to
	 */
	FCurveModelID GetID() const
	{
		return ID;
	}

	/**
	 * Retrieve the draw information for drawing the specified type of curve point
	 */
	const FKeyDrawInfo& GetKeyDrawInfo(ECurvePointType Type, const int32 InPointIndex) const
	{
		switch (Type)
		{
		case ECurvePointType::ArriveTangent: return ArriveTangentDrawInfo;
		case ECurvePointType::LeaveTangent:  return LeaveTangentDrawInfo;
		case ECurvePointType::Key:			 return Points[InPointIndex].DrawInfo;
		}
		return InvalidDrawInfo;
	}

private:

	/** Immutable curve ID */
	FCurveModelID ID;

	/** Immutable dummy key draw info to return */
	FKeyDrawInfo InvalidDrawInfo;
};

