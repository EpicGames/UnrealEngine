// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorHelpers.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "ICurveEditorBounds.h"
#include "ICurveEditorDragOperation.h"
#include "ICurveEditorModule.h"
#include "ICurveEditorToolExtension.h"
#include "ITimeSlider.h"
#include "Internationalization/Text.h"
#include "Math/Axis.h"
#include "Math/Range.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/KeyPasteArgs.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Modification//TransactionManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Tree/CurveEditorTree.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FCurveEditor;
class FCurveModel;
class FUICommandList;
class IBufferedCurveModel;
class ICurveEditorExtension;
class ICurveEditorToolExtension;
class ITimeSliderController;
class SCurveEditorPanel;
class SCurveEditorView;
class UCurveEditorCopyBuffer;
class UCurveEditorCopyableCurveKeys;
class UCurveEditorSettings;
struct FCurveDrawParams;
struct FCurveEditorInitParams;
struct FCurveEditorSnapMetrics;
struct FGeometry;
namespace UE::CurveEditor { class FSelectionCleanser; }
namespace UE::CurveEditor { struct FCurveModelIdCache; }
namespace UE::CurveEditor { class FResolveCurveModelPasskey; }

DECLARE_DELEGATE_OneParam(FOnSetBoolean, bool)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveToolChanged, FCurveEditorToolID)
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnCurveArrayChanged, FCurveModel*, bool /*displayed*/,const FCurveEditor*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCurveColorsChanged, TConstArrayView<FCurveModelID>); 

/** Enums to describe supported tangent types, by default support all but smart auto*/
enum class ECurveEditorTangentTypes : int32
{
	InterpolationConstant = 0x1,
	InterpolationLinear = 0x2,
	InterpolationCubicAuto = 0x4,

	InterpolationCubicUser = 0x8,
	InterpolationCubicBreak = 0x10,
	InterpolationCubicWeighted = 0x20,
	InterpolationCubicSmartAuto = 0x40,

};

class FCurveEditor 
	: public FEditorUndoClient
	, public TSharedFromThis<FCurveEditor>
{
public:

	/**
	 * Container holding the current key/tangent selection
	 */
	FCurveEditorSelection Selection;

public:

	/** Attribute used to retrieve the current input snap rate (also used for display) */
	TAttribute<FFrameRate> InputSnapRateAttribute;

	/** Attribute used to retrieve the current value-axis grid line state */
	TAttribute<TOptional<float>> FixedGridSpacingAttribute;

	/** Attribute used to determine if we should snap input values */
	TAttribute<bool> InputSnapEnabledAttribute;

	/** Attribute used to determine if we should snap output values */
	TAttribute<bool> OutputSnapEnabledAttribute;

	/** Delegate that is invoked when the input snapping has been enabled/disabled */
	FOnSetBoolean OnInputSnapEnabledChanged;

	/** Delegate that is invoked when the output snapping has been enabled/disabled */
	FOnSetBoolean OnOutputSnapEnabledChanged;

	/** Attribute used for determining default attributes to apply to a newly create key */
	TAttribute<FKeyAttributes> DefaultKeyAttributes;

	/** Grid line label text format strings for the X and Y axis */
	TAttribute<FText> GridLineLabelFormatXAttribute, GridLineLabelFormatYAttribute;

	/** Padding applied to zoom-to-fit the input */
	TAttribute<double> InputZoomToFitPadding;

	/** Padding applied to zoom-to-fit the output */
	TAttribute<double> OutputZoomToFitPadding;

	/** Delegate that is invoked when a tool becomes active. Also fired when the tool goes inactive. */
	FOnActiveToolChanged OnActiveToolChangedDelegate;

	/** Delegate that is invoked when the curve colors are changed. */
	FOnCurveColorsChanged OnCurveColorsChangedDelegate;
	
public:

	/**
	 * Constructor
	 */
	UE_API FCurveEditor();

	/**
	 * Non-copyable (shared ptr semantics)
	 */
	FCurveEditor(const FCurveEditor&) = delete;
	FCurveEditor& operator=(const FCurveEditor&) = delete;

	UE_API virtual ~FCurveEditor();

	UE_API void InitCurveEditor(const FCurveEditorInitParams& InInitParams);

	UE_API virtual int32 GetSupportedTangentTypes();

	UE::CurveEditor::FTransactionManager* GetTransactionManager() const { return TransactionManager.Get(); }
	/** @return Metadata cache for reusing FCurveModelIDs when re-adding curves. For now, not API exported as it's for internal CurveEditor use only. */
	UE::CurveEditor::FCurveModelIdCache* GetCurveMetaDataCache() const { return CurveModelIdCache.Get(); }

public:

	UE_API void SetPanel(TSharedPtr<SCurveEditorPanel> InPanel);

	UE_API TSharedPtr<SCurveEditorPanel> GetPanel() const;

	UE_API void SetView(TSharedPtr<SCurveEditorView> InPanel);

	UE_API TSharedPtr<SCurveEditorView> GetView() const;

	UE_API FCurveEditorScreenSpaceH GetPanelInputSpace() const;

	UE_API void ResetMinMaxes();

public:
	/**
	 * Zoom the curve editor to fit all the selected curves (or all curves if none selected)
	 *
	 * @param Axes         (Optional) Axes to lock the zoom to
	 */
	UE_API void ZoomToFit(EAxisList::Type Axes = EAxisList::All);
	/**
	 * Zoom the curve editor to fit all the currently visible curves
	 *
	 * @param Axes         (Optional) Axes to lock the zoom to
	 */
	UE_API void ZoomToFitAll(EAxisList::Type Axes = EAxisList::All);
	/**
	 * Zoom the curve editor to fit the requested curves.
	 *
	 * @param CurveModelIDs The curve IDs to zoom to fit.
	 * @param Axes         (Optional) Axes to lock the zoom to
	 */
	UE_API void ZoomToFitCurves(TArrayView<const FCurveModelID> CurveModelIDs, EAxisList::Type Axes = EAxisList::All);
	/**
	 * Zoom the curve editor to fit all the current key selection. Zooms to fit all if less than 2 keys are selected.
	 *
	 * @param Axes         (Optional) Axes to lock the zoom to
	 */
	UE_API void ZoomToFitSelection(EAxisList::Type Axes = EAxisList::All);

	/** @return The config to use for scaling the zooms. */
	UE_API const FCurveEditorZoomScaleConfig& GetZoomScaleConfig() const;
	
	/**
	* Assign a new bounds container to this curve editor
	*/
	UE_API void SetBounds(TUniquePtr<ICurveEditorBounds>&& InBounds);
	/**
	 * Retrieve the current curve editor bounds implementation
	 */
	const ICurveEditorBounds& GetBounds() const { return *Bounds.Get(); }
	/**
	 * Retrieve the current curve editor bounds implementation
	 */
	ICurveEditorBounds& GetBounds() { return *Bounds.Get();	}
	/*
	 * Sets a Time Slider controller for this Curve Editor to be sync'd against. Can be null.
	 */
	void SetTimeSliderController(TSharedPtr<ITimeSliderController> InTimeSliderController) { WeakTimeSliderController = InTimeSliderController; }
	/**
	 * Retrieve the optional Time Slider Controller that this Curve Editor may be sync'd with.
	 */
	TSharedPtr<ITimeSliderController> GetTimeSliderController() const { return WeakTimeSliderController.Pin(); }
	/**
	* Retrieve this curve editor's command list
	*/
	TSharedPtr<FUICommandList> GetCommands() const { return CommandList; }
	/**
	* Returns true of the specified tool is currently active.
	*/
	UE_API bool IsToolActive(const FCurveEditorToolID InToolID) const;
	/**
	* Attempts to make the specified tool the active tool. This will cancel the current tool if there is one.
	*/
	UE_API void MakeToolActive(const FCurveEditorToolID InToolID);
	/**
	* Attempts to get the currently active tool. Will return nullptr if there is no active tool.
	* Do not store a reference to this returned pointer, instead only store FCurveEditorToolIDs!
	*/
	UE_API ICurveEditorToolExtension* GetCurrentTool() const;

	UE_API FCurveEditorToolID AddTool(TUniquePtr<ICurveEditorToolExtension>&& InTool);

	/**
	 * Adds a new axis to this curve editor with the specified name.
	 * @note Calling this function with the same identifier as a previously registered axis will
	 *       overwrite the axis with the new one.
	 * 
	 * @param InIdentifier      A unique identifer for this axis that can be subsequently used to look up this axis
	 * @param InAxis            The axis definition
	 */
	UE_API void AddAxis(const FName& InIdentifier, TSharedPtr<FCurveEditorAxis> InAxis);

	/**
	 * Find an axis by its ID
	 */
	UE_API TSharedPtr<FCurveEditorAxis> FindAxis(const FName& InIdentifier) const;

	/**
	 * Remove the axis with the specified ID
	 */
	UE_API void RemoveAxis(const FName& InIdentifier);

	/**
	 * Remove all custom axes definitions
	 */
	UE_API void ClearAxes();

	/** Nudge left or right*/
	UE_API void TranslateSelectedKeys(double SecondsToAdd);
	UE_API void TranslateSelectedKeysLeft();
	UE_API void TranslateSelectedKeysRight();

	/** Snap time to the first selected key */
	UE_API void SnapToSelectedKey();

	/** Selection range for ie. looping playback */
	UE_DEPRECATED(5.8, "SetSelectionRangeStart is no longer specialized for the Curve Editor")
	UE_API void SetSelectionRangeStart() {}
	UE_DEPRECATED(5.8, "SetSelectionRangeEnd is no longer specialized for the Curve Editor")
	UE_API void SetSelectionRangeEnd() {}
	UE_DEPRECATED(5.8, "ClearSelectionRange is no longer specialized for the Curve Editor")
	UE_API void ClearSelectionRange() {}

	/** Selection */
	UE_API void SelectAllKeys();
	UE_API void SelectForward();
	UE_API void SelectBackward();
	UE_API void SelectNone();
	UE_API void InvertSelection();

	/** Toggle the expansion state of the selected nodes or all nodes if none selected */
	UE_API void ToggleExpandCollapseNodes(bool bRecursive);

	/** Expand all nodes in the tree */
	UE_API void ExpandAllNodes();

	/** Collapse all nodes in the tree */
	UE_API void CollapseAllNodes();

	/** Sort all nodes in the tree */
	UE_API void SortAllNodes();

	/**
	 * Find a curve by its ID
	 *
	 * @return a ptr to the curve if found, nullptr otherwise
	 */
	UE_API FCurveModel* FindCurve(FCurveModelID CurveID) const;
	
	/**
	* Add a new curve to this editor
	*/
	UE_API FCurveModelID AddCurve(TUniquePtr<FCurveModel>&& InCurve);
	UE_API FCurveModelID AddCurveForTreeItem(TUniquePtr<FCurveModel>&& InCurve, FCurveEditorTreeItemID TreeItemID);
	
	/**
	* Remove a curve from this editor.
	*/
	UE_API void RemoveCurve(FCurveModelID InCurveID);

	/** Remove all curves from this editor */
	UE_API void RemoveAllCurves();

	UE_API bool IsCurvePinned(FCurveModelID InCurveID) const;

	UE_API void PinCurve(FCurveModelID InCurveID);

	UE_API void UnpinCurve(FCurveModelID InCurveID);

	const TSet<FCurveModelID>& GetPinnedCurves() const
	{
		return PinnedCurves;
	}

	UE_API const SCurveEditorView* FindFirstInteractiveView(FCurveModelID InCurveID) const;

	/**
	 * Retrieve this curve editor's settings
	 */
	UCurveEditorSettings* GetSettings() const { return Settings; }
	/**
	* Access all the curves currently contained in the Curve Editor regardless of visibility.
	*/
	UE_API const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& GetCurves() const;

	FCurveEditorSelection& GetSelection() { return Selection; }
	const FCurveEditorSelection& GetSelection() const { return Selection; }
	
	/**
	 * Generate a utility struct for snapping values
	 */
	UE_API FCurveSnapMetrics GetCurveSnapMetrics(FCurveModelID CurveModel) const;

	/**
	 * Returns the value grid line spacing state
	 */
	TOptional<float> GetGridSpacing() const { return FixedGridSpacingAttribute.Get(); }

	/**
	 * Returned the cached struct for snapping editing movement to a specific axis based on user preferences.
	 */
	FCurveEditorAxisSnap GetAxisSnap() const { return AxisSnapMetrics; }
	void SetAxisSnap(const FCurveEditorAxisSnap& InAxisSnap) { AxisSnapMetrics = InAxisSnap; }
	TAttribute<FText> GetGridLineLabelFormatXAttribute() const { return GridLineLabelFormatXAttribute; }
	TAttribute<FText> GetGridLineLabelFormatYAttribute() const { return GridLineLabelFormatYAttribute; }
	TAttribute<FKeyAttributes> GetDefaultKeyAttributes() const { return DefaultKeyAttributes; }

	void SuppressBoundTransformUpdates(bool bSuppress) { bBoundTransformUpdatesSuppressed = bSuppress; }
	bool AreBoundTransformUpdatesSuppressed() const { return bBoundTransformUpdatesSuppressed; }

	/** Return the curve model IDs that are selected in the tree or have selected keys */
	UE_API TSet<FCurveModelID> GetSelectionFromTreeAndKeys() const;

	UE_API TSet<FCurveModelID> GetEditedCurves() const;
	/** Attribute used for determining default attributes to apply to a newly create key */
	TAttribute<FKeyAttributes> GetDefaultKeyAttribute() const { return DefaultKeyAttributes; }
	/** Create a copy of the specified set of curves. */
	UE_API void AddBufferedCurves(const TSet<FCurveModelID>& InCurves);
	/** Attempts to apply the buffered curve to the passed in curve set. Returns true on success. */
	UE_API bool ApplyBufferedCurves(const TSet<FCurveModelID>& InCurvesToApplyTo, const bool bSwapBufferCurves);
	/** Return the number of stored Buffered Curves. */
	int32 GetNumBufferedCurves() const { return BufferedCurves.Num(); }
	/** Return the array of buffered curves */
	const TArray<TUniquePtr<IBufferedCurveModel>>& GetBufferedCurves() const { return BufferedCurves; }
	/** Returns whether the buffered curve is to be acted on, ie. selected, in the tree view or with selected keys */
	UE_API bool IsActiveBufferedCurve(const TUniquePtr<IBufferedCurveModel>& BufferedCurve) const;

	// ~FCurveEditor

	// FEditorUndoClient
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	// ~FEditorUndoClient

	const TArray<TSharedRef<ICurveEditorExtension>> GetEditorExtensions() const
	{
		return EditorExtensions;
	}

	const TMap<FCurveEditorToolID, TUniquePtr<ICurveEditorToolExtension>>& GetToolExtensions() const
	{
		return ToolExtensions;
	}

public:

	/**
	 * Retrieve a tree item from its ID
	 */
	UE_API FCurveEditorTreeItem& GetTreeItem(FCurveEditorTreeItemID ItemID);

	/**
	 * Retrieve a tree item from its ID
	 */
	UE_API const FCurveEditorTreeItem& GetTreeItem(FCurveEditorTreeItemID ItemID) const;

	/**
	 * Finds a tree item from its ID
	 */
	UE_API FCurveEditorTreeItem* FindTreeItem(FCurveEditorTreeItemID ItemID);

	/**
	 * Finds a tree item from its ID
	 */
	UE_API const FCurveEditorTreeItem* FindTreeItem(FCurveEditorTreeItemID ItemID) const;

	/**
	 * Get const access to the entire set of root tree items
	 */
	UE_API const TArray<FCurveEditorTreeItemID>& GetRootTreeItems() const;


	/**
	 * Find a tree ID id associated with a CurveModelID
	 */
	UE_API FCurveEditorTreeItemID GetTreeIDFromCurveID(FCurveModelID CurveID) const;

	/**
	 * Add a new tree item to this curve editor
	 */
	UE_API FCurveEditorTreeItem* AddTreeItem(FCurveEditorTreeItemID ParentID);

	/**
	 * Remove a tree item from the curve editor
	 */
	UE_API void RemoveTreeItem(FCurveEditorTreeItemID ItemID);

	/**
	 * Remove all tree items from the curve editor
	 */
	UE_API void RemoveAllTreeItems();

	/**
	 * Set the tree selection directly
	 */
	UE_API void SetTreeSelection(TArray<FCurveEditorTreeItemID>&& TreeItems);

	/**
	 * Removes items from the current tree selection.
	 */
	UE_API void RemoveFromTreeSelection(TArrayView<const FCurveEditorTreeItemID> TreeItems);

	/**
	 * Check whether this tree item is selected
	 */
	UE_API ECurveEditorTreeSelectionState GetTreeSelectionState(FCurveEditorTreeItemID TreeItemID) const;

	/**
	 * Retrieve the current tree selection
	 */
	UE_API const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& GetTreeSelection() const;

	/**
	 * Access the curve editor tree.
	 */
	FCurveEditorTree* GetTree()
	{
		return &Tree;
	}

	/**
	 * Access the curve editor tree.
	 */
	const FCurveEditorTree* GetTree() const
	{
		return &Tree;
	}

	/**
	Whether or not we are are doign a direct selection, could be used to see why a curve model is being created or destroyed, by direct selection or by sequencer filtering?
	*/
	bool IsDoingDirectSelection() const
	{
		return Tree.IsDoingDirectSelection();
	}


	/**
	 * Retrieve a serial number that is incremented any time a curve is added or removed
	 */
	uint32 GetActiveCurvesSerialNumber() const
	{
		return ActiveCurvesSerialNumber;
	}

public:

	/**
	 * Check whether this curve editor can automatically zoom to the current selection
	 */
	UE_API bool ShouldAutoFrame() const;
public:

	/**
	 * Check whether keys should be snapped to the input display rate when dragging around
	 */
	UE_API bool IsInputSnappingEnabled() const;
	UE_API void ToggleInputSnapping();

	/**
	 * Check whether keys should be snapped to the output snap interval when dragging around
	 */
	UE_API bool IsOutputSnappingEnabled() const;
	UE_API void ToggleOutputSnapping();

	/** Curve flip direction types **/
	enum class ECurveFlipDirection : uint8
	{
		Horizontal,
		Vertical
	};

	/** Curve Flip selection types **/
	enum class ECurveFlipRangeType : uint8
	{
		KeyRange,
		CurveRange,
		CustomRange
	};

	/** Settings for Curve Flip dropdown menu selections **/
	struct FCurveFlipRangeSettings
	{
		ECurveFlipRangeType RangeType;
		float MinRange;
		float MaxRange;

		FCurveFlipRangeSettings()
			: RangeType(ECurveFlipRangeType::CustomRange)
			, MinRange(0.0f)
			, MaxRange(1.0f)
		{
		}
	};
	FCurveFlipRangeSettings HorizontalCurveFlipRangeSettings;
	FCurveFlipRangeSettings VerticalCurveFlipRangeSettings;

	/**
	 * Flip the selected curve horizontally or vertically
	 */
	UE_API void FlipCurve(ECurveFlipDirection Direction);
	static UE_API void FlipCurveHorizontal(TArray<FKeyPosition>& AllKeyPositions, TArray<FKeyAttributes>& AllKeyAttributes, ECurveFlipRangeType RangeType,
											float InRangeMin, float InRangeMax, double CurveMinTime, double CurveMaxTime);
	static UE_API void FlipCurveVertical(TArray<FKeyPosition>& AllKeyPositions, TArray<FKeyAttributes>& AllKeyAttributes, ECurveFlipRangeType RangeType,
											float InRangeMin, float InRangeMax, double CurveMinVal, double CurveMaxVal);

public:

	/**
	 * Cut the currently selected keys
	 */
	UE_API void CutSelection();
	
	/**
	 * Copy the currently selected keys
	 */
	UE_API void CopySelection() const;

	/**
	 * Returns whether the current clipboard contains objects which CurveEditor can paste
	 */
	UE_API bool CanPaste(const FString& TextToImport) const;

protected:
	UE_API void ImportCopyBufferFromText(const FString& TextToImport, /*out*/ TArray<UCurveEditorCopyBuffer*>& ImportedCopyBuffers) const;
	UE_API TSet<FCurveModelID> GetTargetCurvesForPaste() const;
	
	UE_DEPRECATED(5.6, "Use the version that takes in ECurveEditorPasteMode and ECurveEditorPasteFlags instead.")
	UE_API bool CopyBufferCurveToCurveID(const UCurveEditorCopyableCurveKeys* InSourceCurve, const FCurveModelID InTargetCurve, TOptional<double> InTimeOffset, const bool bInAddToSelection, const bool bInOverwriteRange);
	UE_API bool CopyBufferCurveToCurveID(const UCurveEditorCopyableCurveKeys* InSourceCurve, const FCurveModelID InTargetCurve, TOptional<double> InTimeOffset,
		UE::CurveEditor::ECurveEditorPasteMode InMode, UE::CurveEditor::ECurveEditorPasteFlags InFlags
		);
	
	UE_API void GetChildCurveModelIDs(const FCurveEditorTreeItemID TreeItemID, TSet<FCurveModelID>& CurveModelIDs) const;

public:
	/** Paste keys*/
	UE_DEPRECATED(5.6, "Use the version that takes FKeyPasteArg instead.")
	UE_API void PasteKeys(TSet<FCurveModelID> CurveModelIDs, const bool bInOverwriteRange = false);

	/**
	 * Pastes keys.
	 * @param CurveModelIds Only the curve model IDs specified in this set. If empty, paste all in the clipboard.
	 * @param Flags Flags that modify the operation.
	 */
	UE_API void PasteKeys(UE::CurveEditor::FKeyPasteArgs InArgs = {});

	/**
	 * Delete the currently selected keys
	 */
	UE_API void DeleteSelection();

	/**
	 * Reset selected keys to their curve's default value, for curves that support it.
	 */
	UE_API void ResetSelectedKeysToDefaultValues();

	/**
	 * Returns true if any selected key belongs to a curve that has a default value.
	 */
	UE_API bool CanResetSelectedKeysToDefaultValues() const;

	/**
	 * Flatten the tangents on the selected keys
	 */
	UE_API void FlattenSelection();

	/**
	 * Straighten the tangents on the selected keys
	 */
	UE_API void StraightenSelection();

	/** 
	* Sets the last keys tangents to match the first, if bFirstToLast is false do the opposite
	*/
	UE_API void MatchLastTangentToFirst(bool bLastToFirst = true);

	/**
	 * Set random curve colors
	 */
	UE_API void SetRandomCurveColorsForSelected();

	/**
	 * Pick a curve color and set on selected
	 */	
	UE_API void SetCurveColorsForSelected();

	/**
	* Do we currently have keys to flatten or straighten?
	*/
	UE_API bool CanFlattenOrStraightenSelection() const;

	/** Snaps the selected keys to their nearest whole frames and positions them on the frame intersection with the frame. */
	UE_API void SmartSnapSelection();
	/** @return Whether there is any key selection. */
	UE_API bool CanSmartSnapSelection() const;

public:

	/**
	 * Called by SCurveEditorPanel to update the allocated geometry for this curve editor.
	 */
	UE_API void UpdateGeometry(const FGeometry& CurrentGeometry);

public:

	/**
	 * Called by SCurveEditorPanel to determine where to draw grid lines along the X-axis. This allows
	 * synchronization with an external data source (such as Sequencer's Timeline ticker). A similar
	 * function for the Y grid lines is not provided due to the Curve Editor's ability to have multiple
	 * views with repeated gridlines and values.
	 */
	virtual void GetGridLinesX(TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
	{
		ConstructXGridLines(MajorGridLines, MinorGridLines, MajorGridLabels);
	}

	/**
	 * Bind UI commands that this curve editor responds to
	 */
	UE_API void BindCommands();

	/** @return The object managing which filters are promoted to the toolbar for quick access. */
	UE_API TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> GetToolbarPromotedFilters() const;

public:

	/** Suspend or resume broadcast of curve array changing  */
	void SuspendBroadcast()
	{
		SuspendBroadcastCount++;
	}

	void ResumeBroadcast()
	{
		SuspendBroadcastCount--;
		checkf(SuspendBroadcastCount >= 0, TEXT("Suspend/Resume broadcast mismatch Curve Editor!"));
	}

	bool IsBroadcasting()
	{
		return SuspendBroadcastCount == 0;
	}

	UE_API void BroadcastCurveChanged(FCurveModel* InCurve);

protected:

	/**
	 * Construct grid lines along the current display frame rate or time-base
	 */
	UE_API void ConstructXGridLines(TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const;

	/**
	 * Internal zoom to fit implementation
	 *
	 * @param Axes        The axes to zoom (only X or Y supported)
	 * @param CurveKeySet Map from curve ID to keys that should be considered for zoom. Empty key sets will cause the entire curve range to be zoomed.
	 */
	UE_API void ZoomToFitInternal(EAxisList::Type Axes, const TMap<FCurveModelID, FKeyHandleSet>& CurveKeySet);

	/**
	*	Apply a specific buffered curve to a specific target curve.
	*/
	UE_API void ApplyBufferedCurveToTarget(const IBufferedCurveModel* BufferedCurve, FCurveModel* TargetCurve);

	UE_API void OnCustomColorsChanged();
	UE_API void OnAxisSnappingChanged();

protected:

	/** Curve editor bounds implementation */
	TUniquePtr<ICurveEditorBounds> Bounds;

	/** Map from curve model ID to the actual curve model */
	TMap<FCurveModelID, TUniquePtr<FCurveModel>> CurveData;
	/** Map from curve model ID to its originating tree item */
	TMap<FCurveModelID, FCurveEditorTreeItemID> TreeIDByCurveID;
	/** Map of child curves */
	TMultiMap<FCurveModelID, FCurveModelID> ChildCurves;

	/** Map of all axes */
	TSortedMap<FName, TSharedPtr<FCurveEditorAxis>, FDefaultAllocator, FNameFastLess> CustomAxes;

	/** Set of pinned curve models */
	TSet<FCurveModelID> PinnedCurves;

	TWeakPtr<SCurveEditorPanel> WeakPanel;

	TWeakPtr<SCurveEditorView> WeakView;

	/** Hierarchical information pertaining to curve data */
	FCurveEditorTree Tree;

	/** The currently active tool if any. If unset then no tool is currently active and the next selection will default to the first tool. */
	TOptional<FCurveEditorToolID> ActiveTool;

	/** UI command list of actions mapped to this curve editor */
	TSharedPtr<FUICommandList> CommandList;

	/** Curve editor settings object */
	UCurveEditorSettings* Settings;

	/** List of editor extensions we have initialized. */
	TArray<TSharedRef<ICurveEditorExtension>> EditorExtensions;

	/** The zoom scale config that was passed in to us in InitCurveEditor. */
	TAttribute<const FCurveEditorZoomScaleConfig*> ZoomScalingAttr;

	/** List of tool extensions we have initialized. */
	TMap<FCurveEditorToolID, TUniquePtr<ICurveEditorToolExtension>> ToolExtensions;

	/** Optional external Time Slider controller to sync with. Enables some additional functionality. */
	TWeakPtr<ITimeSliderController> WeakTimeSliderController;

	/** 
	* Should attempts to update the bounds of each curve be ignored? This allows tools to keep the bounds from being automatically updated each frame 
	* which allows Normalized views to push past their boundaries without the normalization ratio changing per-frame as you drag.
	*/
	bool bBoundTransformUpdatesSuppressed;

	/** Track which axis UI movements should be snapped to (where applicable) based on limitations imposed by the UI. */
	FCurveEditorAxisSnap AxisSnapMetrics;

	/** Buffered Curves. When a curve is buffered it is copied and the new copy is uniquely owned by the Curve Editor. */
	TArray<TUniquePtr<IBufferedCurveModel>> BufferedCurves;

	/** A serial number that is incremented any time the currently active set of curves are changed */
	uint32 ActiveCurvesSerialNumber;

	/** Counter to suspend broadcasting of changed delegates*/
	int32 SuspendBroadcastCount;

	/** Orchestrates transaction that affect this curve editor. */
	TUniquePtr<UE::CurveEditor::FTransactionManager> TransactionManager;
	
	/** Makes sure that selection does not reference stale FKeyHandles or FCurveModelIds, i.e. after undo or an operation has removed keys. */
	TPimplPtr<UE::CurveEditor::FSelectionCleanser> SelectionCleanser;

	/**
	 * Tries to re-use FCurveModelIDs when the conceptually same FCurveModel is added back to FCurveEditor using AddCurve.
	 * @see FCurveModelIdCache.
	 */
	TPimplPtr<UE::CurveEditor::FCurveModelIdCache> CurveModelIdCache;

private:

	friend class UE::CurveEditor::FResolveCurveModelPasskey;
	
	/**
	 * Internally used for editing FCurveModels after they've been removed FCurveEditor::CurveData (e.g. once they're no longer visible).
	 * Important for applying undo / redo commands on curves that are not visible when the user issues the undo / redo operation.
	 */
	UE::CurveEditor::FResolveCurveModelDelegate ResolveCurveModelDelegate;
	
	/** Tries to re-create a FCurveModel that was in this FCurveEditor previously but has since been removed. */
	TUniquePtr<FCurveModel> ResolveCurve(const FCurveModelID& InCurveID) const;

private:

	
	/** Cached physical size of the panel representing this editor */
	FVector2D CachedPhysicalSize;

	/** Invoked when a curve's color changes (FCurveModel::SetColor is called). */
	void HandleCurveColorChanged(FCurveModelID CurveId);

public:
	/**
	* Delegate that's broadcast when the curve display changes.
	*/
	FOnCurveArrayChanged OnCurveArrayChanged;
};

#undef UE_API
