// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FCurveEditor;
class SCurveEditorView;
struct FCurveEditorViewAxisID;

template<typename NumericType> struct INumericTypeInterface;

/** Curve editor axis orientation enumeration */
enum class ECurveEditorAxisOrientation : uint8
{
	Horizontal = 1,
	Vertical   = 2,

	Both = Horizontal | Vertical,
};
ENUM_CLASS_FLAGS(ECurveEditorAxisOrientation);

/**
 * Class that models an underlying curve data structure through a generic abstraction that the curve editor understands.
 */
class FCurveEditorAxis
{
public:

	virtual ~FCurveEditorAxis() = default;

	/**
	 * Check whether this axis can draw labels
	 */
	CURVEEDITOR_API virtual bool HasLabels() const;


	/**
	 * Make a label for the specified value
	 */
	CURVEEDITOR_API virtual FText MakeLabel(double Value) const;


	/**
	 * Get the grid lines for this axis definition
	 */
	CURVEEDITOR_API virtual void GetGridLines(const FCurveEditor& CurveEditor, const SCurveEditorView& View, FCurveEditorViewAxisID AxisID, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines, ECurveEditorAxisOrientation Axis) const;


public:


	/**
	 * An optional numeric type interface for displaying values on this axis
	 */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;
};


/**
 * Implementation of FCurveEditorAxis that draws grid lines on a linear basis
 */
class FLinearCurveEditorAxis : public FCurveEditorAxis
{

	CURVEEDITOR_API virtual void GetGridLines(const FCurveEditor& CurveEditor, const SCurveEditorView& View, FCurveEditorViewAxisID AxisID, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines, ECurveEditorAxisOrientation Axis) const override;

};
