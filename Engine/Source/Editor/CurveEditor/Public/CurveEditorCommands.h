// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FCurveEditorCommands, CURVEEDITOR_API)

/**
 * Defines commands for SCurveEditorPanel (and UnrealEd::SCurveEditor)
 */
class FCurveEditorCommands : public TCommands<FCurveEditorCommands>
{
public:
	FCurveEditorCommands()
		: TCommands<FCurveEditorCommands>
		(
			TEXT("GenericCurveEditor"),
			NSLOCTEXT("Contexts", "GenericCurveEditor", "Curve Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	TSharedPtr<FUICommandInfo> ZoomToFitHorizontal;
	TSharedPtr<FUICommandInfo> ZoomToFitVertical;
	TSharedPtr<FUICommandInfo> ZoomToFit;
	TSharedPtr<FUICommandInfo> ZoomToFitAll;
	TSharedPtr<FUICommandInfo> ToggleInputSnapping;
	TSharedPtr<FUICommandInfo> ToggleOutputSnapping;

	TSharedPtr<FUICommandInfo> FlipCurveHorizontal;
	TSharedPtr<FUICommandInfo> FlipCurveVertical;

	TSharedPtr<FUICommandInfo> ToggleExpandCollapseNodes;
	TSharedPtr<FUICommandInfo> ToggleExpandCollapseNodesAndDescendants;
	TSharedPtr<FUICommandInfo> ExpandAllNodes;
	TSharedPtr<FUICommandInfo> CollapseAllNodes;
	TSharedPtr<FUICommandInfo> SortAllNodes;

	TSharedPtr<FUICommandInfo> InterpolationConstant;
	TSharedPtr<FUICommandInfo> InterpolationLinear;
	TSharedPtr<FUICommandInfo> InterpolationCubicAuto;
	TSharedPtr<FUICommandInfo> InterpolationCubicSmartAuto;
	TSharedPtr<FUICommandInfo> InterpolationCubicUser;
	TSharedPtr<FUICommandInfo> InterpolationCubicBreak;
	TSharedPtr<FUICommandInfo> InterpolationToggleWeighted;


	TSharedPtr<FUICommandInfo> FlattenTangents;
	TSharedPtr<FUICommandInfo> StraightenTangents;

	TSharedPtr<FUICommandInfo> SmartSnapKeys;

	TSharedPtr< FUICommandInfo > MatchLastTangentToFirst;
	TSharedPtr< FUICommandInfo > MatchFirstTangentToLast;
	
	TSharedPtr<FUICommandInfo> BakeCurve;
	TSharedPtr<FUICommandInfo> ReduceCurve;

	TSharedPtr<FUICommandInfo> SetPreInfinityExtrapCycle;
	TSharedPtr<FUICommandInfo> SetPreInfinityExtrapCycleWithOffset;
	TSharedPtr<FUICommandInfo> SetPreInfinityExtrapOscillate;
	TSharedPtr<FUICommandInfo> SetPreInfinityExtrapLinear;
	TSharedPtr<FUICommandInfo> SetPreInfinityExtrapConstant;

	TSharedPtr<FUICommandInfo> SetPostInfinityExtrapCycle;
	TSharedPtr<FUICommandInfo> SetPostInfinityExtrapCycleWithOffset;
	TSharedPtr<FUICommandInfo> SetPostInfinityExtrapOscillate;
	TSharedPtr<FUICommandInfo> SetPostInfinityExtrapLinear;
	TSharedPtr<FUICommandInfo> SetPostInfinityExtrapConstant;

	TSharedPtr<FUICommandInfo> SetAllTangentsVisibility;
	TSharedPtr<FUICommandInfo> SetUserTangentsVisibility;
	TSharedPtr<FUICommandInfo> SetSelectedKeysTangentVisibility;
	TSharedPtr<FUICommandInfo> SetNoTangentsVisibility;

	TSharedPtr<FUICommandInfo> ToggleAutoFrameCurveEditor;
	TSharedPtr<FUICommandInfo> ToggleSnapTimeToSelection;
	TSharedPtr<FUICommandInfo> ToggleShowBufferedCurves;
	TSharedPtr<FUICommandInfo> ToggleShowCurveEditorCurveToolTips;
	TSharedPtr<FUICommandInfo> ToggleShowTransportControls;
	TSharedPtr<FUICommandInfo> ToggleShowBars;
	TSharedPtr<FUICommandInfo> ToggleShowValueIndicatorLines;

	TSharedPtr<FUICommandInfo> SetRandomCurveColorsForSelected;
	TSharedPtr<FUICommandInfo> SetCurveColorsForSelected;

	TSharedPtr<FUICommandInfo> AddKeyHovered;
	TSharedPtr<FUICommandInfo> PasteKeysHovered;
	TSharedPtr<FUICommandInfo> PasteAndMerge;
	TSharedPtr<FUICommandInfo> PasteRelative;

	TSharedPtr<FUICommandInfo> AddKeyToAllCurves;
	TSharedPtr<FUICommandInfo> AddKeyToSelectedCurves;
	TSharedPtr<FUICommandInfo> ResetKeysToDefaultValues;


	TSharedPtr<FUICommandInfo> SetViewModeAbsolute;
	TSharedPtr<FUICommandInfo> SetViewModeStacked;
	TSharedPtr<FUICommandInfo> SetViewModeNormalized;

	TSharedPtr<FUICommandInfo> DeactivateCurrentTool;

	TSharedPtr<FUICommandInfo> SelectAllKeys;
	TSharedPtr<FUICommandInfo> DeselectAllKeys;

	TSharedPtr<FUICommandInfo> SelectForward;
	TSharedPtr<FUICommandInfo> SelectBackward;
	TSharedPtr<FUICommandInfo> SelectNone;
	TSharedPtr<FUICommandInfo> InvertSelection;

	TSharedPtr<FUICommandInfo> BufferVisibleCurves;
	TSharedPtr<FUICommandInfo> SwapBufferedCurves;
	TSharedPtr<FUICommandInfo> ApplyBufferedCurves;

	// User Filtering
	TSharedPtr<FUICommandInfo> OpenUserImplementableFilterWindow;

	// Axis Snapping
	TSharedPtr<FUICommandInfo> SetAxisSnappingNone;
	TSharedPtr<FUICommandInfo> SetAxisSnappingHorizontal;
	TSharedPtr<FUICommandInfo> SetAxisSnappingVertical;

	//Key Movement
	TSharedPtr<FUICommandInfo> TranslateSelectedKeysLeft;
	TSharedPtr<FUICommandInfo> TranslateSelectedKeysRight;

	// Time Management
	TSharedPtr<FUICommandInfo> ScrubTime;

public:
	CURVEEDITOR_API virtual void RegisterCommands() override;
};
