// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDrawInfo.h"
#include "CurveEditorSettings.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FCurveModel;
class SCurveEditorView;
struct FCurveDrawParams;
struct FCurveEditorScreenSpace;
struct FCurveInfo;
struct FCurveModelID;

namespace UE::CurveEditor
{		
	struct FCurveDrawParamsHandle;
	
	/** Invoked when FCurveModel::HasChangedAndResetTest returns true. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCurveHasChangedExternally, const FCurveModelID&);

	/** Caches curve draw params for curves of a curve editor. */
	class FCurveDrawParamsCache
		: public TSharedFromThis<FCurveDrawParamsCache>
	{
		// Let SCurveEditorView write directly to the array of cached curve params when CVar CurveEditor.UseCurveCache is disabled
		friend class ::SCurveEditorView;

		// Allow draw params handle to reference draw params
		friend struct FCurveDrawParamsHandle;

	public:
		/** Flag enum signifying how the curve cache has changed since it was last generated	 */
		enum class ECurveCacheFlags : uint8
		{
			CheckCurves = 0,       // The cache may be valid need to check each curve to see if they are still valid
			UpdateAll = 1 << 0,	   // Update all curves
		};

		/** Set of cached values we need to check each tick to see if we need to redo cache */
		struct FCachedCurveEditorData
		{
			/** Serial number cached from FCurveEditor::GetActiveCurvesSerialNumber() on tick */
			uint32 ActiveCurvesSerialNumber = 0;

			/** Serial number bached from CurveEditorSelecstion::GetSerialNumber */
			uint32 SelectionSerialNumber = 0;

			/** Cached Tangent Visibility*/
			ECurveEditorTangentVisibility TangentVisibility = ECurveEditorTangentVisibility::NoTangents;

			/** Cached input min value */
			double InputMin = 0.0;
			
			/** Cached input max value */
			double InputMax = 0.0;
			
			/** Cached output min value */
			double OutputMin = 0.0;
			
			/** Cached output max value */
			double OutputMax = 0.0;

			/** Cached Geometry Size */
			FVector2D CachedGeometrySize = FVector2D::ZeroVector;
		};

		UE_API FCurveDrawParamsCache();

		/** 
		 * Invalidates the cache for specified view and curve model IDs and causes an update on cached data.
		 * The cache may perform updates that are not instantly relevant asynchronously. To update data synchronous use UpdateCurveDrawParamSynchonous instead.
		 */
		UE_API void Invalidate(const TSharedRef<SCurveEditorView>& CurveEditorView, TConstArrayView<FCurveModelID> ModelIDs = {});

		/**
		 * Returns how the curve cache has changed since it was last generated.
		 * Note for a data change it may only effect certain data (curves) not every drawn curve
		 */
		ECurveCacheFlags GetCurveCacheFlags() const { return CurveCacheFlags; }

		/** Returns cached data common to the curve editor */
		const FCachedCurveEditorData& GetCurveEditorData() const { return CachedCurveEditorData; }

		/** Returns the current draw params */
		const TArray<FCurveDrawParams>& GetCurveDrawParams() const { return CachedDrawParams; }

		/** Updates curve draw params for all specified curve models, synchronous. */
		UE_API void UpdateAllCurveDrawParamSynchonous(const TSharedRef<SCurveEditorView>& CurveEditorView, const TArray<FCurveModelID>& CurveModelIDs, TArray<FCurveDrawParams>& OutParams);

		/** Gets curve draw params for a curve model, synchronous. */
		UE_API void UpdateCurveDrawParamsSynchonous(const FCurveEditorScreenSpace& CurveSpace, FCurveModel* CurveModel, const FCurveModelID& ModelID, FCurveDrawParams& Params);

	private:
		/** Updates the curve cache flags */
		UE_API void UpdateCurveCacheFlags(const TSharedRef<SCurveEditorView>& CurveEditorView);

		/** Draws the curves, possibly async */
		UE_API void DrawCurves(const TSharedRef<SCurveEditorView>& CurveEditorView, TConstArrayView<FCurveModelID> ModelIDs = {});

		/** Updates curve draw params, possibly async */
		UE_API void UpdateCurveDrawParams(const TSharedRef<SCurveEditorView>& CurveEditorView, const TArray<FCurveDrawParamsHandle>& CurveDrawParamsHandles);

		/** Curve cache flags that change based upon data or view getting modified*/
		ECurveCacheFlags CurveCacheFlags = ECurveCacheFlags::UpdateAll;

		/** The actual cached draw params */
		TArray<FCurveDrawParams> CachedDrawParams;

		/** Cached data common to the curve editor */
		FCachedCurveEditorData CachedCurveEditorData;

		/** The curve editor that owns this cache */
		TWeakPtr<FCurveEditor> WeakCurveEditor;

		/**
		 * Invoked when FCurveModel::HasChangedAndResetTest returns true.
		 * Only used by SCurveEditorView at the moment thus not public API. 
		 */
		FOnCurveHasChangedExternally OnCurveHasChangedExternallyDelegate;
	};
}

#undef UE_API
