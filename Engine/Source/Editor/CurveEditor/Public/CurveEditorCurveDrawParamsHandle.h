// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"

#define UE_API CURVEEDITOR_API

class FCurveDrawParamsCache;
struct FCurveDrawParams;

namespace UE::CurveEditor
{
	class FCurveDrawParamsCache;

	/** Handle for an entry in the draw params array */
	struct FCurveDrawParamsHandle
	{
		FCurveDrawParamsHandle() = default;

		/** Constructs the handle from the index in curve draw params. */
		UE_API FCurveDrawParamsHandle(const TSharedRef<FCurveDrawParamsCache>& InDrawParamsCache, const int32 InIndex);

		/** 
		 * Returns the draw params or nullptr if the draw params no longer exists. 
		 * Does a slow lookup if the array changed.
		 */
		UE_API FCurveDrawParams* Get() const;

		/** Returns the curve model ID that corresponds to these curve draw params. */
		const FCurveModelID& GetID() const { return CurveModelID; }

	private:
		/** The curve model ID that corresponds to the curve draw params */
		FCurveModelID CurveModelID;

		/** Index of the draw params in the array of the draw params cache */
		mutable int32 Index = INDEX_NONE;

		/** Weak ptr to the draw params cache that owns the draw params */
		TWeakPtr<class FCurveDrawParamsCache> WeakDrawParamsCache;
	};
}

#undef UE_API
