// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Delegates/Delegate.h"

class FCurveModel;

namespace UE::CurveEditor
{
struct FCurveModelLookUpArgs;
	
/**
 * Tries to resolve a FCurveModel.
 * 
 * This is intended for asking the external systems owning the FCurveEditor instance to construct the temporary FCurveModel API,
 * e.g. when perform an undo / redo operation for a FCurveModel that is not currently visible (FCurveEditor::CurveData only contains visible curves).
 *
 * Can return nullptr if the curve cannot be resolved.
 */
DECLARE_DELEGATE_RetVal_OneParam(TUniquePtr<FCurveModel>, FResolveCurveModelDelegate, const FCurveModelLookUpArgs& /*InArgs*/);
}
