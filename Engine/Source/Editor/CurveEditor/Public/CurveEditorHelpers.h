// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveEditorScreenSpace.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"

class FText;
struct FCurveEditorScreenSpace;

namespace CurveEditor
{

	CURVEEDITOR_API FVector2D ComputeScreenSpaceTangentOffset(const FCurveEditorScreenSpace& CurveSpace, float Tangent, float Weight);

	void TangentAndWeightFromOffset(const FCurveEditorScreenSpace& CurveSpace, const FVector2D& TangentOffset, float& OutTangent, float& OutWeight);

	CURVEEDITOR_API FVector2D GetVectorFromSlopeAndLength(float Slope, float Length);

	void PopulateGridLineValues(float PhysicalSize, double ViewMin, double ViewMax, uint8 InMinorDivisions, TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines);

	void ConstructYGridLines(const FCurveEditorScreenSpace& ViewSpace, uint8 InMinorDivisions, TArray<float>& OutMajorGridLines, TArray<float>& OutMinorGridLines, FText GridLineLabelFormatY, TArray<FText>* OutMajorGridLabels);

	void ConstructFixedYGridLines(const FCurveEditorScreenSpace& ViewSpace, uint8 InMinorDivisions, double InMinorGridStep, TArray<float>& OutMajorGridLines, TArray<float>& OutMinorGridLines, FText GridLineLabelFormatY, 
		TArray<FText>* OutMajorGridLabels, TOptional<double> InOutputMin, TOptional<double> InOutputMax);

} // namespace CurveEditor