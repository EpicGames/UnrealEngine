// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "CurveEditorKeyProxy.generated.h"

UINTERFACE(MinimalAPI)
class UCurveEditorKeyProxy : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that can be implemented by any object passed to a curve editor to use as a key editing proxy.
 * When used, UpdateValuesFromRawData will be called every frame to optionally retrieve the current values
 * of the key for this proxy.
 */
class ICurveEditorKeyProxy
{
public:

	GENERATED_BODY()

	/**
	 * Called by the curve editor to update this instance's properties with the underlying raw data, if necessary
	 */
	virtual void UpdateValuesFromRawData() = 0;
};
