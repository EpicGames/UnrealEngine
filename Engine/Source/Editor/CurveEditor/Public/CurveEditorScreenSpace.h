// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Matrix.h"
#include "Math/Plane.h"
#include "Math/TransformCalculus2D.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"


/**
 * Utility struct used for converting to/from curve editor screen space
 */
struct FCurveEditorScreenSpaceH
{
	/**
	 * Construction from a physical size, and input/output range
	 */
	FCurveEditorScreenSpaceH(double InPixelWidth, double InInputMin, double InInputMax)
		: PixelWidth(FMath::Max(InPixelWidth, 1.0))
		, InputMin(InInputMin), InputMax(InInputMax)
	{
		if (InputMax <= InputMin)
		{
			InputMax = InputMin + KINDA_SMALL_NUMBER;
		}
	}

public:

	/** Convert a horizontal screen position in slate units to a value in seconds */
	FORCEINLINE double ScreenToSeconds(double ScreenPosition) const
	{
		return InputMin + ScreenPosition / PixelsPerInput();
	}

	/** Convert a value in seconds to a horizontal screen position in slate units */
	FORCEINLINE double SecondsToScreen(double InSeconds) const
	{
		return (InSeconds - InputMin) * PixelsPerInput();
	}

public:

	/** Retrieve the number of slate units per input value */
	FORCEINLINE double PixelsPerInput() const
	{
		double InputDiff = FMath::Max(InputMax - InputMin, 1e-10);
		return PixelWidth / InputDiff;
	}

public:

	/** Get the minimum input value displayed on the screen */
	FORCEINLINE double GetInputMin() const { return InputMin; }
	/** Get the maximum input value displayed on the screen */
	FORCEINLINE double GetInputMax() const { return InputMax; }
	/** Get the physical width of the screen */
	FORCEINLINE double GetPhysicalWidth() const { return PixelWidth; }

public:

	/**
	 * Transform this screen space into a curve space using the specified transform
	 */
	FCurveEditorScreenSpaceH ToCurveSpace(const FTransform2d& CurveTransform) const
	{
		const FVector2D& T = CurveTransform.GetTranslation();

		double m00, m01, m10, m11;
		CurveTransform.GetMatrix().GetMatrix(m00, m01, m10, m11);

		double NewInputMin  = InputMin * m00 + T.X;
		double NewInputMax  = InputMax * m00 + T.X;

		return FCurveEditorScreenSpaceH(PixelWidth, NewInputMin, NewInputMax);
	}

private:

	double PixelWidth;
	double InputMin, InputMax;
};


/**
 * Utility struct used for converting to/from curve editor screen space
 */
struct FCurveEditorScreenSpaceV
{
	/**
	 * Construction from a physical size, and input/output range
	 */
	FCurveEditorScreenSpaceV(double InPixelHeight, double InOutputMin, double InOutputMax)
		: PixelHeight(FMath::Max(InPixelHeight, 1.0))
		, OutputMin(InOutputMin), OutputMax(InOutputMax)
	{
		if (!ensure(OutputMax > OutputMin))
		{
			OutputMax = OutputMin + 1.0;
		}
	}

public:

	/** Convert a vertical screen position in slate units to a value */
	FORCEINLINE double ScreenToValue(double ScreenPosition) const
	{
		return OutputMin + (PixelHeight - ScreenPosition) / PixelsPerOutput();
	}

	/** Convert a value to a vertical screen position in slate units */
	double ValueToScreen(double InValue) const
	{
		return (PixelHeight - (InValue - OutputMin) * PixelsPerOutput());
	}

public:

	/** Retrieve the number of slate units per output value */
	FORCEINLINE double PixelsPerOutput() const
	{
		double OutputDiff = FMath::Max(OutputMax - OutputMin, 1e-10);
		return PixelHeight / OutputDiff;
	}

public:

	/** Get the minimum output value displayed on the screen */
	FORCEINLINE double GetOutputMin() const { return OutputMin; }
	/** Get the maximum output value displayed on the screen */
	FORCEINLINE double GetOutputMax() const { return OutputMax; }
	/** Get the physical height of the screen */
	FORCEINLINE double GetPhysicalHeight() const { return PixelHeight; }

public:

	/**
	 * Transform this screen space into a curve space using the specified transform
	 */
	FCurveEditorScreenSpaceV ToCurveSpace(const FTransform2d& CurveTransform) const
	{
		const FVector2D& T = CurveTransform.GetTranslation();

		double m00, m01, m10, m11;
		CurveTransform.GetMatrix().GetMatrix(m00, m01, m10, m11);

		double NewOutputMin = OutputMin * m11 + T.Y;
		double NewOutputMax = OutputMax * m11 + T.Y;

		return FCurveEditorScreenSpaceV(PixelHeight, NewOutputMin, NewOutputMax);
	}

private:

	double PixelHeight;
	double OutputMin, OutputMax;
};

/**
 * Utility struct used for converting to/from curve editor screen space
 */
struct FCurveEditorScreenSpace : FCurveEditorScreenSpaceH, FCurveEditorScreenSpaceV
{
	/**
	 * Construction from a physical size, and input/output range
	 */
	FCurveEditorScreenSpace(FVector2D InPixelSize, double InInputMin, double InInputMax, double InOutputMin, double InOutputMax)
		: FCurveEditorScreenSpaceH(InPixelSize.X, InInputMin, InInputMax)
		, FCurveEditorScreenSpaceV(InPixelSize.Y, InOutputMin, InOutputMax)
	{}

public:

	/** Get the physical size of the screen */
	FORCEINLINE FVector2D GetPhysicalSize() const { return FVector2D(GetPhysicalWidth(), GetPhysicalHeight()); }

public:

	/**
	 * Transform this screen space into a curve space using the specified transform
	 */
	FCurveEditorScreenSpace ToCurveSpace(const FTransform2d& CurveTransform) const
	{
		FCurveEditorScreenSpace New = *this;
		FCurveEditorScreenSpaceH& H = static_cast<FCurveEditorScreenSpaceH&>(New);
		FCurveEditorScreenSpaceV& V = static_cast<FCurveEditorScreenSpaceV&>(New);

		H = H.ToCurveSpace(CurveTransform);
		V = V.ToCurveSpace(CurveTransform);

		return New;
	}

	bool operator==(const FCurveEditorScreenSpace& Other) const
	{
		return
			GetInputMax() == Other.GetInputMin() &&
			GetInputMax() == Other.GetInputMax() &&
			GetPhysicalHeight() == Other.GetPhysicalHeight() &&
			GetOutputMin() == Other.GetOutputMin() &&
			GetOutputMax() == Other.GetOutputMax() &&
			GetPhysicalWidth() == Other.GetPhysicalWidth();
	}

	bool operator !=(const FCurveEditorScreenSpace& Other)
	{
		return !(*this == Other);
	}
};
