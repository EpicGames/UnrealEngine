// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "Misc/SuppressibleEventBroadcaster.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
struct FCurveEditorSelection;

namespace UE::CurveEditor::KeySelection
{
	class FAddInternal;
	class FSetSerialNumber;
	class FIncrementOnSelectionChangedSuppressionCount;
	class FDecrementOnSelectionChangedSuppressionCount;
}

struct FCurvePointHandle;

/**
 * A set of key handles implemented as a sorted array for transparent passing to TArrayView<> APIs.
 * Lookup is achieved via binary search: O(log(n)).
 */
struct FKeyHandleSet
{
	/**
	 * Add a new key handle to this set
	 */
	UE_API void Add(FKeyHandle Handle, ECurvePointType PointType);

	/**
	 * Remove a handle from this set if it already exists, otherwise add it to the set
	 */
	UE_API void Toggle(FKeyHandle Handle, ECurvePointType PointType);

	/**
	 * Remove a handle from this set
	 */
	UE_API void Remove(FKeyHandle Handle, ECurvePointType PointType);

	/**
	 * Replace this set with exactly these keys and assign them this point type
	 *
	 * @param InKeys The array of key handles to reset the set with.
	 * @param InPointType The point type to associate with the keys.
	 */
	UE_API void Replace(const TArrayView<const FKeyHandle>& InKeys, const ECurvePointType InPointType);

	/**
	 * Check whether the specified handle exists in this set
	 */
	UE_API bool Contains(FKeyHandle Handle, ECurvePointType PointType) const;

	/**
	 * Retrieve the number of handles in this set
	 */
	FORCEINLINE int32 Num() const { return SortedHandles.Num(); }

	/**
	 * Retrieve a constant view of this set as an array
	 */
	FORCEINLINE TArrayView<const FKeyHandle> AsArray() const { return SortedHandles; }

	/**
	 *  Retrieve the point type for this handle
	 */
	ECurvePointType PointType(FKeyHandle Handle) const { return HandleToPointType.FindChecked(Handle); }

	/**
	 * Serializes the handle set
	 */
	friend FArchive& operator<<(FArchive& Ar, FKeyHandleSet& HandleSet)
	{
		Ar << HandleSet.SortedHandles;
		Ar << HandleSet.HandleToPointType;
		return Ar;
	}

private:

	/** Sorted array of key handles */
	TArray<FKeyHandle, TInlineAllocator<1>> SortedHandles;

	/** Map of handle to point type (point, left, or right tangent) */
	TMap<FKeyHandle, ECurvePointType> HandleToPointType;
};

class FScopedSelectionChangedEventSuppressor
{
public:
	UE_API explicit FScopedSelectionChangedEventSuppressor(FCurveEditorSelection& InSelection);
	UE_API ~FScopedSelectionChangedEventSuppressor();

private:
	FCurveEditorSelection& Selection;
};

/**
 * Class responsible for tracking selections of keys.
 * Only one type of point selection is supported at a time (key, arrive tangent, or leave tangent)
 */
struct FCurveEditorSelection
{
	friend UE::CurveEditor::KeySelection::FAddInternal;
	friend UE::CurveEditor::KeySelection::FSetSerialNumber;
	friend UE::CurveEditor::KeySelection::FIncrementOnSelectionChangedSuppressionCount;
	friend UE::CurveEditor::KeySelection::FDecrementOnSelectionChangedSuppressionCount;
	friend FScopedSelectionChangedEventSuppressor;
	/**
	 * Default constructor
	 */
	UE_API FCurveEditorSelection();

	/**
	 * Constructor which takes a reference to the curve editor, 
	 * which is used to find if a model is read only
	 */
	UE_API FCurveEditorSelection(TWeakPtr<FCurveEditor> InWeakCurveEditor);

	/**
	 * Retrieve this selection's serial number. Incremented whenever a change is made to the selection.
	 */
	FORCEINLINE uint32 GetSerialNumber() const { return SerialNumber; }

	/**
	 * Check whether the selection is empty
	 */
	FORCEINLINE bool IsEmpty() const { return CurveToSelectedKeys.Num() == 0; }

	/**
	 * Retrieve all selected key handles, organized by curve ID
	 */
	FORCEINLINE const TMap<FCurveModelID, FKeyHandleSet>& GetAll() const { return CurveToSelectedKeys; }

	/**
	 * Retrieve a set of selected key handles for the specified curve
	 */
	UE_API const FKeyHandleSet* FindForCurve(FCurveModelID InCurveID) const;

	/**
	 * Count the total number of selected keys by accumulating the number of selected keys for each curve
	 */
	UE_API int32 Count() const;

	/**
	 * Check whether the specified handle is selected
	 */
	UE_API bool IsSelected(FCurvePointHandle InHandle) const;

	/**
	 * Check whether the specified handle and curve ID is contained in this selection.
	 */
	UE_API bool Contains(FCurveModelID CurveID, FKeyHandle KeyHandle, ECurvePointType PointType) const;

	/** Invoked when the selection changes. */
	UE_API FSimpleMulticastDelegate& OnSelectionChanged();

public:

	/**
	 * Add a point handle to this selection, changing the selection type if necessary.
	 */
	UE_API void Add(FCurvePointHandle InHandle);

	/**
	 * Add a key handle to this selection, changing the selection type if necessary.
	 */
	UE_API void Add(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle);

	/**
	 * Add key handles to this selection, changing the selection type if necessary.
	 */
	UE_API void Add(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys);

	/**
	 * Replaces the current key selection in one batched update using prebuilt per-curve key-handle sets.
	 * Curves that are invalid or read-only are skipped using the same validation rules as Add(...).
	 *
	 * @param InCurveToSelectedKeys Map of curve IDs to the exact selected keys to apply for each curve.
	 */
	UE_API void Replace(const TMap<FCurveModelID, FKeyHandleSet>& InCurveToSelectedKeys);

	/**
	 * Replaces the current key selection in one batched update using raw per-curve key-handle arrays.
	 * Curves that are invalid or read-only are skipped using the same validation rules as Add(...).
	 * All keys supplied for a given curve are associated with InPointType.
	 *
	 * @param InCurveToSelectedKeys Map of curve IDs to the selected key handles to apply for each curve.
	 * @param InPointType Point type to assign to every supplied key handle.
	 */
	UE_API void Replace(const TMap<FCurveModelID, TArray<FKeyHandle>>& InCurveToSelectedKeys, const ECurvePointType InPointType);

public:

	/**
	 * Toggle the selection of the specified point handle, changing the selection type if necessary.
	 */
	UE_API void Toggle(FCurvePointHandle InHandle);

	/**
	 * Toggle the selection of the specified key handle, changing the selection type if necessary.
	 */
	UE_API void Toggle(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle);

	/**
	 * Toggle the selection of the specified key handles, changing the selection type if necessary.
	 */
	UE_API void Toggle(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys);

public:

	/**
	 * Remove the specified point handle from the selection
	 */
	UE_API void Remove(FCurvePointHandle InHandle);

	/**
	 * Remove the specified key handle from the selection
	 */
	UE_API void Remove(FCurveModelID CurveID, ECurvePointType PointType, FKeyHandle KeyHandle);

	/**
	 * Remove the specified key handles from the selection
	 */
	UE_API void Remove(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys);

	/**
	 * Remove all key handles associated with the specified curve ID from the selection
	 */
	UE_API void Remove(FCurveModelID InCurveID);

	/**
	 * Clear the selection entirely
	 */
	UE_API void Clear();

public:

	/** Serializes the SerialNumber and CurveToSelectedKeys. */
	UE_API void SerializeSelection(FArchive& Ar);

private:

	/** Weak reference to the curve editor to check whether keys are locked or not */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** A serial number that increments every time a change is made to the selection */
	uint32 SerialNumber;

	/** A map of selected handles stored by curve ID */
	TMap<FCurveModelID, FKeyHandleSet> CurveToSelectedKeys;

	/** Invoked when the selection changes. */
	UE::CurveEditor::FSuppressibleEventBroadcaster OnSelectionChangedBroadcaster;

	/** Prevents OnSelectionChanged from being Broadcast until a matching call to DecrementOnSelectionChangedSuppressionCount is made. */
	void IncrementOnSelectionChangedSuppressionCount();
	/** Allows OnSelectionChanged to be Broadcast if the counter reaches 0. Invokes OnSelectionChanged if a change has been made since. */
	void DecrementOnSelectionChangedSuppressionCount();
	
	/** Adds the specified key handles from the selection without validating whether CurveId and Keys exists in the owning curve editor. */
	void AddInternal(FCurveModelID CurveID, ECurvePointType PointType, TArrayView<const FKeyHandle> Keys, bool bIncrementSerialNumber = true);

	/** Shared logic after some changes have been made. */
	void PostChangesMade();
};

#undef UE_API
