// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorSettings.generated.h"

struct FPointerEvent;

#define UE_API CURVEEDITOR_API

class UClass;

UENUM()
enum class ECurveEditorPanningMouseButton : uint8
{
	Right	  UMETA(DisplayName = "Right"),
	AltMiddle UMETA(DisplayName = "Alt-Middle"),
	RightOrAltMiddle      UMETA(DisplayName = "Right or Alt-Middle")
};

/** Defines visibility states for the tangents in the curve editor. */
UENUM()
enum class ECurveEditorTangentVisibility : uint8
{
	/** All tangents should be visible. */
	AllTangents,
	/** Only tangents from selected keys should be visible. */
	SelectedKeys,
	/** Don't display tangents. */
	NoTangents,
	/** Only tangents that are user edited, i.e. set to user or break should be visible. */
	UserTangents,

	// Add new entries above.
	Num UMETA(Hidden)
};

/** Defines the position to center the zoom about in the curve editor. */
UENUM()
enum class ECurveEditorZoomPosition : uint8
{
	/** Playhead. */
	CurrentTime UMETA(DisplayName = "Playhead"),

	/** Mouse Position. */
	MousePosition UMETA(DisplayName = "Mouse Position"),
};

/** Defines the axis to snap to when dragging. */
UENUM()
enum class ECurveEditorSnapAxis : uint8
{
	/** Don't snap to any axis when dragging. */
	CESA_None UMETA(DisplayName = "None"),
	/* Snap to the x axis when dragging. */
	CESA_X UMETA(DisplayName = "X Only"),
	/* Snap to the y axis when dragging. */
	CESA_Y UMETA(DisplayName = "Y Only")
};

/** Custom Color Object*/
USTRUCT()
struct FCustomColorForChannel
{
	GENERATED_BODY()

	FCustomColorForChannel() : Object(nullptr), Color(1.0f, 1.0f, 1.0f, 1.0f) {};

	UPROPERTY(EditAnywhere, Category = "Custom Colors")
	TSoftClassPtr<UObject>	Object;
	UPROPERTY(EditAnywhere, Category = "Custom Colors")
	FString PropertyName;
	UPROPERTY(EditAnywhere, Category = "Custom Colors")
	FLinearColor Color;


};

/** Custom Color Object*/
USTRUCT()
struct FCustomColorForSpaceSwitch
{
	GENERATED_BODY()

	FCustomColorForSpaceSwitch() : Color(1.0f, 1.0f, 1.0f, 1.0f) {};

	UPROPERTY(EditAnywhere, Category = "Space Switch Colors")
	FString ControlName;
	UPROPERTY(EditAnywhere, Category = "Space Switch Colors")
	FLinearColor Color;

};

/** Serializable options for curve editor. */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UCurveEditorSettings : public UObject
{
public:
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE(FOnCustomColorsChanged);
	DECLARE_MULTICAST_DELEGATE(FOnAxisSnappingChanged);
	DECLARE_MULTICAST_DELEGATE(FOnShowValueIndicatorsChanged);

	UE_API UCurveEditorSettings();

	/** Get whether the mouse event allows panning or editing */
	bool AllowMousePan(const FPointerEvent& MouseEvent) const;
	bool AllowMouseEdit(const FPointerEvent& MouseEvent) const;

	/** Gets whether or not the curve editor auto frames the selected curves. */
	UE_API bool GetAutoFrameCurveEditor() const;
	/** Sets whether or not the curve editor auto frames the selected curves. */
	UE_API void SetAutoFrameCurveEditor(bool InbAutoFrameCurveEditor);

	/** Gets whether or not the curve editor shows the transport controls panel. */
	UE_API bool GetShowTransportControls() const;
	/** Sets whether or not the curve editor shows the transport controls panel. */
	UE_API void SetShowTransportControls(const bool bInShow);

	/** Gets whether or not the curve editor shows key bar style curves, like for constraints and spaces. */
	UE_API bool GetShowBars() const;
	/** Sets whether or not the curve editor shows key bar style curves, like for constraints and spaces. */
	UE_API void SetShowBars(bool InShowBars);

	/** Gets the number of pixels to pad input framing */
	UE_API int32 GetFrameInputPadding() const;
	/** Sets the number of pixels to pad input framing */
	UE_API void SetFrameInputPadding(int32 InFrameInputPadding);

	/** Gets the number of pixels to pad output framing */
	UE_API int32 GetFrameOutputPadding() const;
	/** Sets the number of pixels to pad output framing */
	UE_API void SetFrameOutputPadding(int32 InFrameOutputPadding);

	/** Gets whether or not to show buffered curves in the curve editor. */
	UE_API bool GetShowBufferedCurves() const;
	/** Sets whether or not to show buffered curves in the curve editor. */
	UE_API void SetShowBufferedCurves(bool InbShowBufferedCurves);


	/** Get whether inserted key will be user tangent when possible. */
	UE_API bool GetCubicAddKeyPrefersUserTangent() const;
	/** Sets wGet whether inserted key will be user tangent when possible. */
	UE_API void SetCubicAddKeyPrefersUserTangent(bool bInCubicAddKeyPrefersUserTangent);

	/** Gets whether or not to show curve tool tips in the curve editor. */
	UE_API bool GetShowCurveEditorCurveToolTips() const;
	/** Sets whether or not to show curve tool tips in the curve editor. */
	UE_API void SetShowCurveEditorCurveToolTips(bool InbShowCurveEditorCurveToolTips);

	/** Gets the current tangent visibility. */
	UE_API ECurveEditorTangentVisibility GetTangentVisibility() const;
	/** Sets the current tangent visibility. */
	UE_API void SetTangentVisibility(ECurveEditorTangentVisibility InTangentVisibility);

	/** Get zoom in/out position (mouse position or current time). */
	UE_API ECurveEditorZoomPosition GetZoomPosition() const;
	/** Set zoom in/out position (mouse position or current time). */
	UE_API void SetZoomPosition(ECurveEditorZoomPosition InZoomPosition);

	/** Get snap axis. */
	UE_API ECurveEditorSnapAxis GetSnapAxis() const;
	/** Set snap axis. */
	UE_API void SetSnapAxis(ECurveEditorSnapAxis InSnapAxis);

	/** Get whether to snap the time to the currently selected key. */
	UE_API bool GetSnapTimeToSelection() const;
	/** Set whether to snap the time to the currently selected key. */
	UE_API void SetSnapTimeToSelection(bool bInSnapTimeToSelection);

	/** Set the selection color. */
	UE_API void SetSelectionColor(const FLinearColor& InColor);
	/** Get the selection color. */
	UE_API FLinearColor GetSelectionColor() const;

	/** Get custom color for object and property if it exists, if it doesn't the optional won't be set */
	UE_API TOptional<FLinearColor> GetCustomColor(UClass* InClass, const FString& InPropertyName) const;
	/** Set Custom Color for the specified parameters. */
	UE_API void SetCustomColor(UClass* InClass, const FString& InPropertyName, FLinearColor InColor);
	/** Delete Custom Color for the specified parameters. */
	UE_API void DeleteCustomColor(UClass* InClass, const FString& InPropertyName);
	
	/** Gets the multicast delegate which is run whenever custom colors have changed. */
	FOnCustomColorsChanged& GetOnCustomColorsChanged() { return OnCustomColorsChangedEvent; }
	/** Gets the multicast delegate which is run whenever axis snapping has changed. */
	FOnAxisSnappingChanged& GetOnAxisSnappingChanged() { return OnAxisSnappingChangedEvent; }
	/** Gets the multicast delegate which is run whenever showing the value indicator lines has changed. */
	FOnShowValueIndicatorsChanged& OnShowValueIndicatorsChanged() { return OnShowValueIndicatorsChangedEvent; }

	/** Get custom color for space name. Parent and World are reserved names and will be used instead of the specified control name. */
	UE_API TOptional<FLinearColor> GetSpaceSwitchColor(const FString& InControlName) const;
	/** Set Custom Space SwitchColor for the specified control name. */
	UE_API void SetSpaceSwitchColor(const FString& InControlName, FLinearColor InColor);
	/** Delete Custom Space Switch Color for the specified control name. */
	UE_API void DeleteSpaceSwitchColor(const FString& InControlName);

	/** Helper function to get next random linear color*/
	static UE_API FLinearColor GetNextRandomColor();

	/** Gets the tree view width percentage */
	float GetTreeViewWidth() const { return TreeViewWidth; }
	/** Sets the tree view width percentage */
	UE_API void SetTreeViewWidth(float InTreeViewWidth);

	/** Gets how sensitive the selection marquee should be when selecting points. */
	float GetMarqueePointSensitivity() const { return FMath::Clamp(MarqueePointSensitivity, 0.f, 1.f); }
	/** Sets how sensitive the selection marquee should be when selecting points. */
	UE_API void SetMarqueePointSensitivity(float InMarqueePointSensitivity);

	/** @return Whether to draw a value indicator line for the minimum and maximum key in the selected key range. */
	bool GetShowValueIndicators() const { return bShowValueIndicators; }
	/** Sets whether to draw a value indicator line for the minimum and maximum key in the selected key range. */
	UE_API void SetShowValueIndicators(bool bValue);

	/** Gets whether or not scrubbing time hot key starts from cursor. */
	UE_API bool GetScrubTimeStartFromCursor() const;
	/** Sets whether or not scrubbing time hot key starts from cursor. */
	UE_API void SetScrubTimeStartFromCursor(bool bInValue);

	/** Gets whether middle mouse button adds a keyframe in the curve editor. */
	UE_API bool GetMiddleMouseButtonAddsKey() const;
	/** Sets whether middle mouse button adds a keyframe in the curve editor. */
	UE_API void SetMiddleMouseButtonAddsKey(bool bInValue);

protected:

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	ECurveEditorPanningMouseButton PanningMouseButton;

	/** When enabled, clicking middle mouse button on a curve adds a keyframe at that position */
	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bMiddleMouseButtonAddsKey;

	UPROPERTY( config, EditAnywhere, Category="Curve Editor" )
	bool bAutoFrameCurveEditor;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bShowTransportControls;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bShowBars;

	/* Number of pixels to add as padding in the input axis when framing curve keys */
	UPROPERTY( config, EditAnywhere, Category="Curve Editor", meta=(ClampMin=0) )
	int32 FrameInputPadding;

	/* Number of pixels to add as padding in the output axis when framing curve keys */
	UPROPERTY( config, EditAnywhere, Category="Curve Editor", meta=(ClampMin=0) )
	int32 FrameOutputPadding;

	UPROPERTY( config, EditAnywhere, Category="Curve Editor" )
	bool bShowBufferedCurves;

	UPROPERTY( config, EditAnywhere, Category="Curve Editor" )
	bool bShowCurveEditorCurveToolTips;

	UPROPERTY( config, EditAnywhere, Category="Curve Editor" )
	ECurveEditorTangentVisibility TangentVisibility;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	ECurveEditorZoomPosition ZoomPosition;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	ECurveEditorSnapAxis SnapAxis;

	UPROPERTY( config, EditAnywhere, Category="Curve Editor")
	bool bSnapTimeToSelection;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	FLinearColor SelectionColor;

	UPROPERTY(config, EditAnywhere, Category="Curve Editor")
	TArray<FCustomColorForChannel> CustomColors;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	FLinearColor ParentSpaceCustomColor;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	FLinearColor WorldSpaceCustomColor;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	TArray<FCustomColorForSpaceSwitch> ControlSpaceCustomColors;

	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	float TreeViewWidth;

	/** When enabled, scrubbing time hotkey will move time from initial cursor position  */
	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bScrubTimeStartFromCursor;

	/** When enabled, inserted key will be user tangent when possible */
	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bCubicAddKeyPrefersUserTangent;
	/**
	 * Determines how close you must move the selection marque to the center of a point in order to select it.
	 * 
	 * This is the percentage of point's center to the point's widget border that must be overlapped with the marquee in order for the point to be selected.
	 * 1.0 means as soon as marquee overlaps any portion of the point.
	 * 0.0 means you must touch the point's center to select it.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Curve Editor", meta = (ClampMin = 0.0, ClampMax = 1.0))
	float MarqueePointSensitivity = 0.45f;

	/** When you select a single curve, whether to draw a dotted line for the minimum and maximum key in the selected key range. */
	UPROPERTY(config, EditAnywhere, Category = "Curve Editor")
	bool bShowValueIndicators = true;

private:
	FOnCustomColorsChanged OnCustomColorsChangedEvent;
	FOnAxisSnappingChanged OnAxisSnappingChangedEvent;
	FOnShowValueIndicatorsChanged OnShowValueIndicatorsChangedEvent;
};

#undef UE_API
