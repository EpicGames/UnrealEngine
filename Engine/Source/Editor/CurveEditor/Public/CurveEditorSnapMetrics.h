// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/MinElement.h"
#include "Input/Events.h"
#include "Misc/FrameRate.h"
#include "Math/Axis.h"
#include "Math/Vector2D.h"
#include "CurveEditorSettings.h"

struct FCurveSnapMetrics
{
	FCurveSnapMetrics()
	{
		bSnapOutputValues = 0;
		bSnapInputValues = 0;
	}

	/** Whether we are snapping to the output snap interval */
	uint8 bSnapOutputValues : 1;

	/** Whether we are snapping to the input snap rate */
	uint8 bSnapInputValues : 1;

	/** Grid lines to snap to */
	TArray<double> AllGridLines;

	/** The input snap rate */
	FFrameRate InputSnapRate;

	/** Snap the specified input time to the input snap rate if necessary */
	FORCEINLINE double SnapInputSeconds(double InputTime) const
	{
		return bSnapInputValues && InputSnapRate.IsValid() ? (InputTime * InputSnapRate).RoundToFrame() / InputSnapRate : InputTime;
	}
	
	/** Snap the specified output value to the output snap interval if necessary */
	FORCEINLINE double SnapOutput(double OutputValue) const
	{
		return bSnapOutputValues && !AllGridLines.IsEmpty() ? *Algo::MinElement(AllGridLines,
			[OutputValue](double Val1, double Val2) { return FMath::Abs(Val1 - OutputValue) < FMath::Abs(Val2 - OutputValue); }
		) : OutputValue;
	}
};

/**
 * Utility struct that acts as a way to control snapping to a specific axis based on UI settings, or shift key.
 */
struct FCurveEditorAxisSnap
{
	/**
	 * Snapping is not stateless but we want to manage it through the central area. This allows
	 * state to be passed into from the calling area but still centralize the logic of handling it.
	*/
	struct FSnapState
	{
		FSnapState()
		{
			Reset();
		}

		void Reset()
		{
			MouseLockVector = FVector2D::UnitVector;
			MousePosOnShiftStart = FVector2D::ZeroVector;
			bHasPassedThreshold = false;
			bHasStartPosition = false;
		}

		FVector2D MousePosOnShiftStart;
		FVector2D MouseLockVector;
		bool bHasPassedThreshold;
		bool bHasStartPosition;
	};

	/** Can be set to either X, Y, or None to control which axis GetSnappedPosition snaps to. User can override None by pressing shift. */
	ECurveEditorSnapAxis RestrictedAxisList;

	FCurveEditorAxisSnap()
	{
		RestrictedAxisList = ECurveEditorSnapAxis::CESA_None;
	}

	/**
	 * Combines an InitialPosition and mouse movement to produce a final position that respects the axis snapping settings.
	 * Pressing shift ignores the snapping settings.
	 * 
	 * For example, if movement is constrained to x-axis only and the mouse moves in direction FVector2D{ 100, 200 }, only the delta movement of
	 * FVector2D{ 100, 0} is applied.
	 * 
	 * @return The end position resulting from applying the snapping behavior to the mouse movement.
	 */
	FVector2D GetSnappedPosition(
		const FVector2D& InitialPosition,
		const FVector2D& LastPosition,
		const FVector2D& CurrentPosition,
		const FPointerEvent& MouseEvent,
		FSnapState& InOutSnapState,
		const bool bIgnoreAxisLock = false
		)
	{
		FVector2D MouseLockVector = FVector2D::UnitVector;

		if (MouseEvent.IsShiftDown())
		{
			if (!InOutSnapState.bHasStartPosition)
			{
				InOutSnapState.MousePosOnShiftStart = LastPosition;
				InOutSnapState.bHasStartPosition = true;
			}
			// If they have passed the threshold they should have a lock vector they're snapped to.
			if (!InOutSnapState.bHasPassedThreshold)
			{
				InOutSnapState.MouseLockVector = MouseLockVector;
					
				// They have not passed the threshold yet, let's see if they've passed it now.
				const FVector2D DragDelta = CurrentPosition - InOutSnapState.MousePosOnShiftStart;
				if (DragDelta.Size() > 0.001f)
				{
					InOutSnapState.bHasPassedThreshold = true;
					InOutSnapState.MouseLockVector = FVector2D::UnitVector;
					if (FMath::Abs(DragDelta.X) > FMath::Abs(DragDelta.Y))
					{
						InOutSnapState.MouseLockVector.Y = 0;
					}
					else
					{
						InOutSnapState.MouseLockVector.X = 0;
					}
				}
			}
			
			MouseLockVector = InOutSnapState.MouseLockVector;
		}
		else
		{
			// If they don't have shift pressed anymore just disable the lock.
			InOutSnapState.Reset();

			if (!bIgnoreAxisLock && RestrictedAxisList == ECurveEditorSnapAxis::CESA_X)
			{
				MouseLockVector.Y = 0;
			}
			else if (!bIgnoreAxisLock && RestrictedAxisList == ECurveEditorSnapAxis::CESA_Y)
			{
				MouseLockVector.X = 0;
			}
		}

		return InitialPosition + (CurrentPosition - InitialPosition) * MouseLockVector;
	}
};
