// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"
#include "Curves/KeyHandle.h"
#include "Containers/BitArray.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

/**
 * Enum for representing the type of a key point in the curve editor
 */
enum class ECurvePointType : uint8
{
	None			= 0x000,
	Key 			= 0x001,
	ArriveTangent 	= 0x002,
	LeaveTangent 	= 0x004,
	Any = Key | ArriveTangent | LeaveTangent
};
ENUM_CLASS_FLAGS(ECurvePointType)

enum class ECurveEditorTreeSelectionState : uint8
{
	None, Explicit, ImplicitChild
};

/** Enumeration identifying a single specific view type, or a combination thereof */
enum class ECurveEditorViewID : uint64
{
	Invalid      = 0,

	Absolute     = 1 << 0,
	Normalized   = 1 << 1,
	Stacked      = 1 << 2,

	CUSTOM_START = 1 << 3,
	ANY_BUILT_IN = Absolute | Normalized | Stacked,
};
ENUM_CLASS_FLAGS(ECurveEditorViewID);


/**
 * A unique identifier for a curve model existing on a curve editor
 */
struct FCurveModelID
{
	/** Default constructor is needed by e.g. TArray::operator<< */
	FCurveModelID() : ID(0) {}

	/**
	 * Generate a new curve model ID
	 */
	static UE_API FCurveModelID Unique();

	/**
	 * @return Whether this ID is valid (was constructed with Unique)
	 */
	bool IsValid() const { return ID != 0; }

	/**
	 * Check two IDs for equality
	 */
	FORCEINLINE friend bool operator==(FCurveModelID A, FCurveModelID B)
	{
		return A.ID == B.ID;
	}

	/**
	 * Check two IDs for inequality
	 */
	FORCEINLINE friend bool operator!=(FCurveModelID A, FCurveModelID B)
	{
		return A.ID != B.ID;
	}

	/**
	 * Test whether A is less than B
	 */
	FORCEINLINE friend bool operator<(FCurveModelID A, FCurveModelID B)
	{
		return A.ID < B.ID;
	}

	/**
	 * Hash a curve model ID
	 */
	FORCEINLINE friend uint32 GetTypeHash(FCurveModelID In)
	{
		return GetTypeHash(In.ID);
	}

	/**
	 * Save a curve model ID
	 */
	FORCEINLINE friend FArchive& operator<<(FArchive& Ar, FCurveModelID& Handle)
	{
		Ar << Handle.ID;
		return Ar;
	}

private:

	/** Internal serial ID */
	uint32 ID;
};




/**
 * A unique handle to a particular point handle (key, tangent handle etc) on a curve, represented by the key's handle, its curve ID, and its type
 */
struct FCurvePointHandle
{
	FCurvePointHandle(FCurveModelID InCurveID, ECurvePointType InPointType, FKeyHandle InKeyHandle)
		: CurveID(InCurveID), PointType(InPointType), KeyHandle(InKeyHandle)
	{}

	/** The curve ID of the key's curve */
	FCurveModelID CurveID;
	/** The type of this point */
	ECurvePointType PointType;
	/** The key handle for the underlying key */
	FKeyHandle KeyHandle;
};

struct FCurveEditorTreeItemID
{
	FCurveEditorTreeItemID()
		: Value(0)
	{}

	friend bool operator==(FCurveEditorTreeItemID A, FCurveEditorTreeItemID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FCurveEditorTreeItemID A, FCurveEditorTreeItemID B)
	{
		return A.Value != B.Value;
	}

	friend uint32 GetTypeHash(FCurveEditorTreeItemID ID)
	{
		return GetTypeHash(ID.Value);
	}

	static FCurveEditorTreeItemID Invalid()
	{
		return FCurveEditorTreeItemID();
	}

	bool IsValid() const
	{
		return Value != 0;
	}

	uint32 GetValue() const
	{
		return Value;
	}

protected:

	friend class FCurveEditorTree;
	uint32 Value;
};

#undef UE_API
