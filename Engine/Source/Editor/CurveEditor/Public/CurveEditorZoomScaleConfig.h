// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "CurveEditorZoomScaleConfig.generated.h"

USTRUCT()
struct FCurveEditorZoomScaleConfig
{
	GENERATED_BODY()

	/** Multiplier when zooming uniformly using mouse wheel. 1.0 means normal. Values < 1.0, make zoom 'slower', and > 1.0 zoom 'faster'. */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta = (ClampMin = 0.05, ClampMax = 32.0))
	double MouseWheelZoomMultiplier = 1.0;

	/**
	 * Easing function when zooming out on the X-axis using Alt+Shift+RMB + move mouse left/right.
	 *
	 * X-value is how far the mouse has travelled on the X-axis. Positive when zooming out, negative when zooming in.
	 * Y-Value is the resulting zoom multiplier. Must be >= 0.0. When zooming out (i.e. x < 0), the Y value must be <= 1.0 or zooming is unstable.
	 * Tip: You can use "CurveEditor.LogHorizontalZoomMultipliers" CVar to make tuning the x and y values easier.
	 *
	 * Example:
	 * - For Sequencer, the frame rate is the length of 1 (unit length).
	 * - So if Sequencer's target FPS is set to 30, and 45 frames are visible on the X-Axis in Curve Editor, we say the X-axis is 1.5 units wide.
	 * - We'd look up the value 1.5 in this easing function and use that as zoom multiplier.
	 */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta=(
		DisplayName="Right mouse (input axis)",
		XAxisName="Input Axis Length",
		YAxisName="Zoom Factor"))
	FRuntimeFloatCurve HorizontalZoomScale;
	
	/**
	 * Easing function when zooming out on the Y-axis using Alt+RMB + move mouse up/down.
	 *
	 * X-value is how far the mouse has travelled on the Y-axis.  Positive when zooming out, negative when zooming in.
	 * Y-Value is the resulting zoom multiplier. Must be >= 0.0. When zooming out (i.e. x < 0), the Y value must be <= 1.0 or zooming is unstable.
	 * Tip: You can use "CurveEditor.LogVerticalZoomMultipliers" CVar to make tuning the x and y values easier.
	 *
	 * Example:
	 * - Suppose in Curve Editor, the Y-axis bounds are +2000 to -1000.
	 * - The height of the Y-axis is thus 3000.
	 * - We'd look up the value of 3000 in this easing function and use that as zoom multiplier.
	 */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta=(
		DisplayName="Right mouse (output axis)",
		XAxisName="Output Axis Length",
		YAxisName="Zoom Factor"))
	FRuntimeFloatCurve VerticalZoomScale;

	/** If true, when zooming out using right-click the vertical axis size will be limited. */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta = (ClampMin = 0.00))
	bool bLimitHorizontalZoomOut = false;
	/**
	 * If bLimitHorizontalZoomOut is true, then this max allowed value range the input axis can show.
	 * Example: If this is 1000, and you have zoomed out so the axis shows -200 and 800, you can zoom out no further.
	 * 
	 * For Sequencer, this is the max number of seconds. So, you need multiply this with the target FPS. So 1000 would mean a max of 3000 frames
	 * with a target FPS of 30. 
	 */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta = (ClampMin = 0.00, EditCondition = "bLimitHorizontalZoomOut", EditConditionHides))
	double MaxHorizontalZoomOut = 10000.0;
	
	/** If true, when zooming out using right-click the horizontal axis size will be limited. */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta = (ClampMin = 0.00))
	bool bLimitVerticalZoomOut = false;
	/**
	 * If bLimitVerticalZoomOut is true, then this max allowed value range the output axis can show.
	 * Example: If this is 1000, and you have zoomed out so the axis shows -200 and 800, you can zoom out no further.
	 */
	UPROPERTY(config, EditAnywhere, Category="Curve Editor", meta = (ClampMin = 0.00, EditCondition = "bLimitVerticalZoomOut", EditConditionHides))
	double MaxVerticalZoomOut = 10000.0;
	
	FCurveEditorZoomScaleConfig()
	{
		// By default, set this data up to do not additional scaling of the zoom.
		HorizontalZoomScale.EditorCurveData.AddKey(0.0, 1.0);
		HorizontalZoomScale.EditorCurveData.AddKey(1.0, 1.0); // So post infinity linear interpolation works
		HorizontalZoomScale.EditorCurveData.DefaultValue = 1.0;
		// While there is a limit for zooming in, for zooming out we want linear extrap so the user does not have to define an infinitely large X.
		HorizontalZoomScale.EditorCurveData.PostInfinityExtrap = RCCE_Linear;
		
		VerticalZoomScale.EditorCurveData.AddKey(0.0, 1.0);
		VerticalZoomScale.EditorCurveData.AddKey(1.0, 1.0); // So post infinity linear interpolation works
		VerticalZoomScale.EditorCurveData.DefaultValue = 1.0;
		// While there is a limit for zooming in, for zooming out we want linear extrap so the user does not have to define an infinitely large X.
		VerticalZoomScale.EditorCurveData.PostInfinityExtrap = RCCE_Linear;
	}

	double GetMouseWheelZoomMultiplierClamped() const { return FMath::Max(MouseWheelZoomMultiplier, 0.05); }

	/**
	 * Evaluates VerticalZoomScale.
	 * @param InAxisSize Absolute value is Max - Min displayed axis values. Positive when zooming out, negative when zooming in.
	 * @return Correctly clamped zoom factor.
	 */
	double EvalHorizontalZoom(double InAxisSize) const
	{
		const double Factor = HorizontalZoomScale.EditorCurveData.Eval(InAxisSize);
		const bool bIsZoomingIn = Factor < 0.f;
		return bIsZoomingIn ? FMath::Clamp(Factor, 0.05, 1.f) : FMath::Max(0.05, Factor);
	}

	/**
	 * Evaluates VerticalZoomScale.
	 * @param InAxisSize Absolute value is Max - Min displayed axis values. Positive when zooming out, negative when zooming in.
	 * @return Correctly clamped zoom factor.
	 */
	double EvalVerticalZoom(double InAxisSize) const
	{
		const double Factor = VerticalZoomScale.EditorCurveData.Eval(InAxisSize);
		const bool bIsZoomingIn = Factor < 0.f;
		return bIsZoomingIn ? FMath::Clamp(Factor, 0.05, 1.f) : FMath::Max(0.05, Factor);
	}
};