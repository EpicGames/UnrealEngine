// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "UObject/UnrealType.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FCurveEditorAxis;
class FName;
class IBufferedCurveModel;
class SCurveEditorView;
class SWidget;
struct FCurveAttributes;
struct FCurveDrawParams;
struct FCurveEditorScreenSpace;
struct FCurveModelID;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyHandle;
struct FKeyHandleSet;
struct FKeyPosition;

enum class ECurveEditorAxisOrientation : uint8;
enum class ECurvePointType : uint8;

namespace UE::CurveEditor { class ICurveEditorCurveCachePool; }
namespace UE::CurveEditor { struct FCurveDrawParamsHandle; }
namespace UE::CurveEditor { class FScopedCurveChange; }

/**
 * Class that models an underlying curve data structure through a generic abstraction that the curve editor understands.
 */
class FCurveModel
{
	friend UE::CurveEditor::FScopedCurveChange; // For EnterChangeScope & ExitChangeScope.
public:

	FCurveModel()
		: Color(0.2f,0.2f,0.2f)
		, bKeyDrawEnabled(true)
		, SupportedViews(ECurveEditorViewID::ANY_BUILT_IN)
	{}

	virtual ~FCurveModel()
	{
		ensureMsgf(!IsInChangeScope(), TEXT("Change scope was not closed"));
	}

	/**
	 * Access the raw pointer of the curve data
	 */
	virtual const void* GetCurve() const = 0;

	/**
	 * Explicitly modify the curve data. Called before any change is made to the curve.
	 */
	virtual void Modify() = 0;

	/**
	 * Draw the curve for the specified curve editor by populating an array with points on the curve between which lines should be drawn
	 *
	 * @param CurveEditor             Reference to the curve editor that is drawing the curve. Can be used to cull the interpolating points to the visible region.
	 * @param ScreenSpace			  A transform which indicates the use case for the drawn curve. This lets you simplify curves based on their screenspace representation.
	 * @param OutInterpolatingPoints  Array to populate with points (time, value) that lie on the curve.
	 */
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const = 0;

	/**
	 * Draws curve draw params to a cache pool. Curves can implement this to improve performance over DrawCurve by caching their data.
	 *
	 * @param WeakCurveEditor			Weak reference to the curve editor that is drawing the curve. 
	 * @param CurveDrawParamsHandle		A handle to the actual curve draw params.
	 * @param ScreenSpace				The screenspace of this curve
	 * 
	 * @return							Returns the cache pool or nullptr if the curve model does not implement a cache pool.
	 */
	virtual UE::CurveEditor::ICurveEditorCurveCachePool* DrawCurveToCachePool(
		const TSharedRef<FCurveEditor>& CurveEditor,
		const UE::CurveEditor::FCurveDrawParamsHandle& CurveDrawParamsHandle,
		const FCurveEditorScreenSpace& ScreenSpace) 
	{ 
		return nullptr; 
	}

	UE_DEPRECATED(5.5, "Implement the version that doesn't take the Curve Editor parameter.")
	virtual void GetKeys(const FCurveEditor& CurveEditor, double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
	{
		GetKeys(MinTime, MaxTime, MinValue, MaxValue, OutKeyHandles);
	}

	/**
	 * Retrieve all keys that lie in the specified time and value range
	 *
	 * @param MinTime                 Minimum key time to return in seconds
	 * @param MaxTime                 Maximum key time to return in seconds
	 * @param MinValue                Minimum key value to return
	 * @param MaxValue                Maximum key value to return
	 * @param OutKeyHandles           Array to populate with key handles that reside within the specified ranges
	 */
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const = 0;

	/**
	 * Gets all keys in the curve.
	 * Prefer this function over GetKeys because it has the potential to be much faster (GetValueRange evaluation can be slow for bezier curves).
	 */
	UE_API virtual void GetAllKeys(TArray<FKeyHandle>& OutKeyHandles) const;
	/**
	 * Gets all keys in the curve.
	 * Prefer this function over GetKeys because it has the potential to be much faster (GetValueRange evaluation can be slow for bezier curves).
	 */
	TArray<FKeyHandle> GetAllKeys() const { TArray<FKeyHandle> Result; GetAllKeys(Result); return Result; }

	/**
	 * Add keys to this curve
	 *
	 * @param InPositions             Key positions for the new keys
	 * @param InAttributes            Key attributes for the new keys, one per key position
	 * @param OutKeyHandles           (Optional) Pointer to an array view of size InPositions.Num() that should be populated with newly added key handles
	 */
	virtual void AddKeys(TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles = nullptr) = 0;

	UE_DEPRECATED(5.6, "Implement the version that uses the current time.")
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys)
	{
		RemoveKeys(InKeys, 0.0);
	}
	/**
	 * Remove all the keys with the specified key handles from this curve
	 *
	 * @param InKeys				Array of key handles to be removed from this curve
	 * @param InCurrentTime			Current time slider position, may be used to set any default value to the current value at that time if no keys exist
	 */
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) = 0;

	/** Adds the given keys if they are not yet present, or overwrites their current data if already present. */
	UE_API virtual void PutKeys(TConstArrayView<FKeyHandle> InKeys, TConstArrayView<FKeyPosition> InKeyPositions, TConstArrayView<FKeyAttributes> InAttributes);

	/**
	 * Remaps InNewHandles to reference the keys that InCurrentHandles reference.
	 * After this function concludes, the FKeyHandle InCurrentHandles no longer reference any keys in this curve.
	 *
	 * This operation is needed if you have to add keys that are mapped to specific FKeyHandle instances.
	 * A use case for this is undo / redo. Suppose the following user action sequence: 1. Move FKeyHandle(42), 2. Delete FKeyHandle(42), 3. Undo.
	 * In step 3, we must add the key back. Calling FCurveEditor::AddKeys generates a new FKeyHandle, but we need to add it back as FKeyHandle(42);
	 * otherwise step 1's reference to FKeyHandle(42) breaks and undoing the movement action would not work.
	 *
	 * FKeyHandle in InNewHandles that are already mapped to a key are skipped..
	 * The corresponding InCurrentHandles is left unchanged and continues to reference the same key as before the call.
	 * 
	 * FKeyHandles in InCurrentHandles that reference no keys are skipped.
	 *
	 * @param InCurrentHandles Key handles that are currentl in the curve model
	 * @param InNewHandles The key handles to remap to. Each index corresponds to the same index in InCurrentHandles.
	 */
	virtual void ReplaceKeyHandles(TConstArrayView<FKeyHandle> InCurrentHandles, TConstArrayView<FKeyHandle> InNewHandles) {}

	/**
	 * Retrieve all key positions that pertain to the specified input key handles
	 *
	 * @param InKeys                  Array of key handles to get positions for
	 * @param OutKeyPositions         Array to receive key positions, one per index of InKeys
	 */
	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const = 0;

	/**
	 * Assign key positions for the specified key handles
	 *
	 * @param InKeys                 Array of key handles to set positions for
	 * @param InKeyPositions         Array of desired key positions to be applied to each of the corresponding key handles
	 */
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) = 0;
	
	/**
	 * Populate the specified draw info structure with data describing how to draw the specified point type
	 *
	 * @param PointType              The type of point to be drawn
	 * @param InKeyHandle			 The specific key (if possible, otherwise FKeyHandle::Invalid()) to get the info for.
	 * @param OutDrawInfo            Data structure to be populated with draw info for this type of point
	 */
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const = 0;

	/** Get range of input time.
	* @param MinTime Minimum Time
	* @param MaxTime Minimum Time
	*
	*/
	virtual void GetTimeRange(double& MinTime, double& MaxTime) const = 0;

	/** Get range of output values.
	* @param MinValue Minimum Value
	* @param MaxValue Minimum Value
	*/
	virtual void GetValueRange(double& MinValue, double& MaxValue) const = 0;

	/** Get range of output value based on specified input times. By default will just get the range
	* without a specified time
	* @param MinTime Minimum Time
	* @param MaxTime Maximium Time
	* @param MinValue Minimum Value
	* @param MaxValue Minimum Value
	*/
	virtual void GetValueRange(double InMinTime, double InMaxTime, double& MinValue, double& MaxValue) const { GetValueRange(MinValue, MaxValue); }


	/** Get the number of keys
	* @param The number of keys
	*/
	virtual int32 GetNumKeys() const = 0;

	/** Get neighboring keys given the key handle
	 *
	 * @param InKeyHandle The key handle to get neighboring keys for
	 * @param OutPreviousKeyHandle The previous key handle
     * @param OutNextKeyHandle The next key handle
	 */
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const = 0;

	/**
	 * Finds the key before and after InTime.
	 * 
	 * If there is a key at InTime, then OutPreviousKeyHandle and OutNextKeyHandle are set to the same value. If there are multiple keys at InTime
	 * (stacked), the implementation can return an arbitrary one as long as it is consistent (e.g. always the "first one") provided the internal state
	 * has not changed.
	 *
	 * @param InTime The time near which to search for keys
	 * @param OutPreviousKeyHandle The key with time <= InTime that is closest to InTime. Unset if there is no such key.
	 * @param OutNextKeyHandle The key with time >= InTime that is closest to InTime. Unset if there is no such key.
	 */
	UE_API virtual void GetClosestKeysTo(double InTime, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const;

	/**
	 * Get the interpolation mode to use at a specified time
	 *
	 * @param InTime						The time we are looking for an interpolation mode
	 * @param DefaultInterpolationMode		Current default interpolation mode, returned if other keys not found or interpolation not supported
	 * @return Interpolation mode to use at that frame
	 */
	virtual TPair<ERichCurveInterpMode, ERichCurveTangentMode> GetInterpolationMode(const double& InTime, ERichCurveInterpMode DefaultInterpolationMode, ERichCurveTangentMode DefaultTangentMode) const 
	{
		return TPair<ERichCurveInterpMode, ERichCurveTangentMode>(DefaultInterpolationMode, DefaultTangentMode);
	}

	/**
	 * Evaluate this curve at the specified time
	 *
	 * @param InTime                 The time to evaluate at, in seconds.
	 * @param Outvalue               Value to receive the evaluation result
	 * @return true if this curve was successfully evaluated, false otherwise
	 */
	virtual bool Evaluate(double InTime, double& OutValue) const = 0;

public:

	/**
	 * Retrieve ALL key attributes that pertain to the specified input key handles.
	 *
	 * This function is legacy. Prefer use of GetKeyAttributesIncludingAutoComputed or GetKeyAttributesExcludingAutoComputed.
	 * @see GetKeyAttributesIncludingAutoComputed (introduced 5.6).
	 *
	 * @param InKeys                Array of key handles to get attributes for
	 * @param OutAttributes         Array to receive key attributes, one per index of InKeys
	 */
	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
	{}

	/**
	 * Gets the value of ALL attributes including those that are auto-computed.
	 * Use this version if you want to know values e.g. the tangents or weights. Useful for UI code that visualizes tangents.
	 * The result is not intended to be passed to SetKeyAttributes; calling SetKeyAttributes would correct the passed in attributes to reflect the user settings.
	 *
	 * Typically, attributes reflect the settings that the user has manually configured for the keys, so certain attributes may remain unset.
	 * For instance, when TangentMode == RCTM_Auto, tangents and weights are automatically computed, meaning attributes like ArriveTangent are not explicitly set.
	 * Setting ArriveTangent would imply a user-defined value, which is incompatible with TangentMode == RCTM_Auto.
	 *
	 * In some situations, you don't want to know the values the user has set but the values as they would be used for evaluation, e.g. tangents,
	 * weights, etc. For these cases, you can use GetKeyAttributesIncludingAutoComputed.
	 */
	virtual void GetKeyAttributesIncludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
	{
		GetKeyAttributes(InKeys, OutAttributes);
	}

	/**
	 * Gets the attributes set by the user. This specifically excludes auto-computed values.
	 * Use this version if you want the "true" state of the keys. You can pass the result of this to SetKeyAttributes. Useful e.g. for copy-pasting keys.
	 * 
	 * Typically, attributes reflect the settings that the user has manually configured for the keys, so certain attributes may remain unset.
	 * For instance, when TangentMode == RCTM_Auto, tangents and weights are automatically computed, meaning attributes like ArriveTangent are not explicitly set.
	 * Setting ArriveTangent would imply a user-defined value, which is incompatible with TangentMode == RCTM_Auto.
	 *
	 * This function returns only user-set values.
	 *  
	 * @param InKeys                Array of key handles to get attributes for
	 * @param OutAttributes         Array to receive key attributes, one per index of InKeys
	 */
	virtual void GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
	{}

	/**
	 * Assign key attributes for the specified key handles
	 *
	 * @param InKeys                 Array of key handles to set attributes for
	 * @param InAttributes           Array of desired key attributes to be applied to each of the corresponding key handles
	 */
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified)
	{}

	/**
	 * Retrieve curve attributes for this curve
	 *
	 * @param OutAttributes          Attributes structure to populate with attributes that are relevant for this curve
	 */
	virtual void GetCurveAttributes(FCurveAttributes& OutAttributes) const
	{}

	/**
	 * Assign curve attributes for this curve
	 *
	 * @param InAttributes           Attributes structure to assign to this curve
	 */
	virtual void SetCurveAttributes(const FCurveAttributes& InAttributes)
	{}

	/**
	 * Retrieve an option input display offset (in seconds) to apply to all this curve's drawing
	 */
	virtual double GetInputDisplayOffset() const
	{
		return 0.0;
	}

	/**
	 * Retrieve this curve's color
	 */
	virtual FLinearColor GetColor() const
	{
		return IsReadOnly() ? Color.Desaturate(.6f) : Color;
	}

	/**
	 * Create key proxy objects for the specified key handles. One object should be assigned to OutObjects per index within InKeyHandles
	 *
	 * @param InWeakCurveEditor      The curve editor
	 * @param InCurveModelID         The curve model id for these key handles
	 * @param InKeyHandles           Array of key handles to create edit objects for
	 * @param OutObjects             (Out) Array to receive objects that should be used to edit each of the input key handles.
	 */
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
	{}

	UE_DEPRECATED(5.7, "Implement the version that takes a curve editor and curve model ID")
	virtual void CreateKeyProxies(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
	{}

	/**
	 * Called when this curve model is added to a curve editor in order to construct additional 'child'
	 *     curves that are owned by this curve
	 */
	virtual void MakeChildCurves(TArray<TUniquePtr<FCurveModel>>& OutChildCurves) const
	{}

	/**
	 * Creates a copy of this curve, stored in a minimal buffered curve object.
	 * Buffered curves are used to cache the positions and attributes of a curve's keys. After creation, a buffered curve 
	 * can be applied to any curve to set it to its saved state. Each curve must implement its own buffered curve which 
	 * inherits IBufferedCurve and implements the DrawCurve method in order for it to be drawn on screen.
	 * Optionally implemented
	 */
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const
	{
		return nullptr;
	}

	/**
	 * Return a per-curve transform for this model, if used by the view
	 */
	virtual FTransform2d GetCurveTransform() const
	{
		return FTransform2d();
	}

	/**
	 * Returns whether the curve model should be edited or not
	 */
	virtual bool IsReadOnly() const
	{
		return false;
	}

	/**
	 * Returns whether this curve has a default value that selected keys can be reset to.
	 */
	virtual bool HasDefaultValue() const { return false; }

	/**
	 * Returns the default value for this curve. Only meaningful when HasDefaultValue() returns true.
	 */
	virtual double GetDefaultValue() const { return 0.0; }

	/**
	* Get the UObject that owns this CurveModel, for example for Sequencer this would be the UMovieSceneSection
	*/
	virtual UObject* GetOwningObject() const
	{
		return nullptr;
	}

	/** Get if has changed and then reset it, this can be used for caching*/
	virtual bool HasChangedAndResetTest()
	{
		return true;
	}

	/**
	 * Allocate and/or assign axes for the this curve
	 */
	virtual void AllocateAxes(FCurveEditor* InCurveEditor, TSharedPtr<FCurveEditorAxis>& OutHorizontalAxis, TSharedPtr<FCurveEditorAxis>& OutVerticalAxis) const
	{
	}

	/**
	* Get the Object and the name to be used to store the curve model color (see UCurveEditorSettings). By default
	* this is the owning object and the intent name, but it can be overriden, for example for Sequencer it may be the bound object
	*/
	virtual void GetCurveColorObjectAndName(UObject** OutObject, FString& OutName) const
	{
		*OutObject = GetOwningObject();
		OutName = GetIntentionName();
	}
	/**
	 * Helper function for assigning a the same attributes to a number of keys
	 */
	UE_API void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, const FKeyAttributes& InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);

	/**
	 * Helper function for adding a single key to this curve
	 */
	UE_API TOptional<FKeyHandle> AddKey(const FKeyPosition& NewKeyPosition, const FKeyAttributes& InAttributes);


	/** Get a multicast delegate, fired when modifications are made to this curve */
	FORCEINLINE FSimpleMulticastDelegate& OnCurveModified()
	{
		return CurveModifiedDelegate;
	}

	/** @return Multicast delegate invoked after the color is changed. */
	FORCEINLINE FSimpleMulticastDelegate& OnPostColorChanged()
	{
		return PostColorChangedDelegate;
	}

	/**
	 * Get the owning object of this curve as the specified type, or search its outer chain if it
	 * is not the specified type. Subclasses can override the UClass-taking virtual below to inject
	 * objects (e.g. a directly-stored UMovieSceneSection on time warp curves whose owning getter
	 * is sequence-outered rather than section-outered).
	 */
	UE_API virtual UObject* GetOwningObjectOrOuter(UClass* Class) const;

	template<typename T>
	T* GetOwningObjectOrOuter() const
	{
		return Cast<T>(GetOwningObjectOrOuter(T::StaticClass()));
	}

public:

	/**
	 * Access this curve's short display name. This is useful when there are other UI elements which describe
	 * enough context about the curve that a long name is not needed (ie: Showing just "X" because other
	 * UI elements give the object/group context).
	 */
	FORCEINLINE FText GetShortDisplayName() const
	{
		return ShortDisplayName;
	}

	/**
	 * Assign a short display name for this curve
	 */
	FORCEINLINE void SetShortDisplayName(const FText& InDisplayName)
	{
		ShortDisplayName = InDisplayName;
	}

	/**
	 * Access this curve's long display name. This is useful when you want more context about
	 * the curve, such as the object it belongs to, or the group (ie: "Floor.Transform.X") instead
	 * of just "X" or "Transform.X"
	 */
	FORCEINLINE FText GetLongDisplayName() const
	{
		// For convenience fall back to the short display name if they fail to specify a long one.
		if (LongDisplayName.IsEmptyOrWhitespace())
		{
			return GetShortDisplayName();
		}

		return LongDisplayName;
	}

	/**
	 * Assign a long display name for this curve used in contexts where additional context is useful.
	 */
	FORCEINLINE void SetLongDisplayName(const FText& InLongDisplayName)
	{
		LongDisplayName = InLongDisplayName;
	}

	/**
	 * This is an internal name used to try to match different curves with each other. When saving
	 * and later restoring curves on a different set of curves we need a name that gives enough context
	 * to match them up by intention, and not long or short name. For example, a curve might have a short
	 * name of "X", and a long name of "Floor.Transform.Location.X". If you wanted to copy a set of transform
	 * curves and paste them onto another transform, we use this context to match the names together to ensure
	 * your Transform.X gets applied to the other Transform.X - in this example the intention is
	 * for the curve to represent a "Location.X" (so it should be pasteable on any other curve which says
	 * their context is a "Location.X" as well). This is more reliable and more flexible than relying on
	 * short display names (not enough context in the case of seeing Location.X, and Scale.X) and better
	 * than relying on long display names (too much context and no reliable way to substring them).
	 */
	FORCEINLINE FString GetIntentionName() const
	{
		return IntentionName;
	}

	/**
	 * Assign an intention name for this curve which is used internally when applying one curve to another in situations where
	 * multiple curves are visible.
	 */
	FORCEINLINE void SetIntentionName(const FString& InIntentionName)
	{
		IntentionName = InIntentionName;
	}

	FORCEINLINE void SetLongIntentionName(const FString& InIntentionName)
	{
		LongIntentionName = InIntentionName;
	}

	FORCEINLINE FString GetLongIntentionName() const
	{
		return LongIntentionName;
	}

	FORCEINLINE void SetChannelName(const FName& InChannelName)
	{
		ChannelName = InChannelName;
	}

	FORCEINLINE FName GetChannelName() const
	{
		return ChannelName;
	}

	FORCEINLINE void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
		PostColorChangedDelegate.Broadcast();
	}

	/**
	 * Get the dash length used for rendering this curve (0 indicates a solid line should be used)
	 */
	FORCEINLINE float GetDashLength() const
	{
		return DashLengthPx;
	}

	/**
	 * Set the dash length used for rendering this curve (0 indicates a solid line should be used)
	 */
	FORCEINLINE void SetDashLength(float InDashLengthPx)
	{
		DashLengthPx = InDashLengthPx;
	}

	/**
	 * Get this curve's visual thickness in screen space
	 */
	FORCEINLINE float GetThickness() const
	{
		return Thickness;
	}

	/**
	 * Set this curve's visual thickness in screen space
	 */
	FORCEINLINE void SetThickness(float InThickness)
	{
		Thickness = InThickness;
	}

	/**
	 * Retrieves whether or not to disable drawing keys
	 */
	FORCEINLINE bool IsKeyDrawEnabled() const
	{
		return bKeyDrawEnabled.Get();
	}

	/**
	 * Assign whether or not to disable drawing keys
	 */
	FORCEINLINE void SetIsKeyDrawEnabled(TAttribute<bool> bInKeyDrawEnabled)
	{
		bKeyDrawEnabled = bInKeyDrawEnabled;
	}

	/**
	 * Retrieve this curve's supported views
	 */
	ECurveEditorViewID GetSupportedViews() const
	{
		return SupportedViews;
	}

	/** Inits this curve with an optionally specified ID. */
	void InitCurveId(FCurveModelID InCurveModelID = FCurveModelID::Unique())
	{
		if (ensureAlways(!CurveId))
		{
			CurveId = InCurveModelID;
		}
	}

	/** @return A valid curve ID */
	const FCurveModelID& GetOrInitId()
	{
		if (!CurveId)
		{
			InitCurveId();
		}
		return *CurveId;
	}
	
	/** @return The curve ID - can be unset. */
	const TOptional<FCurveModelID>& GetId() const { return CurveId; }

protected:

	/** This curve's short display name. Used in situations where other mechanisms provide enough context about what the curve is (such as "X") */
	FText ShortDisplayName;

	/** This curve's long display name. Used in situations where the UI doesn't provide enough context about what the curve is otherwise (such as "Floor.Transform.X") */
	FText LongDisplayName;

	/** This curve's short intention (such as Transform.X or Scale.X). Used internally to match up curves when saving/restoring curves between different objects. */
	FString IntentionName;
	
	/** 
	* This curve's long intention (such as foot_fk_l.Transform.X or foot_fk_r.Scale.X). Used internally to match up curves when saving/restoring curves between different objects.
	* Long intention names have priority in copy/paste over short intention names, but we fall back to short intention if it's unclear what the user is trying to do.
	*/
	FString LongIntentionName;

	/**
	* The original channel name, used mostly to make sure names match with BP/Scripting
	*/
	FName ChannelName;

	/** This curve's display color */
	FLinearColor Color;

	/** Whether or not to draw curve's keys */
	TAttribute<bool> bKeyDrawEnabled;

	/** When this curve is to be drawn dashed, the length of each dash (or 0.f if solid) */
	float DashLengthPx = 0.f;

	/** Default thickness for this curve */
	float Thickness = 2.f;

	/** A set of views supported by this curve */
	ECurveEditorViewID SupportedViews;

	/** Multicast delegate broadcast on curve modification */
	FSimpleMulticastDelegate CurveModifiedDelegate;

	/** Invokes by SetColor */
	FSimpleMulticastDelegate PostColorChangedDelegate;

	/** @return Whether this curve model is currently in a change scope. */
	bool IsInChangeScope() const { return ChangeScopeCounter > 0; }

	/**
	 * Starts a scope of related changes.
	 *
	 * The curve model will receive a matching OnCloseRootChangeScope call when the scope ends.
	 * The curve model implementation should use this to defer any post-change type of delegates until OnCloseRootChangeScope is invoked.
	 *
	 * The primary use case is recording changes using FScopedCurveChange. Do not call Modify() on the FCurveObject::GetOwningObject().
	 * In the future, this may be extended so the curve model can append internal FChange commands as well, which are to be ordered correctly with
	 * the changes made to the curve model. An example may be that SetKeyPositions should cause a call to UMovieSceneSection::ExpandToFrame.
	 * @see FScopedCurveChange "Interaction with Modify()-based undo" section
	 */
	virtual void OnOpenRootChangeScope() {}
	/** Closes the scope of related changes. The curve model can now call any callbacks it had deferred during the change scope. */
	virtual void OnCloseRootChangeScope() {}
	
private:

	/**
	 * This curve's unique ID.
	 * You can set the ID when you construct the curve. If left unspecified, this is initialized by FCurveEditor when you add the curve.
	 * The ID can only be set once during the lifetime of the FCurveModel to inconsistencies, e.g. TMap<FCurveModelId, TUniquePtr<FCurveModel>>.
	 */
	TOptional<FCurveModelID> CurveId;

	/**
	 * When positive, we're in a change scope. Otherwise, in no change scope.
	 *
	 * It is valid to mutate the curve model outside a change scope. The changes are just not considered to be related.
	 */
	int32 ChangeScopeCounter = 0;

	/**
	 * Starts a scope of related changes.
	 * This is used by the new undo system to avoid Modify call on the FCurveModel::GetOwningObject().
	 *
	 * Every call must be matched by CloseChangeScope.
	 * It's ok to call this multiple times after the scope has been started already.
	 * @see OnOpenRootChangeScope, which is called when the scope is initially started.
	 */
	void OpenChangeScope();
	/**
	 * Closes a scope of related changes.
	 * This is used by the new undo system.
	 *
	 * Every call matches a previous call to OpenChangeScope.
	 * It's ok to call this multiple times after the scope has been started already.
	 * @see OnCloseRootChangeScope, which is called when the scope outermost scope is closed.
	 */
	void CloseChangeScope();
};

#undef UE_API
