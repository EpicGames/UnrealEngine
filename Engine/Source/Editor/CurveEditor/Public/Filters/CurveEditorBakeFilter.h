// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "Filters/CurveEditorFilterBase.h"
#include "ICurveEditorToolExtension.h"
#include "Misc/FrameNumber.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorBakeFilter.generated.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class UObject;
struct FCurveModelID;
struct FKeyHandleSet;

USTRUCT(BlueprintType)
struct FCurveEditorBakeFilterRange
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FFrameNumber Min = FFrameNumber(0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FFrameNumber Max = FFrameNumber(10);
};

UCLASS(MinimalAPI, DisplayName="Bake")
class UCurveEditorBakeFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
public:
	UCurveEditorBakeFilter()
	{
		bUseSeconds = true;

		BakeIntervalInSeconds = 0.1f;
		CustomRangeMinInSeconds = 0.f;
		CustomRangeMaxInSeconds = 1.f;

		BakeInterval = FFrameNumber(1);
	}
protected:
	UE_API virtual void InitializeFilter_Impl(TSharedRef<FCurveEditor> InCurveEditoro) override;
	UE_API virtual void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;
	UE_API virtual bool CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor) override;

public:

	/** The interval between baked keys when there is no valid Display Rate and Tick Resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float BakeIntervalInSeconds;

	/** The interval between baked keys. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	FFrameNumber BakeInterval;

	/** When enabled, CustomRange will be used for the bake region. Otherwise the selected keys will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bCustomRangeOverride;

	/** Specifies a custom range to use for baking when there is no valid Display Rate and Tick Resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bCustomRangeOverride"))
	float CustomRangeMinInSeconds;

	/** Specifies a custom range to use for baking when there is no valid Display Rate and Tick Resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bCustomRangeOverride"))
	float CustomRangeMaxInSeconds;

	/** Specifies a custom range to use for baking */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bCustomRangeOverride"))
	FCurveEditorBakeFilterRange CustomRange;

	/** When enabled, will use BakeIntervalInSeconds and CustomRangeInSeconds. Otherwise, use BakeInterval and CustomRange which are defined by the Display Rate and Tick Resolution. */
	UPROPERTY()
	bool bUseSeconds;

private:
	FFrameRate InitialDisplayRate;
	FFrameRate InitialTickResolution;
};

#undef UE_API
