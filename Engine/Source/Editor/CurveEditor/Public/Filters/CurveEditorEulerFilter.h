// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorEulerFilter.generated.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class UObject;
struct FCurveModelID;
struct FKeyHandleSet;

UCLASS(MinimalAPI, DisplayName = "Euler", meta = (
	CurveEditorLabel = "Euler Filter",
	CurveEditorDescription = "Uses the Euler filter to correct jumps in rotation caused by angle wrapping or gimbal lock. For example, instead of wrapping around with a large jump from 178° to -178°, it smoothly interpolates a 4° transition, ensuring natural and continuous movement.")
	)
class UCurveEditorEulerFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
public:
	UCurveEditorEulerFilter(){}
protected:

	//~ Begin UCurveEditorFilterBase Interface
	UE_API virtual void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;
	UE_API virtual bool CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor) override;
	//~ End UCurveEditorFilterBase Interface
};

#undef UE_API
