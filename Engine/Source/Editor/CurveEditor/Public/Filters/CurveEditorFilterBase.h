// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "CurveEditorSelection.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "CurveEditorFilterBase.generated.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FText;
struct FSlateIcon;
template <typename T> class TSubclassOf;

/**
* An abstract base class for Curve Editor Filtering. If you inherit from this class your class will automatically
* show up in the Curve Editor's Filter dropdown. This allows projects to easily create custom filters to edit curve
* data as they only have to implement this class. Any UPROPERTY(EditAnywhere) properties exposed on the class will
* show up in the Curve Editor Filter panel which allows you to easily create customizable properties or advanced filters
* that rely on user supplied settings.
*
* The CDO is directly edited in the Curve Editor Filter panel so if you wish to save user settings between runs you can
* add the "config" property to your UPROPERTY() and it will automatically get saved.
*/
UCLASS(MinimalAPI, BlueprintType, config=EditorSettings)
class UCurveEditorFilterBase : public UObject
{
	GENERATED_BODY()

public:

	/** @return The value of the UCLASS(meta = (CurveEditorLabel = "value")) */
	static UE_API FText GetLabel(const TSubclassOf<UCurveEditorFilterBase>& InClass);
	/** @return The value of the UCLASS(meta = (CurveEditorDescription = "value")) */
	static UE_API FText GetDescription(const TSubclassOf<UCurveEditorFilterBase>& InClass);
	/**
	 * @return The icon associated with this class. Icons can be overriden in editor styles (e.g. FStarshipEditorStyle) with "ClassIcon.ClassName".
	 * @see FSlateIconFinder::FindIconForClass
	 */
	static UE_API FSlateIcon GetIcon(const TSubclassOf<UCurveEditorFilterBase>& InClass);

	/**
	 * Initialize this filter and any parameters it depends on
	 */
	void InitializeFilter(TSharedRef<FCurveEditor> InCurveEditor)
	{
		InitializeFilter_Impl(InCurveEditor);
	}

	/**
	 * Check whether this filter can be applied to the specified curve editor
	 */
	bool CanApplyFilter(TSharedRef<FCurveEditor> InCurveEditor)
	{
		return CanApplyFilter_Impl(InCurveEditor);
	}

	/** 
	* Applies the filter to all keys on the specified curve.
	* @param		InCurveEditor		The curve editor that owns the FCurveModelIDs to operate on.
	* @param		InCurve				The curve to operate on. 
	* @param		OutKeysToSelect		The filter will empty and initialize the set and return you a set of curves and handles that the filter thinks
	*									should be selected after operating. This is useful for filters that create or destroy keys as it allows them to
	*									maintain the appearance that the selection has not been modified.
	*/
	void ApplyFilter(TSharedRef<FCurveEditor> InCurveEditor, FCurveModelID InCurve, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
	{
		TSet<FCurveModelID> InCurves;
		InCurves.Add(InCurve);

		ApplyFilter(InCurveEditor, InCurves, OutKeysToSelect);
	}

	/**
	* Applies the filter to all keys on the specified curves. The curves are passed to the filter as a set so that a filter can choose
	* to operate on multiple curves at once if data from other curves is important.
	* @param		InCurveEditor		The curve editor that owns the FCurveModelIDs to operate on.
	* @param		InCurve				The curve to operate on.
	* @param		OutKeysToSelect		Returns an array containing the keys that the filter thinks should be selected after operating.
	*									This is useful for filters that create or destroy keys as it allows them to
	*									maintain the appearance that the selection has not been modified.
	*/
	void ApplyFilter(TSharedRef<FCurveEditor> InCurveEditor, FCurveModelID InCurve, TArrayView<FKeyHandle> InKeyHandles, TArray<FKeyHandle>& OutKeysToSelect)
	{
		TMap<FCurveModelID, FKeyHandleSet> KeyMap;
		FKeyHandleSet& HandleSet = KeyMap.Add(InCurve);
		for (FKeyHandle& Handle : InKeyHandles)
		{
			HandleSet.Add(Handle, ECurvePointType::Key);
		}

		TMap<FCurveModelID, FKeyHandleSet> OutSelectionMap;
		ApplyFilter(InCurveEditor, KeyMap, OutSelectionMap);

		FKeyHandleSet& SelectedKeySet = OutSelectionMap.FindOrAdd(InCurve);
		OutKeysToSelect.Reserve(SelectedKeySet.Num());

		for (const FKeyHandle& Handle : SelectedKeySet.AsArray())
		{
			OutKeysToSelect.Add(Handle);
		}
	}

	/**
	* Applies the filter to all keys on the specified curves. The curves are passed to the filter as a set so that a filter can choose
	* to operate on multiple curves at once if data from other curves is important.
	* @param		InCurveEditor		The curve editor that owns the FCurveModelIDs to operate on.
	* @param		InCurve				The set of curves to operate on.
	* @param		OutKeysToSelect		The filter will empty and initialize the set and return you a set of curves and handles that the filter thinks
	*									should be selected after operating. This is useful for filters that create or destroy keys as it allows them to
	*									maintain the appearance that the selection has not been modified.
	*/
	void ApplyFilter(TSharedRef<FCurveEditor> InCurveEditor, const TSet<FCurveModelID>& InCurves, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
	{
		// We select all of the keys on the specified curves and pass them in to the actual method that filters based on your selection set.
		TMap<FCurveModelID, FKeyHandleSet> KeysToOperateOn;
		for (FCurveModelID CurveModelID : InCurves)
		{
			FCurveModel* CurveModel = InCurveEditor->FindCurve(CurveModelID);
			check(CurveModel);

			FKeyHandleSet& HandleSet = KeysToOperateOn.Add(CurveModelID);
			for (const FKeyHandle& Handle : CurveModel->GetAllKeys())
			{
				HandleSet.Add(Handle, ECurvePointType::Key);
			}
		}

		ApplyFilter(InCurveEditor, KeysToOperateOn, OutKeysToSelect);
	}

	/**
	* Applies the filter to all keys on the specified curves and specified keys. The curves are passed to the filter as a set so that a filter can choose
	* to operate on multiple curves at once if data from other curves is important. Be warned that many filter operations operate on ranges so the specified
	* keys may be interpreted as a min/max bounds by a filter and all keys in that curve between the two may get selected.
	* @param		InCurveEditor		The curve editor that owns the FCurveModelIDs to operate on.
	* @param		InKeysToOperateOn	The set of curves to operate on and the keys associated with each curve that the user wishes to apply the filter to.
	* @param		OutKeysToSelect		The filter will empty and initialize the set and return you a set of curves and handles that the filter thinks
	*									should be selected after operating. This is useful for filters that create or destroy keys as it allows them to
	*									maintain the appearance that the selection has not been modified.
	*/
	void ApplyFilter(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
	{
		ApplyFilter_Impl(InCurveEditor, InKeysToOperateOn, OutKeysToSelect);
	}

protected:

	/** Implementation function for initializing this filter */
	virtual void InitializeFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor) {}

	/** An implementation must override this function to implement filtering functionality. This is named different and doesn't use function overloading due to C++ name resolution issues
	* which prevent calling base class functions of the same name (even with different signatures) from a pointer to the derived class. */
	virtual void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) {}

	/** Implementation function for determining whether this filter can be applied */
	UE_API virtual bool CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor);
};

#undef UE_API
