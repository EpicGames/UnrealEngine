// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorGaussianFilter.generated.h"

class FCurveEditor;
class UObject;
struct FCurveModelID;
struct FKeyHandleSet;


USTRUCT(BlueprintType)
struct FGaussianParams
{
	GENERATED_BODY()
	/** Sigma that determines how 'fat' the filter is, higher value means fatter width  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "3", ClampMax = "9999", UIMin = "3", UIMax = "9999"))
	int32 KernelWidth = 5;

	void Reset()
	{
		KernelWidth = 5;
	}
};

UCLASS(DisplayName = "Gaussian", CollapseCategories)
class UCurveEditorGaussianFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
public:
	UCurveEditorGaussianFilter() {};
	/**
	* @param Curve to perform Gaussian
	* @param InParams Params to use to peform reduction
	* @param KeyHandleSet Optional set of keys to operate on, if not set we will operate on all of them
	* @param OutHandleSet The set of new keys.
	*/
	static CURVEEDITOR_API void Gaussian(FCurveModel* Curve, const FGaussianParams& InParams, const TOptional<FKeyHandleSet>& KeyHandleSet,
		FKeyHandleSet& OutHandleSet);
protected:
	void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;

	static TArray<double>  CalculateKernel(int32 KernelWidth);
	static double ApplyKernel(int32 Index, const TArray<double> Kernel, const TArray<FKeyPosition>& KeyPositions);

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FGaussianParams GaussianParams;

};