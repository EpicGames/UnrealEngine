// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorReduceFilter.generated.h"

class FCurveEditor;
class UObject;
struct FCurveModelID;
struct FKeyHandleSet;

UCLASS(DisplayName="Simplify")
class UCurveEditorReduceFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
public:
	UCurveEditorReduceFilter() 
	{
		Tolerance = 0.1f;
		bTryRemoveUserSetTangentKeys = false;
		SampleRate = FFrameRate(120,1);
	}

protected:
	void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;

public:
	/** Minimum change in values required for a key to be considered distinct enough to keep. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float Tolerance;
	
	/** Flag whether or not to use SampleRate for sampling between evaluated keys, which allows for removing user-tangent keys. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bTryRemoveUserSetTangentKeys;
	
	/** Rate at which the curve should be sampled to compare against value tolerance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bTryRemoveUserSetTangentKeys"))
	FFrameRate SampleRate;
};