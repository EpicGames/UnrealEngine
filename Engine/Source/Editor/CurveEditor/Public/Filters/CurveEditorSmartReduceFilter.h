// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Filters/CurveEditorFilterBase.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "CurveEditorSmartReduceFilter.generated.h"

class FCurveEditor;
class UObject;
struct FCurveModelID;
struct FKeyHandleSet;


USTRUCT(BlueprintType)
struct FSmartReduceParams
{
	GENERATED_BODY()
	/** Tolerance threshold, set as a percentage of the value's range  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "0.0", ClampMax = "100.0", UIMin = "0.0", UIMax = "100.0"))
	float TolerancePercentage = 5.0;

	/** Rate at which the curve should be sampled to compare against value tolerance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FFrameRate SampleRate = FFrameRate(30, 1);

	/** Remove all keys from static channels, and set that value as the default */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDeleteStaticChannels = false;

	/** Only reduce static channels */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bOnlyReduceStaticChannels = false;

	void Reset()
	{
		TolerancePercentage = 5.0;
		SampleRate = FFrameRate(30, 1);
		bDeleteStaticChannels = false;
		bOnlyReduceStaticChannels = false;
	}
};

UCLASS(DisplayName = "Smart Reduce", CollapseCategories)
class UCurveEditorSmartReduceFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
public:
	UCurveEditorSmartReduceFilter() {};
	/**
	* @param Curve to perform a reduction that works by finding where values change velocity and then perform a tolerance based bisection on these intervals.
	* @param InParams Params to use to peform reduction
	* @param KeyHandleSet Optional set of keys to operate on, if not set we will operate on all of them
	* @param bNeedToTestExisting If true we will test existing keys for custom tangents so we keep them, if false we don't need to test, for example after a bake
	* @param OutHandleSet The set of new keys.
	*/
	static CURVEEDITOR_API void SmartReduce(FCurveModel* Curve, const FSmartReduceParams& InParams, const TOptional<FKeyHandleSet>& KeyHandleSet,
		const bool bNeedToTestExisting, FKeyHandleSet& OutHandleSet);
protected:
	void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FSmartReduceParams SmartReduceParams;

};