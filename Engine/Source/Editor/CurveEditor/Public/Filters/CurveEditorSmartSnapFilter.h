// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CurveEditorFilterBase.h"
#include "CurveEditorSmartSnapFilter.generated.h"

#define UE_API CURVEEDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI, DisplayName = "Smart Snap", meta = (
	CurveEditorLabel = "Smart Snap",
	CurveEditorDescription = "Snaps selected keys to the closest whole frame.\nThe key is placed where the curve intersects with the frame (imagine vertical line going down from the frame).")
	)
class UCurveEditorSmartSnapFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
protected:

	//~ Begin UCurveEditorFilterBase Interface
	UE_API virtual void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;
	UE_API virtual bool CanApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor) override;
	//~ End UCurveEditorFilterBase Interface
};

#undef UE_API
