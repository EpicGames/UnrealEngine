// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CurveEditorFilterBase.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FUICommandInfo;
class FUICommandList;

namespace UE::CurveEditor
{
class FPromotedFilterContainer;

/**
 * Binds and unbinds commands that are created by FPromotedFilterContainer.
 * When a filter is added to FPromotedFilterContainer, it creates a command to invoke that filter. This class is responsible to bind / unbind it.
 */
class FPromotedFilterCommandBinder : public FNoncopyable
{
public:
	
	/**
	 * @param InContainer Holds the promoted filter commands to bind to. The caller ensures that it outlives the constructed object. 
	 * @param InCommandList The command list to bind commands to.
	 * @param InCurveEditor The curve editor on which the filters will be applied.
	 */
	UE_API explicit FPromotedFilterCommandBinder(
		const TSharedRef<FPromotedFilterContainer>& InContainer,
		const TSharedRef<FUICommandList>& InCommandList,
		const TSharedRef<FCurveEditor>& InCurveEditor
		);
	UE_API ~FPromotedFilterCommandBinder();

private:

	/** The filter container this object binds to. */
	const TWeakPtr<FPromotedFilterContainer> Container;
	/** Commands are added to and removed from this list. */
	const TWeakPtr<FUICommandList> CommandList;
	/** Passed to filter as argument. */
	const TWeakPtr<FCurveEditor> CurveEditor;
	
	UE_API void OnFilterAdded(UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand) const;
	UE_API void OnFilterRemoved(UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand) const;

	UE_API void MapAction(UCurveEditorFilterBase& InFilter, const TSharedRef<FUICommandInfo>& InCommand, FUICommandList& CommandListPin) const;
	
	UE_API void ApplyFilter(UCurveEditorFilterBase* InFilter, TWeakPtr<FUICommandInfo> Command) const;
	UE_API bool CanApplyFilter(UCurveEditorFilterBase* InFilter, TWeakPtr<FUICommandInfo> Command) const;
};
}

#undef UE_API
