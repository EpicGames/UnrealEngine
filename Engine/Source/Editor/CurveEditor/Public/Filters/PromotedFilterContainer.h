// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"

#include <type_traits>

#define UE_API CURVEEDITOR_API

class FBindingContext;
class FCurveEditor;
class FMenuBuilder;
class FToolBarBuilder;
class FUICommandInfo;
class FUICommandList;
class UCurveEditorFilterBase;
template <typename T> class TSubclassOf;

namespace UE::CurveEditor
{
/**
 * Holds the state for surfacing filters to the toolbar from the SCurveEditorFilterPanel, so the user can access & apply them more quickly.
 *
 * TODO UE-230269: For now, the curve editor only adds the Euler filter to this container. Once UE-230269 is addressed (allowing users to surface
 * filters to the toolbar via UI), this class should be extended to persistently save the user-specified filter settings,  e.g. the FGaussianParams
 * for the UCurveEditorGaussianFilter, etc.
 */
class FPromotedFilterContainer
	: public FGCObject
	, public FNoncopyable
{
public:
	
	/**
	 * @param InContextName The ID to give the underlying FBindingContext. Must be globally unique.
	 */
	UE_API explicit FPromotedFilterContainer(FName InContextName);
	UE_API virtual ~FPromotedFilterContainer() override;

	/** Appends the filters to InToolbarBuilder. */
	UE_API void AppendToBuilder(FToolBarBuilder& InToolBarBuilder, const FMenuEntryResizeParams& InResizeParams = {}) const;
	/** Appends the filters to InMenuBuilder. */
	UE_API void AppendToBuilder(FMenuBuilder& InMenuBuilder) const;
	
	/**
	 * Promotes a filter.
	 * For simplicity, each class can only have one instance promoted (e.g. you cannot call AddInstance with two different UCurveEditorEulerFilter instances).
	 */
	UE_API void AddInstance(UCurveEditorFilterBase& InFilter);
	
	/** Removes a filter instance. You can pass in a CDO, too. */
	UE_API void RemoveInstance(UCurveEditorFilterBase& InFilter);

	template<typename TCallback> requires std::is_invocable_v<TCallback, UCurveEditorFilterBase&, const TSharedRef<FUICommandInfo>&>
	void ForEachFilter(TCallback&& Callback);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FFilterDelegate, UCurveEditorFilterBase&, const TSharedRef<FUICommandInfo>&);
	/** Broadcasts when a filter is added to this container. */
	FFilterDelegate& OnFilterAdded() { return OnFilterAddedDelegate; }
	/** Broadcasts when a filter is removed from this container. */
	FFilterDelegate& OnFilterRemoved() { return OnFilterRemovedDelegate; }

	//~ Begin FGCObject Interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FPromotedFilterContainer"); }
	//~ Begin FGCObject Interface

private:

	/** Needed to dynamically create FUICommandInfo. */
	const TSharedRef<FBindingContext> CommandContext;

	/** Broadcasts when a filter is added to this container. */
	FFilterDelegate OnFilterAddedDelegate;
	/** Broadcasts when a filter is removed from this container. */
	FFilterDelegate OnFilterRemovedDelegate;

	struct FFilterData
	{
		/** The instance to apply. This can be the CDO or an actual owned instance. */
		UCurveEditorFilterBase& FilterInstance;

		/**
		 * Command info for this instance, which is used to the filter to menu / toolbar builders.
		 * In the future, it also allows users to bind shortcuts to it.
		 */
		TSharedRef<FUICommandInfo> Command;

		explicit FFilterData(UCurveEditorFilterBase& InFilter UE_LIFETIMEBOUND, TSharedRef<FUICommandInfo> InCommand)
			: FilterInstance(InFilter)
			, Command(InCommand)
		{}
	};

	/** The filters that have been promoted. */
	TArray<FFilterData> PromotedFilters;

	UE_API int32 IndexOf(const UClass* InFilterClass) const;
	UE_API void RemoveAtInternal(int32 InIndex);
};

template <typename TCallback> requires std::is_invocable_v<TCallback, UCurveEditorFilterBase&, const TSharedRef<FUICommandInfo>&>
void FPromotedFilterContainer::ForEachFilter(TCallback&& Callback)
{
	for (const FFilterData& FilterData : PromotedFilters)
	{
		Callback(FilterData.FilterInstance, FilterData.Command);
	}
}
}

#undef UE_API
