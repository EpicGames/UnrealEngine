// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FTabManager;
class IDetailsView;
class SWindow;
class UClass;
class UCurveEditorFilterBase;

class SCurveEditorFilterPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorFilterPanel)
	{}
		
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor, UClass* DefaultFilterClass);
	UE_API void SetFilterClass(UClass* InClass);

public:
	/** Call this to request opening a window containing this panel. */
	static UE_API TSharedPtr<SCurveEditorFilterPanel> OpenDialog(TSharedPtr<SWindow> RootWindow, TSharedRef<FCurveEditor> InHostCurveEditor, TSubclassOf<UCurveEditorFilterBase> DefaultFilterClass);
	
	/** Closes the dialog if there is one open. */
	static UE_API void CloseDialog();

	/** The details view for filter class properties */
	TSharedPtr<class IDetailsView> GetDetailsView() const { return DetailsView; }

	/** Delegate for when the chosen filter class has changed */
	FSimpleDelegate OnFilterClassChanged;

protected:
	UE_API FReply OnApplyClicked();
	UE_API bool CanApplyFilter() const;
	UE_API FText GetCurrentFilterText() const;
private:
	/** Weak pointer to the curve editor which created this filter. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** The Details View in our UI that we update each time they choose a class. */
	TSharedPtr<IDetailsView> DetailsView;

	/** Singleton for the pop-up window. */
	static UE_API TWeakPtr<SWindow> ExistingFilterWindow;
};

#undef UE_API
