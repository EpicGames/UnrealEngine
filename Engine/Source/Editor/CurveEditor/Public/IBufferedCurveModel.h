// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Templates/Tuple.h"
#include "CurveDataAbstraction.h"

struct FCurveEditorScreenSpace;

class FCurveEditor;
class FCurveModel;

/**
 * Represents a buffered curve which can be applied to a standard curve model
 */
class IBufferedCurveModel
{
public:
	IBufferedCurveModel(TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax) 
	: KeyPositions(MoveTemp(InKeyPositions))
	, KeyAttributes(MoveTemp(InKeyAttributes))
	, LongDisplayName(InLongDisplayName)
	, ValueMin(InValueMin)
	, ValueMax(InValueMax)
	{}

	virtual ~IBufferedCurveModel() = default;

	/** draws the curve into an array of (input, output) pairs with a given screen space */
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const = 0;

	/**
	 * @param InTime The time at which to evaluate the curve.
	 * @param OutValue The evaluated curve value at InTime, if this function returned true.
	 * @return Whether the curve was evaluated.
	 */
	virtual bool Evaluate(double InTime, double& OutValue) const = 0;
	
	/**
	 * Retrieve all key positions stored in this buffered curve
	 *
	 * @param OutKeyPositions         Array to receive key positions, one per index of InKeys
	 */
	FORCEINLINE void GetKeyPositions(TArray<FKeyPosition>& OutKeyPositions) const { OutKeyPositions = KeyPositions; }
	
	/**
	 * Retrieve all key attributes stored in this buffered curve
	 * 
	 * @param OutAttributes         Array to receive key attributes, one per index of InKeys
	 */
	FORCEINLINE void GetKeyAttributes(TArray<FKeyAttributes>& OutKeyAttributes) const { OutKeyAttributes = KeyAttributes; }

	/**
	 * Returns the full/long display name of the curves. Buffered curves can only be applied to the original curve it was stored from.
	 */
	FORCEINLINE FString GetLongDisplayName() const { return LongDisplayName; }

	/** returns the lowest output value in curve space for this buffered curve */
	FORCEINLINE double GetValueMin() const { return ValueMin; }

	/** returns the highest output value in curve space for this buffered curve */
	FORCEINLINE double GetValueMax() const { return ValueMax; }

protected:
	TArray<FKeyPosition> KeyPositions;
	TArray<FKeyAttributes> KeyAttributes;
	FString LongDisplayName;
	double ValueMin, ValueMax;
};