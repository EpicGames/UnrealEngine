// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct ICurveEditorBounds
{
	virtual ~ICurveEditorBounds()= default;

	virtual void GetInputBounds(double& OutMin, double& OutMax) const = 0;
	virtual void SetInputBounds(double InMin, double InMax) = 0;
};


struct FStaticCurveEditorBounds : ICurveEditorBounds
{
	double InputMin = 0.0;
	double InputMax = 1.0;

	virtual void GetInputBounds(double& OutMin, double& OutMax) const override final
	{
		OutMin = InputMin;
		OutMax = InputMax;
	}

	virtual void SetInputBounds(double InMin, double InMax) override final
	{
		InputMin = InMin;
		InputMax = InMax;
	}
};
