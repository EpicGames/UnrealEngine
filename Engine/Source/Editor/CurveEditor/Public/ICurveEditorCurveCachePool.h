// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"

class FCurveEditor;
struct FCurveDrawParams;

namespace UE::CurveEditor
{
	class ICurveEditorCurveCachePool
	{
	public:
		/** Draws the cached curves. Called when all curves were drawn to the curve cache pool. */
		virtual void DrawCachedCurves(TWeakPtr<const FCurveEditor> WeakCurveEditor) = 0;
	};
}
