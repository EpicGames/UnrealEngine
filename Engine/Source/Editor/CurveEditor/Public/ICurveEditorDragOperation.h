// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSnapMetrics.h"
#include "CurveEditorTypes.h"
#include "Framework/DelayedDrag.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Math/Vector2D.h"
#include "Misc/OptionalFwd.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"

class FCurveEditor;
class FSlateWindowElementList;
struct FCurvePointHandle;
struct FGeometry;
struct FPointerEvent;

/**
 * Interface for all drag operations in the curve editor
 */
class ICurveEditorDragOperation
{
public:

	ICurveEditorDragOperation()
	{}

	virtual ~ICurveEditorDragOperation() = default;

	/**
	 * Mouse button down
	 */
	void MouseButtonDown(FVector2D InitialPosition, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);


	/**
	 * Mouse button up
	 */
	void MouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * Begin this drag operation with the specified initial and current positions
	 */
	void BeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent);

	/**
	 * Continue this drag operation with the specified initial and current positions
	 */
	void Drag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent);

	/** Process any mouse input that this drag operation may have accumulated. */
	void FinishedPointerInput();

	/**
	 * Potentially Evaluate a MouseWheel event which occcured during this drag operation
	 */
	FReply MouseWheel(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent);

	/**
	 * Finish this drag operation with the specified initial and current positions
	 */
	void EndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent);

	/**
	 * Paint this drag operation onto the specified layer
	 */
	void Paint(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId);

	/**
	 * Cancel this drag operation
	 */
	void CancelDrag();

protected:

	/** Implementation method for derived types to handle mouse down*/
	virtual void OnMouseButtonDown(FVector2D InitialPosition, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{}

	/** Implementation method for derived types to handle mouse up*/
	virtual void OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{}

	/** Implementation method for derived types to begin a drag */
	virtual void OnBeginDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
	{}

	/** Implementation method for derived types to continue a drag */
	virtual void OnDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
	{}

	/** Implementation method for derived types to process any accumulated drags. */
	virtual void OnFinishedPointerInput()
	{}

	/** Implementation method for derived types to evaluate a mousewheel event */
	virtual FReply OnMouseWheel(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
	{
		return FReply::Unhandled();
	}

	/** Implementation method for derived types to finish a drag */
	virtual void OnEndDrag(FVector2D InitialPosition, FVector2D CurrentPosition, const FPointerEvent& MouseEvent)
	{}

	/** Implementation method for derived types to paint this drag */
	virtual void OnPaint(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId)
	{}

	/** Implementation method for derived types to cancel a drag */
	virtual void OnCancelDrag()
	{}
protected:
};

/**
 * Interface for all key drag operations in the curve editor
 */
class ICurveEditorKeyDragOperation : public ICurveEditorDragOperation
{
public:

	/**
	 * Initialize this drag operation using the specified curve editor pointer and an optional cardinal point
	 * @param InCurveEditor       Curve editor pointer. Guaranteed to persist for the lifetime of this drag.
	 * @param CardinalPoint       The point that should be considered the origin of this drag.
	 */
	void Initialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& CardinalPoint);

protected:

	/** Implementation method for derived types to initialize a drag */
	virtual void OnInitialize(FCurveEditor* InCurveEditor, const TOptional<FCurvePointHandle>& CardinalPoint) {}
};

/**
 * Utility struct used to facilitate a delayed drag operation with an implementation interface
 */
struct FCurveEditorDelayedDrag : FDelayedDrag
{
	/** The drag implementation to use once the drag has started */
	TUniquePtr<ICurveEditorDragOperation> DragImpl;

	/**
	 * Transaction associated with the operation. Only created for certain operations as not all operations need to transact.
	 * 
	 * Depending on the operation, this transaction may contain more changes than those created by the drag operation.
	 * For example, when clicking a curve with middle mouse button, we may insert a key first and then start moving that key using the drag operation;
	 * the transaction would contain the key insertion and selection change and the drag operation may append a change for moving the inserted key.
	 * This is the reason that the ICurveEditorDragOperation is not responsible for creating the FScopedTransaction.
	 */
	TUniquePtr<FScopedTransaction> Transaction;

	/**
	 * Start a delayed drag operation at the specified position and effective key
	 */
	FCurveEditorDelayedDrag(FVector2D InInitialPosition, FKey InEffectiveKey)
		: FDelayedDrag(InInitialPosition, InEffectiveKey)
	{
		SetTriggerScaleFactor(0.1f);
	}

	/** Cancels the drag operation and transaction. */
	void Cancel()
	{
		if (DragImpl)
		{
			DragImpl->CancelDrag();
			DragImpl.Reset();
		}

		if (Transaction)
		{
			Transaction->Cancel();
			Transaction.Reset();
		}
	}
};
