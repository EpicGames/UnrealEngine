// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"

class FUICommandList;
class FExtender;

class ICurveEditorExtension
{
public:
	virtual ~ICurveEditorExtension() = default;

	/**
	 * Allows the editors to bind commands.
	 *
	 * @param CommandBindings The existing command bindings to map to.
	*/
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) = 0;

	/**
	 * Allows the editors to extend the toolbar.
	 *
	 * @param InCommandList The existing command bindings to map to.
	 */
	virtual TSharedPtr<FExtender> MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList) { return nullptr; }
};
