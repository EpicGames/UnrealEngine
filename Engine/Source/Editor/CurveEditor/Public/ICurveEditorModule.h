// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "CurveEditorDelegates.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "Containers/ArrayView.h"
#include "CurveEditorTypes.h"

struct FCurveEditorZoomScaleConfig;
class FCurveEditor;
class SCurveEditorView;
class ICurveEditorExtension;
class ICurveEditorToolExtension;
class ITimeSliderController;
class FExtender;
class FUICommandList;
namespace UE::CurveEditor { class FPromotedFilterContainer; }

/** A delegate which will create an extension for the Curve Editor. Used for adding new buttons and functionality to the editor as a whole. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<ICurveEditorExtension>, FOnCreateCurveEditorExtension, TWeakPtr<FCurveEditor>);

/** A delegate which will create a tool extension for the Curve Editor. Used for registering new tools which can only have one active at once. */
DECLARE_DELEGATE_RetVal_OneParam(TUniquePtr<ICurveEditorToolExtension>, FOnCreateCurveEditorToolExtension, TWeakPtr<FCurveEditor>);

/** A delegate used for creating a new curve editor view */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SCurveEditorView>, FOnCreateCurveEditorView, TWeakPtr<FCurveEditor>);

/**
 * Curve Editor initialization parameters.
 */
struct FCurveEditorInitParams
{
	FCurveEditorInitParams()
	{}
	
	/** Extensions you want to inject for this curve editor. */
	TArray<TSharedRef<ICurveEditorExtension>> AdditionalEditorExtensions;

	/** Optional. If set, defines multipliers for zooming. */
	TAttribute<const FCurveEditorZoomScaleConfig*> ZoomScalingAttr;

	/**
	 * Internally used for editing FCurveModels after they've been removed FCurveEditor::CurveData (e.g. once they're no longer visible).
	 * Important for applying undo / redo commands on curves that are not visible when the user issues the undo / redo operation.
	 * You can skip providing this if you never call FCurveEditor::RemoveCurve.
	 */
	UE::CurveEditor::FResolveCurveModelDelegate ResolveCurveModelDelegate;
};

/**
 * Interface for the Curve Editor module.
 */
class ICurveEditorModule : public IModuleInterface
{
public:

static ICurveEditorModule& Get()
	{
		return FModuleManager::Get().GetModuleChecked<ICurveEditorModule>("CurveEditor");
	}
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("CurveEditor");
	}
	
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FExtender>, FCurveEditorMenuExtender, const TSharedRef<FUICommandList>);
	virtual ~ICurveEditorModule() = default;

	virtual FDelegateHandle RegisterEditorExtension(FOnCreateCurveEditorExtension InOnCreateCurveEditorExtension) = 0;
	virtual void UnregisterEditorExtension(FDelegateHandle InHandle) = 0;

	virtual FDelegateHandle RegisterToolExtension(FOnCreateCurveEditorToolExtension InOnCreateCurveEditorToolExtension) = 0;
	virtual void UnregisterToolExtension(FDelegateHandle InHandle) = 0;

	/**
	 * Register a new view factory function that can be used on the curve editor when relevant curves are found
	 * @note: A maximum of 64 registered view types are supported. View type IDs are not recycled.
	 *
	 * @param InCreateViewDelegate (required) A bound delegate that creates a new instance of the view widget. Delegate signature is TSharedRef<SCurveEditorView> Function(TWeakPtr<FCurveEditor>);
	 * @return A new custom view ID that identifies the registered view type. Any curve models that wish to support this view must |= this enum to its FCurveModel::SupportedViews;
	 */
	virtual ECurveEditorViewID RegisterView(FOnCreateCurveEditorView InCreateViewDelegate) = 0;

	/**
	 * Unregister a previously registered view type
	 *
	 * @param InViewID The view ID obtained from calling RegisterCustomView. Must be >= ECurveEditorViewID::CUSTOM_START
	 */
	virtual void UnregisterView(ECurveEditorViewID InViewID) = 0;

	virtual TArray<FCurveEditorMenuExtender>& GetAllToolBarMenuExtenders() = 0;
	
	virtual TArrayView<const FOnCreateCurveEditorExtension> GetEditorExtensions() const = 0;
	virtual TArrayView<const FOnCreateCurveEditorToolExtension> GetToolExtensions() const = 0;

	/**
	 * @return The global setting of which filters should be promoted to the toolbar in the curve editor.
	 */
	virtual TSharedPtr<UE::CurveEditor::FPromotedFilterContainer> GetGlobalToolbarPromotedFilters() const = 0;
};
