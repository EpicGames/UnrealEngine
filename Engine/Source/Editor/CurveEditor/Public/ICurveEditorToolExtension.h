// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"
#include "Input/Reply.h"

class FStructOnScope;
class FUICommandList;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
class SWidget;

struct FGeometry;
struct FPointerEvent;
struct FSlateIcon;

/**
* A unique identifier for a tool in the curve editor
*/
struct FCurveEditorToolID
{
	/**
	 * Generate a new tool ID
	 */
	static FCurveEditorToolID Unique()
	{
		static uint32 CurrentID = 1;

		FCurveEditorToolID ID;
		ID.ID = CurrentID++;
		return ID;
	}

	static FCurveEditorToolID Unset()
	{
		FCurveEditorToolID ID;
		ID.ID = -1;
		return ID;
	}

	/**
	 * Check two IDs for equality
	 */
	FORCEINLINE friend bool operator==(FCurveEditorToolID A, FCurveEditorToolID B)
	{
		return A.ID == B.ID;
	}

	/**
	 * Check two IDs for inequality
	 */
	FORCEINLINE friend bool operator!=(FCurveEditorToolID A, FCurveEditorToolID B)
	{
		return A.ID != B.ID;
	}

	/**
	 * Hash a tool ID
	 */
	FORCEINLINE friend uint32 GetTypeHash(FCurveEditorToolID In)
	{
		return GetTypeHash(In.ID);
	}

private:
	FCurveEditorToolID() {}

	/** Internal serial ID */
	uint32 ID;
};

DECLARE_MULTICAST_DELEGATE(FOnOptionsRefresh);

/**
* You can extend the Curve Editor toolset by implementing this interface. The Curve Editor guarantees that only
* one tool will be active at any given time. A tool needs to specify if they handled certain mouse events so that
* these events can be bubbled to the rest of the Curve Editor to allow common functionality of selecting/deselecting
* keys, panning, etc.
*/
class ICurveEditorToolExtension
{
public:
	ICurveEditorToolExtension()
		: ToolID(FCurveEditorToolID::Unset()) 
	{
	}

	virtual ~ICurveEditorToolExtension() = default;

	// Effectively SWidget Interface
	virtual void OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const = 0;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) {}
	virtual FReply OnKeyDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }
	virtual FReply OnKeyUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) { return FReply::Unhandled(); }
	virtual void OnFinishedPointerInput() {}
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) {}
	virtual TSharedPtr<FStructOnScope> GetToolOptions() const { return nullptr; }
	virtual void OnToolOptionsUpdated(const FPropertyChangedEvent& PropertyChangedEvent) {}
	// ~SWidget

	/**
	* This is called when the tool is activated by switching from another tool. The current tool (if any) will have
	* OnToolDeactivated called first before the new tool has OnToolActivated called.
	*/
	virtual void OnToolActivated() = 0;

	/**
	* This is called when the tool is deactivated by switching to another tool. This will be called before the new
	* tool has OnToolActivated called.
	*/
	virtual void OnToolDeactivated() = 0;

	/**
	* Allows the tool to bind commands.
	*
	* @param CommandBindings The existing command bindings to map to.
	*/
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) {}

	/**
	* Allows the tool to add menu items to the toolbar in the Curve Editor.
	*
	* @param ToolbarBuilder The existing toolbar builder to add new columns to or to hook into existing sections for.
	*/
	// virtual void ExtendToolbar(TSharedPtr<FExtender> ToolbarExtender) = 0;

	virtual void SetToolID(const FCurveEditorToolID InToolID)
	{
		ToolID = InToolID;
	}

	/** @return Label for this tool. Displayed by toolbar combo button. */
	virtual FText GetLabel() const = 0;
	/** @return Description for this tool. Displayed by toolbar combo button. */
	virtual FText GetDescription() const = 0;
	/** @return Icon for this tool. Displayed by toolbar combo button. */
	virtual FSlateIcon GetIcon() const = 0;

	FOnOptionsRefresh OnOptionsRefreshDelegate;

protected:
	FCurveEditorToolID ToolID;
};
