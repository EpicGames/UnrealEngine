// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveModel.h"
#include <type_traits>

namespace UE::CurveEditor
{
template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
class TInvertedCurveModel : public TBase
{
public:

	template<typename... TArg>
	explicit TInvertedCurveModel(TArg&&... Arg);
	
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override;
	virtual void AddKeys(TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles = nullptr) override;
	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override;
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	virtual bool Evaluate(double InTime, double& OutValue) const override;
	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void GetKeyAttributesIncludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;
};
}

namespace UE::CurveEditor
{
namespace Private
{
static FKeyPosition InvertKeyPosition(const FKeyPosition& KeyPosition)
{
	return FKeyPosition(KeyPosition.InputValue, -KeyPosition.OutputValue);
}
	
template <typename InT, typename OutT>
static void InvertKeyPositions(const InT& Input, OutT&& Output)
{
	Algo::Transform(Input, Output, [](const FKeyPosition& KeyPosition)
	{
		return InvertKeyPosition(KeyPosition);
	});
}

static void InvertKeyPositions(TArrayView<FKeyPosition> Output)
{
	for (int32 Index = 0; Index < Output.Num(); ++Index)
	{
		Output[Index] = InvertKeyPosition(Output[Index]);
	}
}

static float InvertTangent(float Tangent)
{
	const float Angle = FMath::Atan(Tangent);
	return FMath::Tan(-Angle);
}

static FKeyAttributes InvertKeyAttributes(FKeyAttributes Attributes)
{
	if (Attributes.HasArriveTangent())
	{
		Attributes.SetArriveTangent(InvertTangent(Attributes.GetArriveTangent()));
	}
	if (Attributes.HasLeaveTangent())
	{
		Attributes.SetLeaveTangent(InvertTangent(Attributes.GetLeaveTangent()));
	}
	return Attributes;
}
	
template <typename InT, typename OutT>
static void InvertKeyAttributes(const InT& Input, OutT&& Output)
{
	Algo::Transform(Input, Output, [](const FKeyAttributes& Attributes)
	{
		return InvertKeyAttributes(Attributes);
	});
}

static void InvertKeyAttributes(TArrayView<FKeyAttributes> Output)
{
	for (int32 Index = 0; Index < Output.Num(); ++Index)
	{
		Output[Index] = InvertKeyAttributes(Output[Index]);
	}
}

static TArray<FKeyPosition> CopyAndInvertKeyPositionsFromBuffer(const IBufferedCurveModel& RealBuffer)
{
	TArray<FKeyPosition> KeyPositions;
	RealBuffer.GetKeyPositions(KeyPositions);
	InvertKeyPositions(KeyPositions);
	return KeyPositions;
}

static TArray<FKeyAttributes> CopyAndInvertKeyAttributesFromBuffer(const IBufferedCurveModel& RealBuffer)
{
	TArray<FKeyAttributes> KeyPositions;
	RealBuffer.GetKeyAttributes(KeyPositions);
	InvertKeyAttributes(KeyPositions);
	return KeyPositions;
}
}
	
template <typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
template <typename ... TArg> 
TInvertedCurveModel<TBase>::TInvertedCurveModel(TArg&&... Arg)
	: TBase(Forward<TArg>(Arg)...)
{}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::DrawCurve(
	const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints
	) const
{
	TBase::DrawCurve(CurveEditor, ScreenSpace, InterpolatingPoints);
	for (int32 Index = 0; Index < InterpolatingPoints.Num(); ++Index)
	{
		InterpolatingPoints[Index].Value = -InterpolatingPoints[Index].Value;
	}
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	const double AdjustedMin = FMath::Min(-MinValue, -MaxValue);
	const double AdjustedMax = FMath::Max(-MinValue, -MaxValue);
	TBase::GetKeys(MinTime, MaxTime, AdjustedMin, AdjustedMax, OutKeyHandles);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::AddKeys(
	TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles
	)
{
	TArray<FKeyPosition> InvertedCopy;
	TArray<FKeyAttributes> InvertedAttributes;
	Private::InvertKeyPositions(InPositions, InvertedCopy);
	Private::InvertKeyAttributes(InAttributes, InvertedAttributes);
	TBase::AddKeys(InvertedCopy, InvertedAttributes, OutKeyHandles);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	TBase::GetKeyPositions(InKeys, OutKeyPositions);
	Private::InvertKeyPositions(OutKeyPositions);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::SetKeyPositions(
	TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType
	)
{
	TArray<FKeyPosition> InvertedCopy;
	Private::InvertKeyPositions(InKeyPositions, InvertedCopy);
	TBase::SetKeyPositions(InKeys, InvertedCopy, ChangeType);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	TBase::GetKeyDrawInfo(PointType, InKeyHandle, OutDrawInfo);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetValueRange(double& MinValue, double& MaxValue) const
{
	double RealMin, RealMax;
	TBase::GetValueRange(RealMin, RealMax);
	MinValue = FMath::Min(-RealMin, -RealMax);
	MaxValue = FMath::Max(-RealMin, -RealMax);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
bool TInvertedCurveModel<TBase>::Evaluate(double InTime, double& OutValue) const
{
	if (TBase::Evaluate(InTime, OutValue))
	{
		OutValue = -OutValue;
		return true;
	}
	return true;
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	TBase::GetKeyAttributes(InKeys, OutAttributes);
	Private::InvertKeyAttributes(OutAttributes);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeyAttributesIncludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	TBase::GetKeyAttributesIncludingAutoComputed(InKeys, OutAttributes);
	Private::InvertKeyAttributes(OutAttributes);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	TBase::GetKeyAttributesExcludingAutoComputed(InKeys, OutAttributes);
	Private::InvertKeyAttributes(OutAttributes);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
void TInvertedCurveModel<TBase>::SetKeyAttributes(
	TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType
	)
{
	TArray<FKeyAttributes> InvertedAttributes;
	Private::InvertKeyAttributes(InAttributes, InvertedAttributes);
	TBase::SetKeyAttributes(InKeys, InvertedAttributes, ChangeType);
}

template<typename TBase> requires std::is_base_of_v<FCurveModel, TBase>
TUniquePtr<IBufferedCurveModel> TInvertedCurveModel<TBase>::CreateBufferedCurveCopy() const
{
	// Not supported
	return nullptr;
}
}