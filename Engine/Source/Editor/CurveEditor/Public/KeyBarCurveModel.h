// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Misc/Optional.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FMenuBuilder;
struct FCurveEditorScreenSpace;
struct FCurvePointHandle;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyPosition;

/**
 * Class that provides functionality to represents keys as continuous ranges.
 */
class FKeyBarCurveModel: public FCurveModel
{
public:

	UE_API FKeyBarCurveModel();
	UE_API ~FKeyBarCurveModel();

	virtual const void* GetCurve() const override { return nullptr; }
	virtual void Modify() override {};
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override{};
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override {};
	virtual void AddKeys(TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles = nullptr) override {};
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override {};
	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override {};
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions,const EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override {};
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override {};
	virtual void GetTimeRange(double& MinTime, double& MaxTime) const override {};
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override {};
	virtual int32 GetNumKeys() const override { return 0; }
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override {};
	virtual bool Evaluate(double InTime, double& OutValue) const override { return false; }

	/*
	*  New Virtuals for Key Bar Curve Model
	*/

	virtual void BuildContextMenu(const FCurveEditor& CurveEditor, FMenuBuilder& MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint) {};

	/**
	*  Bar Ranges for the Keys
	*/
	struct FBarRange
	{
		/* Spaces have ifinite ranges, Constraints don't*/
		bool bRangeIsInfinite = true;
		TRange<double> Range;
		FName  Name;
		FLinearColor Color;
	};
	/**
	 * Find all of the ranges in this model. This should return an increasing set of ranges.
	 */
	virtual TArray<FKeyBarCurveModel::FBarRange> FindRanges() 
	{
		TArray<FKeyBarCurveModel::FBarRange> Temp; 
		return Temp;
	}
};

#undef UE_API
