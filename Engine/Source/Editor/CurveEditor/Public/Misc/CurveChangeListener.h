// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
struct FCurveModelID;

namespace UE::CurveEditor
{
/** Listens to changes made to curves and cleans the subscriptions on destruction. */
class FCurveChangeListener : public FNoncopyable
{
public:
	
	UE_API explicit FCurveChangeListener(const TSharedRef<FCurveEditor>& InCurveEditor, TArray<FCurveModelID> InCurvesToListenTo);
	UE_API ~FCurveChangeListener();

	/** Changes the curves to subscribe to. */
	UE_API void ResubscribeTo(TArray<FCurveModelID> InNewCurvesToSubscribeTo);
	
	/** @return Change listener that subscribes to all FCurveModels on InCurveEditor.*/
	UE_API static FCurveChangeListener MakeForAllCurves(const TSharedRef<FCurveEditor>& InCurveEditor);
	/** @return Change listener that subscribed only to the FCurveModels CURRENTLY selected. */
	UE_API static FCurveChangeListener MakeForSelectedCurves(const TSharedRef<FCurveEditor>& InCurveEditor);

	FSimpleMulticastDelegate& OnCurveModified() { return OnCurveModifiedDelegate; }

private:

	/** Used to unsubscribe on destruction. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	/** The curves we subscribed to */
	TArray<FCurveModelID> SubscribedToCurves;

	/** Invokes when any of the listeners is changed. */
	FSimpleMulticastDelegate OnCurveModifiedDelegate;
	
	void HandleCurveModified() const { OnCurveModifiedDelegate.Broadcast(); }

	void SubscribeToCurves();
	void UnsubscribeFromCurves();
};
}

#undef UE_API