// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

class FModifierKeysState;
class FUICommandInfo;
class FViewport;
struct FInputChord;
struct FKeyEvent;

namespace UE::CurveEditor::InputState
{

/** Initializes the shared input state processor used to track scrub chords across Slate widgets. */
void Initialize();

/** Shuts down the shared input state processor and clears any tracked scrub state. */
void Shutdown();

/**
 * Returns whether the specified input chord is currently pressed.
 * @param InChord The chord to test.
 * @param InModifierKeys The modifier key state to evaluate against the chord.
 * @return True if the chord key is pressed and its modifiers match, otherwise false.
 */
UE_API bool IsChordPressed(const FInputChord& InChord, const FModifierKeysState& InModifierKeys);

/**
 * Returns whether any active chord for the specified command is currently pressed.
 * @param InCommandInfo The command whose active chords should be tested.
 * @return True if an active chord for the command is currently pressed, otherwise false.
 */
UE_API bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo);

/**
 * Returns whether the specified key event matches any active chord for the command.
 * @param InCommandInfo The command whose active chords should be tested.
 * @param InKeyEvent The key event to evaluate against the command's active chords.
 * @return True if the key event matches an active chord for the command, otherwise false.
 */
UE_API bool IsCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo, const FKeyEvent& InKeyEvent);

/**
 * Returns whether scrub time should currently be treated as active for the specified command.
 * This includes both the live command chord state and any latched scrub state tracked across widget handoff.
 * @param InCommandInfo The scrub command whose active chords should be tested.
 * @return True if scrub time is currently active for the command, otherwise false.
 */
UE_API bool IsScrubTimeCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo);

/**
 * Explicitly updates the tracked pressed state for any active command chord that uses the specified key.
 * @param InCommandInfo The command whose active chords should be tested.
 * @param InKey The key whose pressed state should be updated.
 * @param bInPressed True if the key is now pressed, false if it is released.
 */
UE_API void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FKey& InKey, const bool bInPressed);

/**
 * Explicitly updates the tracked pressed state for any active command chord whose modifiers match the provided modifier state.
 * @param InCommandInfo The command whose active chords should be tested.
 * @param InModifierKeys The modifier key state to evaluate against the command's active chords.
 * @param bInPressed True if matching chord keys should be treated as pressed, false otherwise.
 */
UE_API void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FModifierKeysState& InModifierKeys, const bool bInPressed);

/**
 * Explicitly updates the tracked pressed state for any active command chord whose key is currently down in the specified viewport.
 * @param InCommandInfo The command whose active chords should be tested.
 * @param InViewport The viewport whose key state should be queried.
 * @param bInPressed True if matching chord keys should be treated as pressed, false otherwise.
 */
UE_API void SetCommandPressed(const TSharedPtr<FUICommandInfo>& InCommandInfo
	, const FViewport* InViewport, const bool bInPressed);

/**
 * Explicitly updates the tracked pressed state for a non-modifier key.
 * @param InKey The key whose pressed state should be updated.
 * @param bInPressed True if the key is now pressed, false if it is released.
 */
UE_API void SetKeyPressed(const FKey& InKey, const bool bInPressed);

/**
 * Explicitly updates the latched scrub-time state for a key participating in a scrub interaction.
 * @param InKey The key whose scrub-time pressed state should be updated.
 * @param bInPressed True if the key should be considered active for scrub time, false otherwise.
 */
UE_API void SetScrubTimeKeyPressed(const FKey& InKey, const bool bInPressed);

/**
 * Returns whether scrub time is currently latched as active.
 * @return True if scrub time is active, otherwise false.
 */
UE_API bool IsScrubTimeActive();

} // namespace UE::CurveEditor::InputState

#undef UE_API
