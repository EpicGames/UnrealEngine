// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CurveEditorTypes.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Internationalization/Text.h"

namespace UE::CurveEditor
{

/** Defines the merging behaviour of the paste operation. */
enum class ECurveEditorPasteMode : uint8
{
	/** Pastes the keys from the clipboard overwriting any key in destination track between the first and last pasted keys */
	OverwriteRange,
	/** Existing keys are mostly retained. Existing keys are only replaced by pasted keys that have the same X value. */
	Merge,
};

/** Enhances the paste operation */
enum class ECurveEditorPasteFlags : uint8
{
	None,

	/** Set the selection to the pasted keys. */
	SetSelection = 1 << 0,
	/** Pastes the keys from the clipboard aligning them to the nearest key to the left of the scrubber */
	Relative = 1 << 1,

	Default = SetSelection
};
ENUM_CLASS_FLAGS(ECurveEditorPasteFlags);

/** Arguments for pasting keys in FCurveEditor. */
struct FKeyPasteArgs
{
	/** Only the curve model IDs specified in this set. If empty, paste all in the clipboard. */
	TSet<FCurveModelID> CurveModelIds;
	ECurveEditorPasteMode Mode = ECurveEditorPasteMode::OverwriteRange;
	ECurveEditorPasteFlags Flags = ECurveEditorPasteFlags::Default;

	/** The name of the transaction */
	FText OverrideTransactionName = NSLOCTEXT("CurveEditor", "PasteKeys", "Paste Keys");
};
}
