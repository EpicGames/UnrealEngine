// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "Containers/Array.h"
#include "Containers/Map.h"

class FCurveModel;
class FCurveEditor;
struct FCurveModelID;
template<typename OptionalType> struct TOptional;

namespace UE::CurveEditor
{
/**
 * Finds the mid-point of every curve's passed in key handles and mirrors all points on the imaginary line going through it.
 * @param InKeysToMirror Curve to the keys on it to mirror
 * @param InCurveEditor The editor to which the keys belong
 */
template<typename TCallback, typename TMapAllocator = FDefaultSetAllocator, typename TArrayAllocator = FDefaultAllocator>
requires std::is_invocable_v<TCallback, TConstArrayView<FKeyHandle>, const FCurveModelID&, double /*Min*/, double /*Max*/, double /*Midpoint*/>
void MirrorOnMidpoint(
	const TMap<FCurveModelID, TArray<FKeyHandle, TArrayAllocator>, TMapAllocator>& InKeysToMirror, FCurveEditor& InCurveEditor, TCallback&& InProcess
	);

struct FCurveBounds
{
	/** All keys are above this. */
	double Min;
	/** All keys are below this */
	double Max;
	
	double Delta() const { return Max - Min; }
};
/** @return Find the min and max height values. */
CURVEEDITOR_API FCurveBounds FindMinMaxHeight(
	TConstArrayView<FKeyHandle> InKeys, const FCurveModel& InCurveModel
	);

/**
 * Recomputes the key position OutputValues as if the points were mirrored accross an imaginary x-axis aligned mirror edge.
 * The mirror Edge is positioned between the top and bottom edge.
 *
 * @note This function only recalculates key positions. Tangents must be set separately.
 * 
 * @param InKeysToMirror Keys located between BottomHeight and TopHeight, whose tangents will be recomputed.
 * @param InCurveId The curve the keys belong to
 * @param InBottomHeight The height of the bottom edge.
 * @param InTopHeight The height of the top edge.
 * @param InMirrorHeight The height of the imaginary x-axis-aligned mirror edge across which tangents are mirrored.
 * @param InCurveEditor Reference to the curve editor used to set key attributes.
 */
CURVEEDITOR_API void MirrorKeyPositions(
	TConstArrayView<FKeyHandle> InKeysToMirror, const FCurveModelID& InCurveId,
	double InBottomHeight, double InTopHeight, double InMirrorHeight,
	FCurveEditor& InCurveEditor
	);
	
/**
 * Recomputes the tangents of specified keys as if the points were mirrored across an imaginary x-axis aligned mirror edge.
 * The mirror edge is positioned between the top and bottom edges.
 * 
 * @note This function only recalculates tangents. Key positions must be set separately.
 * 
 * @param InKeysToMirror Keys located between BottomHeight and TopHeight, whose tangents will be recomputed.
 * @param InCurveId The curve the keys belong to
 * @param InBottomHeight The height of the bottom edge.
 * @param InTopHeight The height of the top edge.
 * @param InMirrorHeight The height of the imaginary x-axis-aligned mirror edge across which tangents are mirrored.
 * @param InCurveEditor Reference to the curve editor used to set key attributes.
 */
CURVEEDITOR_API void MirrorTangents(
	TConstArrayView<FKeyHandle> InKeysToMirror, const FCurveModelID& InCurveId,
	double InBottomHeight, double InTopHeight, double InMirrorHeight,
	FCurveEditor& InCurveEditor
	);
}

namespace UE::CurveEditor
{
template<typename TCallback, typename TMapAllocator, typename TArrayAllocator>
requires std::is_invocable_v<TCallback, TConstArrayView<FKeyHandle>, const FCurveModelID&, double /*Min*/, double /*Max*/, double /*Midpoint*/>
void MirrorOnMidpoint(
	const TMap<FCurveModelID, TArray<FKeyHandle, TArrayAllocator>, TMapAllocator>& InKeysToMirror, FCurveEditor& InCurveEditor, TCallback&& InProcess
)	
{
	for (const TPair<FCurveModelID, TArray<FKeyHandle>>& Pair : InKeysToMirror)
	{
		if (const FCurveModel* Model = InCurveEditor.FindCurve(Pair.Key))
		{
			const auto[Min, Max] = FindMinMaxHeight(Pair.Value, *Model);
			const double Midpoint = Min + (Max - Min) * 0.5;
			InProcess(Pair.Value, Pair.Key, Min, Max, Midpoint);
		}
	}
}
}

