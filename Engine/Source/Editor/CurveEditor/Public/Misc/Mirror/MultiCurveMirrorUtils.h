// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "UniformMirrorSolver.h"

namespace UE::CurveEditor
{
struct FMirrorableTangentInfo
{
	/** Keys that can be mirrored, i.e. have RCTM_User or RCTM_Break mode. */
	TArray<FKeyHandle> MirrorableKeys;
	/** The attributes corresponding to each of MirrorableKeys. */
	TArray<FKeyAttributes> InitialAttributes;
	/**
	 * The tangents corresponding to each of MirrorableKeys.
	 * X is arrive tangent, Y is leave tangent. Copied values of the corresponding InitialAttributes indices. If there is no leave tangent, Y is 0.
	 */
	TArray<FVector2D> Tangents;
	/** The FKeyPosition::OutputValue corresponding to each of MirrableKeys. */
	TArray<double> KeyHeights;

	bool HasData() const { return !MirrorableKeys.IsEmpty(); }
	operator bool() const { return HasData(); }
};

/** @return Information you can use to FCurveTangentMirrorData. */
CURVEEDITOR_API FMirrorableTangentInfo FilterMirrorableTangents(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys
	);
	
/** Holds data about mirroring keys in a single curve. */
struct FCurveTangentMirrorData
{
	/**
	 * This will mirror the tangents of KeyHandles. The X component corresponds to ArriveTangent and Y to LeaveTangent.
	 * TMirrorSolver::InitialValues and TMirrorSolver::InitialKeyHeights correspond to KeyHandles.
	 */
	const TUniformMirrorSolver<FVector2D> TangentSolver;
	/** The keys to interpolate. Only tangents that are user specified are referenced here. */
	const TArray<FKeyHandle> KeyHandles;
	/** Same length as KeyHandles.*/
	TArray<FKeyAttributes> AttributesToSet;

	explicit FCurveTangentMirrorData(TUniformMirrorSolver<FVector2D> InTangentSolver, TArray<FKeyHandle> InKeys, TArray<FKeyAttributes> InInitialAttributes)
		: TangentSolver(MoveTemp(InTangentSolver))
		, KeyHandles(MoveTemp(InKeys))
		, AttributesToSet(MoveTemp(InInitialAttributes))
	{
		check(TangentSolver.NumValues() == KeyHandles.Num() && KeyHandles.Num() == AttributesToSet.Num());
		ensureMsgf(TangentSolver.NumValues() != 0, TEXT("Pointless construction"));
	}
	
	explicit FCurveTangentMirrorData(
		FMirrorableTangentInfo InInfo, double InStartHeight, double InMirrorMidpoint, const FVector2D& InMidpointOffset = FVector2D::ZeroVector
		)
		: FCurveTangentMirrorData(
			TUniformMirrorSolver<FVector2D>(InStartHeight, InMirrorMidpoint, MoveTemp(InInfo.Tangents), MoveTemp(InInfo.KeyHeights), InMidpointOffset),
			MoveTemp(InInfo.MirrorableKeys), MoveTemp(InInfo.InitialAttributes)
		)
	{}
};

	
/** Mirrors the tangents stores in InCurveData. */
void RecomputeMirroringParallel(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, FCurveTangentMirrorData& InCurveData, double InMirrorValue
	);
/**
 * Mirrors the tangents stores in InCurveData.
 * This version accepts a callback for post-processing, which is useful for e.g. applying a falloff to the interpolated values.
 */
template<typename TCallback> requires std::is_invocable_r_v<FVector2D, TCallback, int32, const FVector2D&>
void RecomputeMirroringParallel(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, FCurveTangentMirrorData& InCurveData, double InMirrorValue,
	TCallback&& InPostProcessTangents
	);
}

namespace UE::CurveEditor
{
inline void RecomputeMirroringParallel(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, FCurveTangentMirrorData& InCurveData, double InMirrorValue
	)
{
	RecomputeMirroringParallel(InCurveEditor, InCurveId, InCurveData, InMirrorValue,
		[](int32, const FVector2D& Interpolated)
		{
			return Interpolated;
		});
}

/** Mirrors the tangents stores in InCurveData. */
template<typename TCallback> requires std::is_invocable_r_v<FVector2D, TCallback, int32/*KeyIndex*/, const FVector2D&/*InterpolatedTangents*/>
void RecomputeMirroringParallel(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, FCurveTangentMirrorData& InCurveData, double InMirrorValue,
	TCallback&& InPostProcessTangents
	)
{
	FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	if (!CurveModel)
	{
		return;
	}

	InCurveData.TangentSolver.ComputeMirroringParallel(InMirrorValue, [&InCurveData, &InPostProcessTangents](int32 KeyIndex, const FVector2D& InterpolatedTangents)
	{
		// KeyIndex is an index to the Tangents and Heights arrays we passed to TangentSolver's constructor.
		// KeysToMirror was constructed in such a way that it coincides with those arrays. 
		FKeyAttributes& Attributes = InCurveData.AttributesToSet[KeyIndex];

		const FVector2D Tangents = InPostProcessTangents(KeyIndex, InterpolatedTangents);
		if (Attributes.HasArriveTangent())
		{
			Attributes.SetArriveTangent(Tangents.X);
		}
		if (Attributes.HasLeaveTangent())
		{
			Attributes.SetLeaveTangent(Tangents.Y);
		}
	});

	CurveModel->SetKeyAttributes(InCurveData.KeyHandles, InCurveData.AttributesToSet);
}
}