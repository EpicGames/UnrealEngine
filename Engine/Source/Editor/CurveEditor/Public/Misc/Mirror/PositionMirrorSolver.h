// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "UniformMirrorSolver.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
template<typename OptionalType> struct TOptional;

namespace UE::CurveEditor
{
/**
 * This class facilitates the movement of a dragged edge and recalculates the key heights for all specified keys.
 * The heights are linearly interpolated, ensuring that if the dragged edge crosses to the opposite side of the mirror edge,
 * the keys will flip to the other side.
 * 
 * Inputs:
 *  - Height of the dragged edge aligned to the x-axis
 *  - Height of the x-axis-aligned mirror edge
 *  - Keys originally located between the dragged and mirror edges
 *
 * Example:
 * Given a grid with a single key at a height 100, moving the top edge:
 *  - An equal distance past the bottom edge will result in a height of -100 for the key.
 *  - Halfway between the starting position and the bottom edge will result in a height of 50 for the key.
 * 
 * @note This class only recalculates key positions. To adjust the tangents of the keys use e.g. FTangentMirrorSolver.
 */
class FPositionMirrorSolver
{
public:
	
	/**
	 * InStartY and InMiddlePointY define two imaginary, x-axis aligned lines.
	 * All keys added must be between or on these two lines for the mirroring to work as expected.
	 * 
	 * @param InCurveEditor Used to get and set tangent values
	 * @param InStartY The height of the line that is moved. It determines how the tangents are interpolated.
	 * @param InMiddlePointY The height of the line on which the points are moved.
	 */
	UE_API explicit FPositionMirrorSolver(
		FCurveEditor& InCurveEditor UE_LIFETIMEBOUND,
		double InStartY, double InMiddlePointY
		);

	/**
	 * Adds key positions that are to be mirrored.
	 *
	 * InStartY and InMiddlePointY define two imaginary, x-axis aligned lines.
	 * All keys must be between or on these two lines for the mirroring to work as expected.
	 * 
	 * @param InCurveId The curve that all keys lie on
	 * @param InKeys The keys whose tangents to recompute
	 * @param InPositions The positions of the keys, if you have them already (optimization). If not, pass in empty and the key positions are determined.
	 * @return Whether any of the keys can be interpolated (only user specified tangents can be interpolated).
	 */
	UE_API bool AddKeyPositions(const FCurveModelID& InCurveId, TArray<FKeyHandle> InKeys, TArray<FKeyPosition> InPositions = {});
	
	/** Recomputes the tangents angles and updates the key attributes. */
	UE_API void OnMoveEdge(double InDraggedEdgeHeight);
	
private:
	
	/** Used to set tangents angles. */
	FCurveEditor& CurveEditor;

	/** Height of the dragged line. */
	const double StartY;
	/** Height of the mirroring line. */
	const double MiddlePointY;
	
	struct FCachedCurveData
	{
		const TUniformMirrorSolver<double> TangentSolver;
		
		const TArray<FKeyHandle> KeyHandles;
		TArray<FKeyPosition> PositionsToSet;

		explicit FCachedCurveData(TUniformMirrorSolver<double> InTangentSolver, TArray<FKeyHandle> InKeys, TArray<FKeyPosition> InPositions)
			: TangentSolver(MoveTemp(InTangentSolver))
			, KeyHandles(MoveTemp(InKeys))
			, PositionsToSet(MoveTemp(InPositions))
		{}
	};
	/** This memory is reuse across OnMoveEdge to hold the updated tangents. Caching the memory speeds up the recomputation. */
	TMap<FCurveModelID, FCachedCurveData> AllCurveData;
};
}

#undef UE_API
