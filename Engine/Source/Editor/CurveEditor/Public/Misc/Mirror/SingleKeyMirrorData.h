// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "MultiCurveMirrorUtils.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

namespace UE::CurveEditor
{
enum class ESide { Left, Right };

/**
 * Information about a center key and its neighbor's heights.
 * @see FTangentSingleKeyFlattener
 */
struct FKeyNeighborData
{
	/** Height of the key in the center. */
	double CenterInitialHeight;
	
	/** Height of the key to the left of the center key. */
	TOptional<double> LeftKeyHeight;
	/** Height of the key to the right of the center key. */
	TOptional<double> RightKeyHeight;
	
	/** @return The height of the neighbor specified. */
	TOptional<double> Choose(ESide InSide) const { return InSide == ESide::Left ? LeftKeyHeight : RightKeyHeight; }
	
	/** @return Whether the center key has any neighbors. */
	bool IsEmpty() const { return !LeftKeyHeight && !RightKeyHeight; }
};

/** 
 * Information used to flatten tangent of a single key. 
 * @see FTangentSingleKeyFlattener
 * */
struct FSingleKeyMirrorData
{
	/** The center key, which is being moved. */
	FKeyHandle CenterKey;
	/** Info about the key's neighbor hood. Used to compute the mirror value. */
	FKeyNeighborData NeighborData;
	
	/** The data used to interpolate the key */
	FCurveTangentMirrorData TangentMirrorData;
	
	/** Indicates the neighbor of the interpolated key with which the base interpolation distance initialized */
	ESide AnchorKeySide;
	/** The distance the center key had to the anchor key, when the data was initialized */
	double InitialDistanceToAnchorPoint;
	
	/** @return The key whose tangent is being flattened */
	FKeyHandle GetKeyHandle() const { return CenterKey; }

	explicit FSingleKeyMirrorData(
		FKeyHandle InCenterKey,
		const FKeyNeighborData& InNeighborData, FCurveTangentMirrorData InTangentMirrorData, 
		ESide InAnchorPoint, double InInitialDistanceToAnchorPoint
		)
		: CenterKey(InCenterKey)
		, NeighborData(InNeighborData)
		, TangentMirrorData(MoveTemp(InTangentMirrorData))
		, AnchorKeySide(InAnchorPoint)
		, InitialDistanceToAnchorPoint(InInitialDistanceToAnchorPoint)
	{}
};
}

#undef UE_API
