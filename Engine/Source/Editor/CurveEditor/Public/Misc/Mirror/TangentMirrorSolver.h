// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "MultiCurveMirrorUtils.h"

class FCurveEditor;
template<typename OptionalType> struct TOptional;

namespace UE::CurveEditor
{
/**
 * Given keys that are between an imaginary, x-axis aligned mirror edge and dragged edge, this class facilitates recalculates the tangents for all
 * specified keys. The tangents are linearly interpolated, ensuring that if the dragged edge crosses to the opposite side of the mirror edge,
 * the tangents will flip, resulting in mirrored curves.
 * 
 * Inputs:
 *  - Height of the dragged edge aligned to the x-axis
 *  - Height of the x-axis-aligned mirror edge
 *  - Keys originally located between the dragged and mirror edges
 *
 * Example: Given a grid with a single key at a 45-degree angle, moving the top edge:
 *  - An equal distance past the bottom edge will result in a tangent of -45 degrees for the key.
 *  - Halfway between the starting position and the bottom edge will result in a tangent of 22.5 degrees for the key.
 * 
 * @note This class only recalculates tangents. To adjust the positions of the keys use e.g. FPositionMirrorSolver.
 */
class FTangentMirrorSolver
{
public:
	
	/** Height of the dragged line. */
	const double StartY;
	/** Height of the mirroring line. */
	const double MiddlePointY;
	
	/** This memory is reused across OnMoveEdge to hold the updated tangents. Caching the memory speeds up the re-computation. */
	TMap<FCurveModelID, FCurveTangentMirrorData> CurveData;
	
	/**
	 * InStartY and InMiddlePointY define two imaginary, x-axis aligned lines.
	 * All keys added must be between or on these two lines for the mirroring to work as expected.
	 * 
	 * @param InStartY The height of the line that is moved. It determines how the tangents are interpolated.
	 * @param InMiddlePointY The height of the line on which the points are moved.
	 */
	explicit FTangentMirrorSolver(
		double InStartY, double InMiddlePointY
		);

	/**
	 * Adds tangents that are be mirrored.
	 *
	 * InStartY and InMiddlePointY define two imaginary, x-axis aligned lines.
	 * All keys must be between or on these two lines for the mirroring to work as expected.
	 * 
	 * @param InCurveEditor Used to get and set tangent values
	 * @param InCurveId The curve that all keys lie on
	 * @param InKeys The keys whose tangents to recompute
	 * @return Whether any of the keys can be interpolated (only user specified tangents can be interpolated).
	 */
	bool AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys);
	
	/**
	 * Version that allows you to do additional processing for the tangents.
	 * 
	 * For example,  you may want to compute falloff values for each key. In that case, you probably want to manually iterate CurveData and use the
	 * version of RecomputeMirroringParallel that allows you to interpolate tangent values further.
	 */
	template<typename TProcessCallback> requires std::is_invocable_v<TProcessCallback, const FMirrorableTangentInfo& /*TangentInfo*/>
	bool AddTangents(
		const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys, TProcessCallback&& InProcessTangentInfo
		);
	
	/** Recomputes the tangents angles and updates the key attributes. */
	void OnMoveEdge(const FCurveEditor& InCurveEditor, double InDraggedEdgeHeight);
};
}

namespace UE::CurveEditor
{
inline FTangentMirrorSolver::FTangentMirrorSolver(double InStartY, double InMiddlePointY)
	: StartY(InStartY)
	, MiddlePointY(InMiddlePointY)
{}

inline bool FTangentMirrorSolver::AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys)
{
	return AddTangents(InCurveEditor, InCurveId, InKeys, [](const FMirrorableTangentInfo&){});
}
	
template <typename TProcessCallback> requires std::is_invocable_v<TProcessCallback, const FMirrorableTangentInfo&>
bool FTangentMirrorSolver::AddTangents(
	const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys, TProcessCallback&& InProcessTangentInfo
	)
{
	FMirrorableTangentInfo TangentInfo = FilterMirrorableTangents(InCurveEditor, InCurveId, InKeys);
	if (TangentInfo)
	{
		InProcessTangentInfo(TangentInfo);
		CurveData.Add(
			InCurveId,
			FCurveTangentMirrorData(MoveTemp(TangentInfo), StartY, MiddlePointY)
		);
		return true;
	}
	return false;
}
	
inline void FTangentMirrorSolver::OnMoveEdge(const FCurveEditor& InCurveEditor, double InDraggedEdgeHeight)
{
	for (TPair<FCurveModelID, FCurveTangentMirrorData>& Pair : CurveData)
	{
		RecomputeMirroringParallel(InCurveEditor, Pair.Key, Pair.Value, InDraggedEdgeHeight);
	}
}
}