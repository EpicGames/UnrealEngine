// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "MultiCurveMirrorUtils.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

namespace UE::CurveEditor
{
/**
 * Given some keys, this class squishes the curves based on how much the selection is squished vertically.
 * The squishing of the curves is achieved by interpolating the tangents to 0.
 * 
 * This is useful if you have a selection of keys, apply some operation that will move the keys, and want to flatten the curves.
 * Example use case is e.g. tweening.
 */
class FTangentSelectionFlattener
{
public:

	FTangentSelectionFlattener() = default;
	explicit FTangentSelectionFlattener(const FCurveEditor& InCurveEditor) { ResetFromSelection(InCurveEditor); }

	/** Inits the tangent data from the curve editor's selection. */
	UE_API bool ResetFromSelection(const FCurveEditor& InCurveEditor);

	/** @return Whether any interpolatable tangents were added (only tangents that are RCTM_User or RCTM_Break). */
	UE_API bool AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys);

	/**
	 * Call in response to the keys having moved. Recomputes user specified tangents and flattens them.
	 * @param InCurveEditor Used to update key positions
	 * @param bTopHasCrossedBottomEdge If since this was initialized, the top edge has moved across the bottom edge.
	 */
	UE_API void ComputeMirroringParallel(const FCurveEditor& InCurveEditor, bool bTopHasCrossedBottomEdge = false);

private:
	
	TMap<FCurveModelID, FCurveTangentMirrorData> CachedCurveData;
};
}

#undef UE_API
