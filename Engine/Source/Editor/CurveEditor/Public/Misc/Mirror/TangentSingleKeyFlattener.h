// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "SingleKeyMirrorData.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

namespace UE::CurveEditor
{
/**
 * Flattens the tangents of a single key per curve whose tangent mode is user-specified (RCTM_User or RCTM_Break).
 * The degree of flattening is proportional to how close the key moves toward a neighboring key vertically:
 * reaching the neighbor results in fully flat (zero) tangents and passing the neighbor causes it to flip.
 *
 * === Algorithm for a center key k ===
 * 
 * AddTangents (call once to initialize):
 *   1. Find the left and right neighbors of k.
 *   2. Pick an anchor neighbor: prefer left, fall back to right.
 *   3. Record initial_distance = |k.Y - anchor.Y|.
 *   (No-ops if k has no neighbors, or if k and anchor share the same Y value.)
 *   
 * ComputeMirroringParallel (call each time k moves to recompute tangents):
 *   1. Compute a blend height; a value that shrinks toward 0 as k approaches either neighbor:
 *      - Moving toward anchor:   blend = |k.Y - anchor.Y|
 *      - Moving toward opposite: blend = initial_distance * (1 - |current_distance_to_anchor - initial_distance| / |k_initial.Y - opposite.Y|)
 *   2. If k has crossed past the target neighbor, flip the tangent direction.
 *   3. Scale the tangents linearly by blend via RecomputeMirroringParallel.
 *
 * @see FTangentSelectionFlattener For flattening consecutive keys.
 */
class FTangentSingleKeyFlattener
{
public:

	FTangentSingleKeyFlattener() = default;

	/** Inits the tangent data from the curve editor's selection. */
	UE_API bool ResetFromSelection(const FCurveEditor& InCurveEditor);

	/** 
	 * Searches the range for exactly 1 key that has user specified tangents and adds the key as blend info.
	 * If there are more than 1 such keys, nothing happens.
	 * @return Whether any data was added. 
	 */
	UE_API bool AddTangents(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys);
	
	/**
	 * Recomputes the keys's tangents based on where on their vertical distance to the specified neighboring keys.
	 * @param InCurveEditor Used to update key positions
	 * @param InKeyToMirrorTo The neighboring key to use as pivot for mirror computation.
	 */
	UE_API void ComputeMirroringParallel(const FCurveEditor& InCurveEditor, ESide InKeyToMirrorTo);

private:
	
	TMap<FCurveModelID, FSingleKeyMirrorData> CachedCurveData;
};
}

#undef UE_API
