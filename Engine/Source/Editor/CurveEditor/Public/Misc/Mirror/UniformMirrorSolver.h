// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"

#include <type_traits>

namespace UE::CurveEditor
{
/**
 * This class computes linear interpolation for values involving the movement of edges in a rectangular selection of keys. While best used for
 * tangent angles, it could be applied to any key attributes that can be interpolated linearly.
 *
 * For example:
 * - When the top edge is moved downward, the interpolated values decrease linearly to 0 as the edge approaches 
 *   the bottom edge. If the top edge moves past the bottom edge (the "midpoint"), the values invert, 
 *   transitioning past zero into the opposite sign.
 * - Similarly, when the bottom edge is moved upward toward the top edge, the same interpolation and inversion logic applies.
 * 
 * This class handles the described interpolation logic in a generic manner. It is agnostic to the type or source 
 * of the values being interpolated and how the computed values are applied. Its sole responsibility is to perform 
 * linear interpolation based on the edge movement.
 *
 * @param TNumeric Must support operator+(TNumeric, TNumeric) itself and operator*(TNumeric,double). E.g. float, double, FVector2D, FVector, etc.
 */
template<typename TNumeric> 
class TUniformMirrorSolver
{
public:
	
	explicit TUniformMirrorSolver(
		double StartY, double MiddlePointY, TArray<TNumeric> InInterpolatedValues, TArray<double> InKeyHeights, TNumeric InMiddlePointBaseValue = TNumeric(0)
		)
		: StartY(StartY)
		, MiddlePointY(MiddlePointY)
		, InitialValues(MoveTemp(InInterpolatedValues))
		, InitialKeyHeights(MoveTemp(InKeyHeights))
		, MiddlePointBaseValue(InMiddlePointBaseValue)
	{
		check(InitialValues.Num() == InitialKeyHeights.Num());
	}

	/**
	 * Recomputes the values in response to the edge being moved.
	 * @param InNewHeight The new height of the moved edge
	 * @param InProcessTangent Processes the new tangent angles (in radians).
	 */
	template<typename TCallback> requires std::is_invocable_v<TCallback, int32 /*InIndex*/, const TNumeric& /*NewValue*/>
	void ComputeMirroringParallel(double InNewHeight, TCallback&& InProcessTangent) const
	{
		const int32 Num = InitialValues.Num(); 
		const double Alpha = (InNewHeight - MiddlePointY) / (StartY - MiddlePointY);
		ParallelFor(Num, [this, Alpha, &InProcessTangent](int32 Index)
		{
			const TNumeric InitialTangent = InitialValues[Index];
			const TNumeric NewValue = InitialTangent * Alpha + MiddlePointBaseValue;
			InProcessTangent(Index, NewValue);
		});
	}

	int32 NumValues() const { return InitialValues.Num(); }

	/** The Y component at which the drag started. */
	const double StartY;
	/** The Y component at which the tangents reach 0 slope. */
	const double MiddlePointY;
	
	/** The initial tangent angles in radians. Indices coincide with InitialKeyHeights. */
	const TArray<TNumeric> InitialValues;
	/** The initial heights of all keys. Indices coincide with InitialKeyHeights. */
	const TArray<double> InitialKeyHeights;
	/** The value that should be at MiddlePointY. */
	const TNumeric MiddlePointBaseValue;
};
}

