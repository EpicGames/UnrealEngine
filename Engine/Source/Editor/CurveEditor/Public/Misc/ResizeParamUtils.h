// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/Platform.h"
#include "Styling/AppStyle.h"

namespace UE::CurveEditor
{
/**
 * Looks up the CurveEditor.AllowClipping.[UISpecifier] and CurveEditor.ClipPriority.[UISpecifier] from the given style.
 * @return Resizing params to use for the toolbar item.
 */
FMenuEntryResizeParams MakeResizeParams(FStringView UISpecifier, const ISlateStyle& Style = FAppStyle::Get());
}

namespace UE::CurveEditor
{
inline FMenuEntryResizeParams MakeResizeParams(FStringView UISpecifier, const ISlateStyle& Style)
{
	// The point of this variable is to check whether UISpecifier exists in the style. We assume that nobody will ever use this ridiculous value...
	constexpr float UnsetValue = -4200000.f; 
	
	const FString AllowClippingString = FString::Printf(TEXT("CurveEditor.AllowClipping.%s"), UISpecifier.GetData());
	const float AllowClipping_Style = Style.GetFloat(*AllowClippingString, nullptr, UnsetValue);
	const bool bAllowClipping = AllowClipping_Style == UnsetValue
		? FMenuEntryResizeParams::DefaultAllowClipping
		: AllowClipping_Style >= 1.f; // There's no bools in styles, so we'll treat non-zero as true.

	const FString ClipPriorityString = FString::Printf(TEXT("CurveEditor.ClipPriority.%s"), UISpecifier.GetData());
	const float ClipPriority_Style =  Style.GetFloat(*ClipPriorityString, nullptr, UnsetValue);
	const float ClipPriority = ClipPriority_Style == UnsetValue ? FMenuEntryResizeParams::DefaultClippingPriority : ClipPriority_Style;
	
	// If any of these trigger, either you've misspelled the identifier or you didn't register it.
	// Add Set("CurveEditor.AllowClipping.[YourSpecifier]", YourPriority) to FStartshipEditorStyle.
	ensureAlwaysMsgf(AllowClipping_Style != UnsetValue, TEXT("Specifier %s is not mapped in the style!"), UISpecifier.GetData());
	// Add Set("CurveEditor.ClipPriority.[YourSpecifier]", YourPriority) to FStartshipEditorStyle.
	ensureAlwaysMsgf(ClipPriority_Style != UnsetValue, TEXT("Specifier %s is not mapped in the style!"), UISpecifier.GetData());
	
	return { .ClippingPriority = ClipPriority, .AllowClipping =  bAllowClipping };
}
}