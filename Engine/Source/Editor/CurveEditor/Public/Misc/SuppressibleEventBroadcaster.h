// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::CurveEditor
{
/**
 * Wraps a simple multicast delegate with a counter.
 *
 * While the counter is positive, the event cannot be broadcast.
 * Once the counter reaches 0, the event is broadcast.
 *
 * This is a reusable optimization that is useful to change-type events.
 */
class FSuppressibleEventBroadcaster
{
public:

	void IncrementSuppressionCount()
	{
		++SuppressionCount;
	}

	void DecrementSuppressionCount()
	{
		if (!ensure(SuppressionCount > 0))
		{
			return;
		}
		
		--SuppressionCount;

		if (SuppressionCount == 0 && bWasEventSuppressed)
		{
			bWasEventSuppressed = false;
			Broadcast();
		}
	}

	void Broadcast()
	{
		if (SuppressionCount == 0)
		{
			bWasEventSuppressed = false;
			Event.Broadcast();
		}
		else
		{
			bWasEventSuppressed = true;
		}
	}

	/** @return The event that is broadcast. This exists only for convenience: by design, you're NOT supposed to call Broadcast on this directly. */
	FSimpleMulticastDelegate& OnChanged() { return Event; }
	
private:

	/** When 0, the event can be broadcast. */
	int32 SuppressionCount = 0;

	/**
	 * This is set to true when Broadcast is called while SuppressionCount > 0.
	 * If true, then DecrementSuppressionCount will broadcast the event when count reaches 0.
	 */
	bool bWasEventSuppressed = false;

	/** The event to broadcast */
	FSimpleMulticastDelegate Event;
};
}


