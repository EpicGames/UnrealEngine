// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Change.h"
#include "Templates/SharedPointer.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

namespace UE::CurveEditor
{
/** Base class for actions that support undo / redo in the curve editor. */
class FCurveEditorCommandChange : public FCommandChange
{
public:

	explicit FCurveEditorCommandChange(TSharedPtr<FCurveEditor> InCurveEditor)
		: WeakCurveEditor(MoveTemp(InCurveEditor))
	{}

	//~ Begin FChange Interface
	UE_API virtual bool HasExpired(UObject* InObject) const override;
	UE_API virtual FString ToString() const override;
	//~ End FChange Interface

protected:

	/** @return The curve editor associated with Object. */
	TSharedPtr<FCurveEditor> GetCurveEditor() const;

private:

	/** The curve editor this operation was performed on. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;
};
}

#undef UE_API
