// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurvesSnapshotBuilder.h"
#include "Data/GenericCurveChangeData.h"
#include "ECurveChangeFlags.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;

namespace UE::CurveEditor
{
class IMultiCurveChangeVisitor;
struct FGenericCurveChangeData;

/** @return FGenericCurveChangeData containing what has changed for the data saved in InSnapshot. */
CURVEEDITOR_API FGenericCurveChangeData ComputeDiff(
	const FCurveEditor& InCurveEditor, const ECurveChangeFlags InOperations, const FCurvesSnapshot& InSnapshot
	);

/** Computes what has changed for the data saved in InSnapshot and passes those changes to IMultiCurveChangeVisitor to perform any processing. */
CURVEEDITOR_API	void ProcessChanges(
	const FCurveEditor& InCurveEditor, const ECurveChangeFlags InOperations, const FCurvesSnapshot& InSnapshot,
	IMultiCurveChangeVisitor& InChangeVisitor
	);

/**
 * Utility that holds a snapshot of some curve data for the purposes of later comparing changes made to those curves.
 * 
 * This is useful for actions that take multiple frames, e.g. the user dragging keys.
 * Start by constructing FCurvesSnapshotBuilder instance, adding the curves you want to diff later, and when the operation is done call ComputeDiff to
 * compute the change.
 *
 * Note: This class introduces some overhead by temporarily saving the curve state.
 * If the change is a one-off (i.e. not interactive) you usually know exactly what will change, so you can consider building the change directly.
 * It's still fine to use this class in that case as it simplifies the code: the overhead is usually acceptable.
 * @see FScopedKeyChange For some profiling data.
 */
class FCurveChangeDiff
{
public:

	explicit FCurveChangeDiff(TWeakPtr<FCurveEditor> InCurveEditor, FCurvesSnapshot InSnapshot, ECurveChangeFlags InOps = ECurveChangeFlags::All)
		: WeakCurveEditor(MoveTemp(InCurveEditor))
		, Operations(InOps)
		, Snapshot(MoveTemp(InSnapshot))
	{}
	
	FCurveChangeDiff(FCurvesSnapshotBuilder InBuilder)
		: FCurveChangeDiff(InBuilder.WeakCurveEditor, MoveTemp(InBuilder.Snapshot), InBuilder.CapturedDataFlags)
	{}

	/** Computes what has changed on the given curves. */
	CURVEEDITOR_API FGenericCurveChangeData ComputeDiff() const;

	/** Passes all changes to the given instance. */
	CURVEEDITOR_API void ProcessDiffs(IMultiCurveChangeVisitor& InChangeBuilder) const;

	/** The snapshot against which is diffed. */
	const FCurvesSnapshot& GetSnapshot() const { return Snapshot; }

private:

	/** Needed to look up curves and compare state. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;
	/** The operations that we should diff with. */
	const ECurveChangeFlags Operations;

	/** Data that ComputeDiff diffs against. */
	const FCurvesSnapshot Snapshot;
};
}
