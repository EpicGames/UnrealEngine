// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Data/CurvesSnapshot.h"
#include "ECurveChangeFlags.h"

class FCurveEditor;

namespace UE::CurveEditor
{
/**
 * Utility for selectively capturing state of curves based on some ECurveChangeFlags.
 * Intended for constructing either a FCurvesSnapshot or FCurveChangeDiff.
 */
struct FCurvesSnapshotBuilder
{
	/** Needed to look up curves and compare state. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;
	/** The operations that we should diff with. */
	const ECurveChangeFlags CapturedDataFlags;

	/** Data that ComputeDiff diffs against. */
	FCurvesSnapshot Snapshot;

	explicit FCurvesSnapshotBuilder(
		TWeakPtr<FCurveEditor> InCurveEditor, ECurveChangeFlags InOps = ECurveChangeFlags::All
		)
		: WeakCurveEditor(MoveTemp(InCurveEditor))
		, CapturedDataFlags(InOps)
	{}

	/** Util constructor for tracking a single curve. */
	explicit FCurvesSnapshotBuilder(
		TWeakPtr<FCurveEditor> InCurveEditor, const FCurveModelID& InCurveId, ECurveChangeFlags InOps = ECurveChangeFlags::All
		)
		: FCurvesSnapshotBuilder(MoveTemp(InCurveEditor), InOps)
	{
		TrackCurve(InCurveId);
	}
	/** Util constructor for tracking a container of curves (TSet, TMap, TArray, TArrayView, TConstArrayView) */
	template<typename TRangeType>
	explicit FCurvesSnapshotBuilder(
		TWeakPtr<FCurveEditor> InCurveEditor, const TRangeType& InRangeType, ECurveChangeFlags InOps = ECurveChangeFlags::All
		)
		: FCurvesSnapshotBuilder(MoveTemp(InCurveEditor), InOps)
	{
		TrackCurves(InRangeType);
	}

	/** Tracks all keys on InCurveModel. */
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackCurve(const FCurveModelID& InCurveId);
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackCurves(TConstArrayView<FCurveModelID> InCurves);
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackCurves(const TSet<FCurveModelID>& InCurves);
	template<typename T>
	FCurvesSnapshotBuilder& TrackCurves(const TMap<FCurveModelID, T>& InCurveMapping);
	
	/** Tracks only the specified keys. Incompatible with EKeyChangeOperationFlags::AddKeys. */
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackCurve(const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> Keys);
	
	/** Tracks all curves that are selected: adds only their selected keys. Incompatible with EKeyChangeOperationFlags::AddKeys. */
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackSelectedKeys();
	/** Tracks all curves that are selected: adds all keys of those curves. */
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackSelectedCurves();

	/** Tracks all curves. */
	CURVEEDITOR_API FCurvesSnapshotBuilder& TrackAllCurves();
};

template <typename T>
FCurvesSnapshotBuilder& FCurvesSnapshotBuilder::TrackCurves(const TMap<FCurveModelID, T>& InCurveMapping)
{
	for (const TPair<FCurveModelID, T>& Pair : InCurveMapping)
	{
		TrackCurve(Pair.Key);
	}
	return *this;
}
}
