// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CurveEditorTypes.h"
#include "Curves/RealCurve.h"

namespace UE::CurveEditor
{
/** Stores data for restoring the curve attributes of a single curve. */
struct FCurveAttributeChangeData_PerCurve
{
	/** The key attributes before the change. Set attributes to this on undo.  */
	FCurveAttributes BeforeChange;
	/** The key attributes after the change. Set attributes to this on undo. */
	FCurveAttributes AfterChange;

	bool HasChanges() const { return BeforeChange != AfterChange; }
	operator bool() const { return HasChanges(); }

	friend bool operator==(const FCurveAttributeChangeData_PerCurve&, const FCurveAttributeChangeData_PerCurve&) = default;
	friend bool operator!=(const FCurveAttributeChangeData_PerCurve&, const FCurveAttributeChangeData_PerCurve&) = default;
};

/** Stores data for restoring curves attributes of multiple curves. */
struct FCurveAttributeChangeData
{
	TMap<FCurveModelID, FCurveAttributeChangeData_PerCurve> ChangeData;

	bool HasChanges() const { return !ChangeData.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const { return ChangeData.GetAllocatedSize(); }
};
}
