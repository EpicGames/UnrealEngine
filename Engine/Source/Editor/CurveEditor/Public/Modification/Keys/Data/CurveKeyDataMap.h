// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDataAbstraction.h"
#include "Algo/Accumulate.h"
#include "Misc/Optional.h"

namespace UE::CurveEditor
{
/** Holds data for restoring keys on a curve. */
struct FCurveKeyData
{
	/**
	 * The tracked key handles. Contains all key handles if EKeyChangeOperationFlags::AddKeys.
	 * Always set.
	 */
	TArray<FKeyHandle> KeyHandles;

	/**
	 * The positions of Handles. Each index matches that of Handles.
	 * Set only for EKeyChangeOperationFlags::MoveKeys.
	 *
	 * Either same length as Handles or 0 length.
	 */
	TArray<FKeyPosition> KeyPositions;

	/**
	 * The attributes of Handles. Each index matches that of Handles.
	 * Set only for EKeyChangeOperationFlags::KeyAttributes.
	 *
	 * Either same length as Handles or 0 length.
	 */
	TArray<FKeyAttributes> KeyAttributes;

	bool HasKeyPositions() const { return !KeyPositions.IsEmpty(); }
	bool HasKeyAttributes() const { return !KeyAttributes.IsEmpty(); }
	
	bool HasData() const { return !KeyHandles.IsEmpty(); }
	operator bool() const { return HasData(); }
	
	SIZE_T GetAllocatedSize() const { return KeyHandles.GetAllocatedSize() + KeyPositions.GetAllocatedSize() + KeyAttributes.GetAllocatedSize(); }
	
	template<typename T>
	static SIZE_T GetAllocatedSize(const TMap<T, FCurveKeyData>& InMapping)
	{
		return Algo::TransformAccumulate(InMapping, [](const TPair<FCurveModelID, FCurveKeyData>& Pair)
		{
			return Pair.Value.GetAllocatedSize();
		}, 0);
	}

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		KeyHandles.Shrink();
		KeyPositions.Shrink();
		KeyAttributes.Shrink();
	}
};

/** Snapshot of curve editor content. */
struct FCurveKeyDataMap
{
	/** The state that the curves had. */
	TMap<FCurveModelID, FCurveKeyData> SavedCurveState;

	FCurveKeyDataMap() = default;
	explicit FCurveKeyDataMap(TMap<FCurveModelID, FCurveKeyData> SavedCurveState) : SavedCurveState(MoveTemp(SavedCurveState)) {}

	SIZE_T GetAllocatedSize() const { return FCurveKeyData::GetAllocatedSize(SavedCurveState); }
};
}
