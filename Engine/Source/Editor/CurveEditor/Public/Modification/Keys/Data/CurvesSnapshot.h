// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "CurveKeyDataMap.h"
#include "Curves/RealCurve.h"
#include "Misc/Optional.h"

namespace UE::CurveEditor
{
/**
 * Holds data that is captured to diff a curves previous state to a current state.
 * 
 * Not all data needs to be captured, e.g. Handles, KeyPositions, Attributes can be empty and CurveAttributes unset.
 * It would be pointless to construct this without any members set.
 * If either KeyPositions or Attributes is set, then Handles is also set.
 * Intended to be built using FCurvesSnapshotBuilder and ECurveChangeFlags.
 */
struct FCurveDiffingData : FCurveKeyData
{
	/** The curve attributes if they were captured. */
	TOptional<FCurveAttributes> CurveAttributes;

	bool HasCurveAttributes() const { return CurveAttributes.IsSet(); }
	
	bool HasData() const { return FCurveKeyData::HasData() || CurveAttributes.IsSet(); }
	operator bool() const { return FCurveDiffingData::HasData(); }
	
	SIZE_T GetAllocatedSize() const { return FCurveKeyData::GetAllocatedSize() + sizeof(CurveAttributes); }
	
	template<typename T>
	static SIZE_T GetAllocatedSize(const TMap<T, FCurveDiffingData>& InMapping)
	{
		return Algo::TransformAccumulate(InMapping, [](const TPair<FCurveModelID, FCurveDiffingData>& Pair)
		{
			return Pair.Value.GetAllocatedSize();
		}, 0);
	}
};

/** Holds selected data about curves. This data is usually used to diff against changes made to those curves. */
struct FCurvesSnapshot
{
	/**
	 * The info about the curves.
	 *
	 * Not all members of FCurveDiffingData need to contain data: just the members intended for diffing.
	 * Intended to be built using FCurvesSnapshotBuilder and ECurveChangeFlags.
	 */
	TMap<FCurveModelID, FCurveDiffingData> CurveData;

	FCurvesSnapshot() = default;
	explicit FCurvesSnapshot(TMap<FCurveModelID, FCurveDiffingData> InCurveData) : CurveData(MoveTemp(InCurveData)) {}
	
	SIZE_T GetAllocatedSize() const { return FCurveDiffingData::GetAllocatedSize(CurveData); }
};
}
