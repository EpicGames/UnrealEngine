// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AddKeysChangeData.h"
#include "CurveAttributeChangeData.h"
#include "KeyAttributeChangeData.h"
#include "MoveKeysChangeData.h"
#include "RemoveKeysChangeData.h"

namespace UE::CurveEditor
{
/** Stores changes made to curves in the curve editor. */
struct FGenericCurveChangeData
{
	FMoveKeysChangeData MoveKeysData;
	FAddKeysChangeData AddKeysData;
	FRemoveKeysChangeData RemoveKeysData;
	FKeyAttributeChangeData KeyAttributeData;
	FCurveAttributeChangeData CurveAttributeData;
	
	bool HasChanges() const { return MoveKeysData.HasChanges() || AddKeysData.HasChanges() || RemoveKeysData.HasChanges() || KeyAttributeData.HasChanges() || CurveAttributeData.HasChanges(); }
	operator bool() const { return HasChanges(); }

	/** @return The number of distinct change types. Useful for testing. */
	int32 NumChangeTypes() const
	{
		return static_cast<int32>(MoveKeysData.HasChanges())
		+ static_cast<int32>(AddKeysData.HasChanges())
		+ static_cast<int32>(RemoveKeysData.HasChanges())
		+ static_cast<int32>(KeyAttributeData.HasChanges())
		+ static_cast<int32>(CurveAttributeData.HasChanges());
	}
	SIZE_T GetAllocatedSize() const
	{
		return MoveKeysData.GetAllocatedSize()
			+ AddKeysData.GetAllocatedSize()
			+ RemoveKeysData.GetAllocatedSize()
			+ KeyAttributeData.GetAllocatedSize()
			+ CurveAttributeData.GetAllocatedSize();
	}
	
	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		MoveKeysData.Shrink();
		AddKeysData.Shrink();
		RemoveKeysData.Shrink();
		KeyAttributeData.Shrink();
	}
};
}
