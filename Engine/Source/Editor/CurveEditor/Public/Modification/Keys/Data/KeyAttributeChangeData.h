// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"

namespace UE::CurveEditor
{
/** Stores data for restoring key attributes for a single curve. */
struct FKeyAttributeChangeData_PerCurve
{
	/** The key handles that changed attributes. */
	TArray<FKeyHandle> Handles;

	// Ideally, we'd only save the delta change values.
	// Using 2 arrays like this, uses 2x amount of RAM than the optimal solution would. 
	// However, the tradeoff is that this implementation is easy to understand (less error-prone).
	// I tried the "ideal" solution: there's many more details to take care of. So I decided, it's not worth to engineer anything better for now.
	
	/** The key attributes before the change. Set attributes to this on undo. Each index corresponds to Handles. */
	TArray<FKeyAttributes> BeforeChange;
	/** The key attributes after the change. Set attributes to this on redo. Each index corresponds to Handles. */
	TArray<FKeyAttributes> AfterChange;

	bool HasChanges() const { return !Handles.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const { return Handles.GetAllocatedSize() + BeforeChange.GetAllocatedSize() + AfterChange.GetAllocatedSize(); }

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		Handles.Shrink();
		BeforeChange.Shrink();
		AfterChange.Shrink();
	}
};

/** Stores data for restoring key attributes of multiple curves. */
struct FKeyAttributeChangeData
{
	TMap<FCurveModelID, FKeyAttributeChangeData_PerCurve> ChangedCurves;
	
	bool HasChanges() const { return !ChangedCurves.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const
	{
		return ChangedCurves.GetAllocatedSize()
			+ Algo::TransformAccumulate(ChangedCurves, [](const TPair<FCurveModelID, FKeyAttributeChangeData_PerCurve>& Pair)
			{
				return Pair.Value.GetAllocatedSize();
			}, 0);
	}

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		ChangedCurves.Shrink();
		for (TPair<FCurveModelID, FKeyAttributeChangeData_PerCurve>& Pair : ChangedCurves)
		{
			Pair.Value.Shrink();
		}
	}
};
}
