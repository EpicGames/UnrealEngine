// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Accumulate.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CurveEditorTypes.h"

namespace UE::CurveEditor
{
struct FMoveKeysChangeData_PerCurve
{
	/** The moved key handles. */
	TArray<FKeyHandle> Handles;
	
	/**
	 * The amount the corresponding key was moved.
	 * 
	 * When applying (redo) the change, add the value to the key position.
	 * When reverting (undo) the change, subtract the value.
	 */
	TArray<FVector2f> Translations;

	bool HasChanges() const { return !Handles.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const { return Handles.GetAllocatedSize() + Translations.GetAllocatedSize(); }

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		Handles.Shrink();
		Translations.Shrink();
	}
};

/**
 * Data for an operation that translates keys.
 * This can still be improved: if all keys were transformed uniformly, then we only need to store a single FVector2f.
 */
struct FMoveKeysChangeData
{
	/** The per curve changes. */
	TMap<FCurveModelID, FMoveKeysChangeData_PerCurve> ChangedCurves;
	
	bool HasChanges() const { return !ChangedCurves.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const
	{
		return ChangedCurves.GetAllocatedSize()
			+ Algo::TransformAccumulate(ChangedCurves, [](const TPair<FCurveModelID, FMoveKeysChangeData_PerCurve>& Pair)
			{
				return Pair.Value.GetAllocatedSize();
			}, 0);
	}

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		ChangedCurves.Shrink();
		for (TPair<FCurveModelID, FMoveKeysChangeData_PerCurve>& Pair : ChangedCurves)
		{
			Pair.Value.Shrink();
		}
	}
};
}
