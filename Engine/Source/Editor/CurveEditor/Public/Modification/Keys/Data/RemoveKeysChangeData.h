// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "CurveKeyDataMap.h"

namespace UE::CurveEditor
{
struct FRemoveKeysChangeData
{
	/**
	 * On undo, add back FKeyDataSnapshot::Keys together with the saved positions and attributes.
	 * On redo, remove the FKeyDataSnapshot::Keys.
	 */
	TMap<FCurveModelID, FCurveKeyData> SavedCurveState;
	
	bool HasChanges() const { return !SavedCurveState.IsEmpty(); }
	operator bool() const { return HasChanges(); }
	SIZE_T GetAllocatedSize() const { return FCurveKeyData::GetAllocatedSize(SavedCurveState); }

	/** Shrinks the containers' used memory to smallest possible to store elements currently in it. */
	void Shrink()
	{
		SavedCurveState.Shrink();
		for (TPair<FCurveModelID, FCurveKeyData>& Pair : SavedCurveState)
		{
			Pair.Value.Shrink();
		}
	}
};
}
