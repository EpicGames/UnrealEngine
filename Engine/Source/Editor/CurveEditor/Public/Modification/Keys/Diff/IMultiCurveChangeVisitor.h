// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

struct FCurveModelID;

namespace UE::CurveEditor
{
class ISingleCurveChangeVisitor;
	
/**
 * Interface for processing changes made to multiple FCurveModels.
 * @see ISingleCurveChangeVisitor
 * @see FCurveChangeDiff
 */
class IMultiCurveChangeVisitor
{
public:
	
	/**
	 * Called when changes are about to be processed for InCurveModel.
	 *
	 * You are supposed to construct some ICurveChangeVisitor and call InProcessCallback with it.
	 * InProcessCallback will only use your visitor for the duration of the function call.
	 * This pattern allows construction of your visitor on the stack (avoiding heap allocating it).
	 * 
	 * A sample implementation could be:
	 * void FFooDiffBuilder::ProcessChange(const FCurveModelID& InCurveModel, TFunctionRef<void(ICurveChangeVisitor&)> InProcessCallback)
	 * {
	 *		FFooCurveChangeVisitor MyVisitor(InCurveModel); 
	 *		InProcessCallback(MyVisitor);
	 * }
	 */
	virtual void ProcessChange(const FCurveModelID& InCurveModel, TFunctionRef<void(ISingleCurveChangeVisitor&)> InProcessCallback) = 0;

	/** Called before any ProcessChange calls are made. */
	virtual void PreProcessChanges() {}
	/** Called after all changes have been listed, i.e. no more ProcessChange will follow. */
	virtual void PostProcessChanges() {}

	virtual ~IMultiCurveChangeVisitor() = default;
};
}
