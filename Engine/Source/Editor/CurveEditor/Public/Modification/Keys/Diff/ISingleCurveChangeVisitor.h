// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::CurveEditor
{
struct FCurveAttributeChangeData_PerCurve;
struct FCurveKeyData;
struct FKeyAttributeChangeData_PerCurve;
struct FMoveKeysChangeData_PerCurve;

/**
 * Interface for processing changes made to a single FCurveModel.
 * @see ICurveDiffBuilder
 * @see FCurveChangeDiff
 */
class ISingleCurveChangeVisitor
{
public:

	/** Processes moved keys. Only called if InData contains any changes, i.e. InData.HasChanges() == true. */
	virtual void ProcessMoveKeys(FMoveKeysChangeData_PerCurve&& InData) {}

	/** Processes added keys. Only called if InData contains any changes, i.e. InData.HasChanges() == true. */
	virtual void ProcessAddKeys(FCurveKeyData&& InData) {}
	
	/** Processes removed keys. Only called if InData contains any changes, i.e. InData.HasChanges() == true. */
	virtual void ProcessRemoveKeys(FCurveKeyData&& InData) {}
	
	/** Processes changed key attributes. Only called if InData contains any changes, i.e. InData.HasChanges() == true. */
	virtual void ProcessKeyAttributesChange(FKeyAttributeChangeData_PerCurve&& InData) {}

	/** Processes changed curve attributes. Only called if InData contains any changes, i.e. InData.HasChanges() == true. */
	virtual void ProcessCurveAttributesChange(FCurveAttributeChangeData_PerCurve&& InData) {}
	
	virtual ~ISingleCurveChangeVisitor() = default;
};
}