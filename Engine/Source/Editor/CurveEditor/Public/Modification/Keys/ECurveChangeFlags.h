// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::CurveEditor
{
/** Flags about the types of changes that can be made to a curve. */
enum class ECurveChangeFlags : uint8
{
	None,

	/** Detect change of FKeyPosition for FKeyHandles that are present before and after the change. */
	MoveKeys = 1 << 0,
	/** Detect FKeyHandles that are present after the change, but were not present before the change. */
	AddKeys = 1 << 1,
	/** Detect FKeyHandles that are not present after the change, but were present before the change. */
	RemoveKeys = 1 << 2,
	/**
	 * Detect FKeyAttributes that are different after the change.
	 * The attribute changes are detected for FKeyHandles that are present before and after the change.
	 */
	KeyAttributes = 1 << 3,
	/** Detect changes to the curve attributes, e.g. pre and post extrapolation, etc. */
	CurveAttributes = 1 << 4,

	/**
	 * Use when you move keys and the keys may end up being stacked (i.e. have the same x positions).
	 * When keys have the same x positions, curve editor removes the stacked keys so there's only 1 at the same x position.
	 * So removed keys must be tracked.
	 */
	MoveKeysAndRemoveStackedKeys = MoveKeys | RemoveKeys,
	/** Track anything that modifies key data. */
	KeyData = MoveKeys | AddKeys | RemoveKeys | KeyAttributes,
	/** All data should be tracked.*/
	All = KeyData | CurveAttributes
};
ENUM_CLASS_FLAGS(ECurveChangeFlags);
}
