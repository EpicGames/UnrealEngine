// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FCurveModel;
class IBufferedCurveModel;
namespace UE::CurveEditor { struct FGenericCurveChangeData; }
struct FCurveModelID;

namespace UE::CurveEditor::GenericCurveChange
{
/** Applies InDeltaChange to InCurves (redo). */
UE_API void ApplyChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FGenericCurveChangeData& InDeltaChange,
	const double InCurrentSliderPos = 0
	);
/** Reverts InDeltaChange from InCurves (undo). */
UE_API void RevertChange(const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& InCurves, const FGenericCurveChangeData& InDeltaChange,
	const double InCurrentSliderPos = 0
	);

/** Applies InDeltaChange to InCurveEditor (redo). */
UE_API void ApplyChange(const FCurveEditor& InCurveEditor, const FGenericCurveChangeData& InDeltaChange);
/** Reverts InDeltaChange from InCurveEditor (undo). */
UE_API void RevertChange(const FCurveEditor& InCurveEditor, const FGenericCurveChangeData& InDeltaChange);
}

#undef UE_API