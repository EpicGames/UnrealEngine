// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Data/CurvesSnapshot.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"

struct FCurveModelID;

namespace UE::CurveEditor
{
/** Utility for retrieving some common performance data about FCurveDiffingData (or a collection thereof). */
template<typename T>
class TCurvesSnapshotPerfInfo
{
	const TMap<T, FCurveDiffingData>& Data;
public:

	explicit TCurvesSnapshotPerfInfo(const TMap<T, FCurveDiffingData>& InMap UE_LIFETIMEBOUND) : Data(InMap) {}
	explicit TCurvesSnapshotPerfInfo(const FCurvesSnapshot& InSnapshot UE_LIFETIMEBOUND) : Data(InSnapshot.CurveData) {}

	SIZE_T GetTotalAllocatedSize() const;

	SIZE_T NumKeys() const;
	SIZE_T NumPositions() const;
	SIZE_T NumAttributes() const;

	SIZE_T GetAllocatedSize_Keys() const;
	SIZE_T GetAllocatedSize_Positions() const;
	SIZE_T GetAllocatedSize_Attributes() const;
};

enum class ECurvesSnapshotPerfFlags
{
	None,

	TotalAllocatedSize = 1 << 0,
	
	Keys = 1 << 1,
	Positions = 1 << 2,
	Attributes = 1 << 3,

	All = TotalAllocatedSize | Keys | Positions | Attributes
};
ENUM_CLASS_FLAGS(ECurvesSnapshotPerfFlags);

/** Converts the specified info into a string. */
template<typename T>
FString DumpSnapshotPerfData(const TCurvesSnapshotPerfInfo<T> InData, ECurvesSnapshotPerfFlags InFlags = ECurvesSnapshotPerfFlags::All);
	
template<typename T>
FString DumpSnapshotPerfData(const TMap<T, FCurveDiffingData>& InMap, ECurvesSnapshotPerfFlags InFlags = ECurvesSnapshotPerfFlags::All)
{
	return DumpSnapshotPerfData(TCurvesSnapshotPerfInfo<T>(InMap), InFlags);
}
inline FString DumpSnapshotPerfData(const FCurvesSnapshot& InSnapshot, ECurvesSnapshotPerfFlags InFlags = ECurvesSnapshotPerfFlags::All)
{
	return DumpSnapshotPerfData(InSnapshot.CurveData, InFlags);
}
}

namespace UE::CurveEditor
{
template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::GetTotalAllocatedSize() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.GetAllocatedSize();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::NumKeys() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyHandles.Num();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::NumPositions() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyPositions.Num();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::NumAttributes() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyAttributes.Num();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::GetAllocatedSize_Keys() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyHandles.GetAllocatedSize();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::GetAllocatedSize_Positions() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyPositions.GetAllocatedSize();
	}, 0);
}

template <typename T>
SIZE_T TCurvesSnapshotPerfInfo<T>::GetAllocatedSize_Attributes() const
{
	return Algo::TransformAccumulate(Data, [](const TPair<T, FCurveDiffingData>& Pair)
	{
		return Pair.Value.KeyAttributes.GetAllocatedSize();
	}, 0);
}

template <typename T>
FString DumpSnapshotPerfData(const TCurvesSnapshotPerfInfo<T> InData, ECurvesSnapshotPerfFlags InFlags)
{
	TStringBuilder<256> Buffer;
	bool bNeedsSeparator = false;
	const auto AddSeparator = [&bNeedsSeparator, &Buffer]
	{
		if (!bNeedsSeparator)
		{
			bNeedsSeparator = true;
		}
		else
		{
			Buffer << TEXT(", ");
		}
	};

	if (EnumHasAnyFlags(InFlags, ECurvesSnapshotPerfFlags::TotalAllocatedSize))
	{
		AddSeparator();
		Buffer << TEXT("Total Size: ") << *FText::AsMemory(InData.GetTotalAllocatedSize()).ToString();
	}

	if (EnumHasAnyFlags(InFlags, ECurvesSnapshotPerfFlags::Keys))
	{
		AddSeparator();
		Buffer << InData.NumKeys() << TEXT(" keys (") << *FText::AsMemory(InData.GetAllocatedSize_Keys()).ToString() << TEXT(")");
	}

	if (EnumHasAnyFlags(InFlags, ECurvesSnapshotPerfFlags::Keys))
	{
		AddSeparator();
		Buffer << InData.NumKeys() << TEXT(" positions (") << *FText::AsMemory(InData.GetAllocatedSize_Positions()).ToString() << TEXT(")");
	}

	if (EnumHasAnyFlags(InFlags, ECurvesSnapshotPerfFlags::Keys))
	{
		AddSeparator();
		Buffer << InData.NumKeys() << TEXT(" attributes (") << *FText::AsMemory(InData.GetAllocatedSize_Attributes()).ToString() << TEXT(")");
	}
	
	return FString(Buffer.ToString());
}
}