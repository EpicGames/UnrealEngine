// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveModel.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::CurveEditor
{
/** Meta data about a FCurveModel useful to cache FCurveModelID when a curve with the same data is re-added to FCurveEditor. */
struct FCurveMetaDataIdentifiers
{
	/** The UObject that owned the CurveModel. */
	TWeakObjectPtr<UObject> Owner;
	
	/** This curve's short display name. Used in situations where other mechanisms provide enough context about what the curve is (such as "X") */
	FText ShortDisplayName;
	/** This curve's long display name. Used in situations where the UI doesn't provide enough context about what the curve is otherwise (such as "Floor.Transform.X") */
	FText LongDisplayName;

	/** This curve's short intention (such as Transform.X or Scale.X). Used internally to match up curves when saving/restoring curves between different objects. */
	FString IntentionName;
	/** 
	 * This curve's long intention (such as foot_fk_l.Transform.X or foot_fk_r.Scale.X). Used internally to match up curves when saving/restoring curves between different objects.
	 * Long intention names have priority in copy/paste over short intention names, but we fall back to short intention if it's unclear what the user is trying to do.
	 */
	FString LongIntentionName;

	/** The original channel name, used mostly to make sure names match with BP/Scripting */
	FName ChannelName;

	explicit FCurveMetaDataIdentifiers(const FCurveModel& InCurveModel)
		: Owner(InCurveModel.GetOwningObject())
		, ShortDisplayName(InCurveModel.GetShortDisplayName())
		, LongDisplayName(InCurveModel.GetLongDisplayName())
		, IntentionName(InCurveModel.GetIntentionName())
		, LongIntentionName(InCurveModel.GetLongIntentionName())
		, ChannelName(InCurveModel.GetChannelName())
	{}

	friend bool operator==(const FCurveMetaDataIdentifiers& Left, const FCurveMetaDataIdentifiers& Right)
	{
		return Left.Owner == Right.Owner
			&& Left.ChannelName == Right.ChannelName
			&& Left.IntentionName == Right.IntentionName
			&& Left.LongIntentionName == Right.LongIntentionName
			&& Left.ShortDisplayName.EqualTo(Right.ShortDisplayName)
			&& Left.LongDisplayName.EqualTo(Right.LongDisplayName);
	}
	friend bool operator!=(const FCurveMetaDataIdentifiers& Left, const FCurveMetaDataIdentifiers& Right)
	{
		return !(Left == Right);
	}
};
}
