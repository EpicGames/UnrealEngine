// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "CurveEditorTypes.h"

class FCurveEditor;
class FCurveModel;

#define UE_API CURVEEDITOR_API

namespace UE::CurveEditor
{
struct FCurveMetaDataIdentifiers;

/**
 * These arguments are used to temporarily resolve FCurveModel after it has been removed from FCurveEditor.
 * 
 * Remember, FCurveModel::CurveData only holds the curves that are currently visible.
 * Some systems, e.g. undo / redo with FGenericCurveCommandChange, require a curve model later.
 */
struct FCurveModelLookUpArgs
{
	/** The curve editor instance we're operating on */
	const FCurveEditor& CurveEditor;
	/** The curve trying to be resolved. */
	FCurveModelID CurveID;

	explicit FCurveModelLookUpArgs(const FCurveEditor& CurveEditor, const FCurveModelID& CurveID) : CurveEditor(CurveEditor), CurveID(CurveID)
	{}

	/** @return Metadata information the curve had when it was last added FCurveEditor::AddCurve. */
	CURVEEDITOR_API const FCurveMetaDataIdentifiers* GetCurveMetaData() const;
};
}

#undef UE_API