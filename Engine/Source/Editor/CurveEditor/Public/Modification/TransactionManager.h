// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorCommandChange.h"
#include "Internationalization/Text.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class UCurveEditorTransactionObject;

namespace UE::CurveEditor
{
struct FGenericCurveChangeData;
	
/** Interacts with the transaction system to enable command-based actions for curve editor. */
class FTransactionManager : public FNoncopyable, public FGCObject
{
public:

	UE_API explicit FTransactionManager(TWeakPtr<FCurveEditor> InCurveEditor);

	/** Appends a changes made to curves. It is associated with the current transaction.  */
	void AppendCurveChange(UObject* InObject, FGenericCurveChangeData InChange, const FText& InDescription = FText::GetEmpty()) const;

	/**
	 * Appends a changes made to curves. It is associated with the current transaction. 
	 * Use this version when there is no UObject the change can be associated with - otherwise use the UObject overload.
	 */
	void AppendCurveChange(FGenericCurveChangeData InChange, const FText& InDescription = FText::GetEmpty()) const;
	
	/**
	 * Appends a change to the current transaction and associates it with an UObject.
	 * 
	 * You should prefer this overload whenever possible because it causes the object's package's dirty flag to be saved and restore by the undo system.
	 * Example: If you're modifying a FCurveModel, you'd pass in FCurveModel::GetOwningObject.
	 *
	 * @param InObject The object being modified
	 * @param InChange The change to associate with the currently open transaction
	 * @param InDescription Description of the change
	 */
	UE_API void AppendChange(UObject* InObject, TUniquePtr<FCurveEditorCommandChange> InChange, const FText& InDescription = FText::GetEmpty()) const;
	
	/**
	 * Appends a change to the current transaction.
	 * Use this version when there is no UObject the change can be associated with - otherwise use the UObject overload.
	 *
	 * The transaction can consist of multiple changes, e.g.
	 * - system 1 may move keys, and
	 * - system 2 could coniditonally snap the keys to full frames after checking that the user has toggled auto-snapping
	 * 
	 * @param InChange The change to associate with the currently open transaction
	 * @param InDescription Description of the change
	 */
	UE_API void AppendChange(TUniquePtr<FCurveEditorCommandChange> InChange, const FText& InDescription = FText::GetEmpty()) const;

	DECLARE_MULTICAST_DELEGATE(FOnCommandAppended);
	/** Invoked after a command has been appended to the undo stack. */
	FOnCommandAppended& OnCommandAppended() { return OnCommandAppendedDelegate; }
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCurvesChanged, const FGenericCurveChangeData&)
	/** Invoked when transaction changes to curves are made. */
	FOnCurvesChanged& OnCurvesChanged() { return OnCurvesChangedDelegate; }

	//~ Begin FGCObject Interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	//~ End FGCObject Interface

private:

	/** The UObject that transactions are associated with. The transaction system needs to associate FCommandChanges with an UObject. */
	TObjectPtr<UCurveEditorTransactionObject> DummyTransactionObject;

	/** Invoked after a command has been appended to the undo stack. */
	FOnCommandAppended OnCommandAppendedDelegate;

	/** Invoked when transaction changes to curves are made. */
	FOnCurvesChanged OnCurvesChangedDelegate;
};
}

#undef UE_API
