// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FCurveEditor;

namespace UE::CurveEditor
{
class FCurveEditorCommandChange;

/**
 * Base class for scoped changes that intend to append a command to the transaction system when the scope ends.
 * It detects if constructed within a FScopedTransaction context: by calling IsCancelled(), you can handle the case of the root transaction being cancelled.
 */
class FScopedChangeBase
	// We don't expect this to be copied / moved.
	// If needed in the future, you can remove this BUT go through the subclasses and make sure their (auto-generated) move constructors work.
	// Const members, like the ones in this class and subclasses, cause the compiler to auto-generate copy constructor instead of move constructors.
	: public FNoncopyable 
{
public:

	[[nodiscard]] explicit FScopedChangeBase(TWeakPtr<FCurveEditor> InCurveEditor, const FText& InDescription);

	/** Prevent the change from being appended. */
	void Cancel() { bIsCancelled = true; }

protected:
	
	/** Used to diff selection and append the change to the transaction system. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;
	/** Description to use for the change */
	const FText Description;
	
	/** Whether the change should be appended. */
	bool IsCancelled() const;

	/** Appends the change if it is not cancelled. */
	void TryAppendCommand(TUniquePtr<FCurveEditorCommandChange> InCommand) const;
	
private:

	/** Whether we were constructed in a transaction. Used to detect whether the transaction has since been cancelled. */
	const bool bWasStartedInTransaction;
	/** Whether to skip submitting the change when the scope ends. */
	bool bIsCancelled = false;
};
}
