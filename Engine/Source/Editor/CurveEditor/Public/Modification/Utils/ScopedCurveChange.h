// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "Modification/Keys/CurveChangeDiff.h"
#include "ScopedChangeBase.h"
#include "ScopedTransaction.h"
#include "Templates/SharedPointer.h"

struct FCurveModelID;
class FCurveEditor;

namespace UE::CurveEditor
{
enum class EScopedKeyChangeFlags : uint8
{
	None,

	/** Revert the change if FScopedKeyChange or surrounding FScopedTransaction becomes cancelled. */
	RevertOnCancel = 1 << 0,
	
	Default = RevertOnCancel
};
ENUM_CLASS_FLAGS(EScopedKeyChangeFlags);

class FModifyCurveObjectDetector;
	
/**
 * Starts a transaction. Detects changes made to the curves specified on construction and adds it as an undo-able action.
 * When the scope ends, checks whether keys were added, removed, moved, or its attributes (FKeyAttributes or FCurveAttributes) were edited.
 *
 * ========== Performance ==========
 * This class exists for convenience, faster iteration times, and more concise code but introduces overhead by temporarily buffering the curves.
 * If the overhead is too great, consider building the change directly, and appending it to the transaction manager manually.
 * However, in practice, the overhead should not be a concern, and for simplicity, usage of FScopedKeyChange is encouraged and preferred.
 * 
 * Stress test 15th of July, 2025, Development Build,
 * Sequencer: ControlRigExample -> CopyPaste_Level-> PossessableBindings_CopyPaste -> Select Possessable_FKCR_CopyPaste
 *  - Set Interp Linear:	109.782 keys,	Undo Size: 5.582 MiB,		Diff buffer size: 2.955 MiB,	Diff time: 4.2 ms
 *  - Move Keys:            109.782 keys,	Undo Size: 1.36 MiB,		Diff buffer size: 2.117 MiB,	Diff time: 5.3 ms
 *	- Drag Tangents:        109.782 keys,	Undo Size: 5.582 MiB,		Diff buffer size: 2.955 MiB,	Diff time: 4.4 ms
 *	- Blend values:         109.782 keys,	Undo Size: 6.583 MiB,		Diff buffer size: 4.654 MiB,	Diff time: 10.7 ms
 *	- Delete Keys:          109.782 keys,	Undo Size: 5.395 MiB,		Diff buffer size: 4.63 MiB,		Diff time: 1.8 ms
 * These are the stats logged by CurveEditor.LogDiffPerfStats 1.
 *
 * ========== Combining FScopedKeyChange and FScopedSelectionChange ==========
 * The following is operations supported:
 * - Combination with FScopedSelectionChange (see below),
 * - Nesting within FScopedTransaction; if the outer FScopedTransaction is cancelled, FScopedKeyChange can detect this (@see EScopedKeyChangeFlags).
 * - Nested FScopedKeyChanges
 * 
 * When combining FScopedKeyChange and FScopedSelectionChange, order matters.
 * Rule of thumb: Declare in the order the undo operation should revert; equivalent: declare in reverse order the redo operation should apply.
 * 
* Keep the following principles in mind:
 * - C++ will deconstruct in reverse order
 * - Unreal's transaction system execution order for custom changes is as follows:
 *	- Revert (undo) in insertion order. Below, the call order would be FSelectionChangeCommand::Revert, then FGenericCurveChangeCommand::Revert.
 *	- Apply (redo) in reverse insertion order. Below, the call order would be FGenericCurveChangeCommand::Apply, then FSelectionChangeCommand::Apply.
 * 
 * Example code for inserting a key and selecting it:
 *		FScopedSelectionChange SelectionChange; // When undoing, first remove from selection...
 *		FScopedKeyChange KeyChange; // ... then remove the key from the curve
 *		// Add key to some curve
 *		// Select the added key
 *		//~KeyChange > adds key change as undo action
 *		//~SelectionChange > adds selection change as undo action
 * If you changed the declaration order, on redo, we'd first add to selection and then add the key to the curve.
 * Redo would no longer work: as FSelectionChangeCommand would add the FKeyHandle to the selection first, the selection code would defensively check
 * that a key exists first and reject the function call as the key has not yet been added to the curve model.
 *
 * ========== Interaction with Modify()-based undo ==========
 * Summary: Do not call Modify() on UObjects that own a FCurveModel being tracked by FScopedCurveChange (@see FCurveModel::GetOwningObject).
 * @see UE-331831 for fixing this issue
 * Calling Modify() while FScopedCurveChange is in progress can cause subtle issues with the order of operations during undo / redo.
 * 
 * Example FChannelCurveModel:
 * {
 *		FCurveModel* FooModel = ...;
 *		FCurveModel* BarModel = ...;
 *		check(FooModel->GetOwningObject() == BarModel->GetOwningObject()); // The same object contains the curves.
 *		FScopedCurveChange Change(...);		// Track changs made to FooModel
 *		FooModel->AddKeys(...);				// Suppose this 1. mutates UObject to add the keys, then 2. calls Modify()
 *		BarModel->AddKeys(...);				// Suppose this 1. mutates UObject to add the keys, then 2. calls Modify()
 * }										// ~FScopedCurveChange now calls StoreUndo.
 * 
 * This scenario used to happen with MovieSceneTools' FChannelCurveModel. AddKeys would mutate the owning UMovieSceneParameterSection, so it contained the keys.
 * At the end of the AddKeys call, the invocation of ISequencer::OnChannelChanged would cause ControlRig's auto-keying feature to call Modify on the
 * owning object (in order to attempt to add a key for the key that was just added...).
 * 
 * This would result in the following undo buffer:
 *   - FTransaction::Records[0] contains the delta changes from the first Modify() call. 
 *   - FTransaction::Records[1] contains the command changes from ~FScopedCurveChange
 *   
 * Note, the first Modify would be called right after the keys were technically added to FooModel but not yet to BarModel.
 * So the resulting base state for UMovieSceneParameterSection::BoolParameterNamesAndCurves would contain:
 * - FooModel -> 1 key
 * - BarModel -> 0 key
 * The delta change would hence return reset BoolParameterNamesAndCurves so FooModel contains 1 key and BarModel contains 0 keys.
 * 
 * When you undo, the records are applied in reverse order:
 * - FTransaction::Records[1] removes the keys added by AddKeys call.
 * - FTransaction::Records[0] delta change is applied. Effectively, the key is added back to FooModel.
 * @see UE-331831 for fixing this issue.
 */
class FScopedCurveChange : public FScopedChangeBase
{
public:
	
	/**
	 * @param InCurveEditor The curve
	 * @param InKeyChangeDiff Contains the data that is supposed to be diffed against
	 * @param InDescription Optional description that will be used for the transaction title. This title does not matter if you have a surrounding FScopedTransaction.
	 * @param InFlags Extra flags to affect the behaviour of the change
	 */
	[[nodiscard]] CURVEEDITOR_API explicit FScopedCurveChange(
		TWeakPtr<FCurveEditor> InCurveEditor, FCurveChangeDiff InKeyChangeDiff, const FText& InDescription, EScopedKeyChangeFlags InFlags
		);
	
	[[nodiscard]] CURVEEDITOR_API explicit FScopedCurveChange(
		FCurvesSnapshotBuilder InDataToDiff, const FText& InDescription = FText::GetEmpty(), EScopedKeyChangeFlags InFlags = EScopedKeyChangeFlags::Default
		);

	CURVEEDITOR_API ~FScopedCurveChange();

private:

	/**
	 * The transaction active during the change.
	 * 
	 * This is required because FCurveModel implementations mark the owning package dirty when keys are modified.
	 * This achieves that undoing the change will unmark the package as dirty, if it was not dirty before the change.
	 * */
	FScopedTransaction Transaction;
	
	/** Checks for changes. */
	const FCurveChangeDiff ChangeDiff;
	/** Flags about the change, e.g. whether to revert changes if cancelled, etc. */
	const EScopedKeyChangeFlags Flags;

	/** Util for warning about curves being modified. */
	TPimplPtr<FModifyCurveObjectDetector> CurveModificationDetector;
	
	void FinalizeChangesForNewUndoSystem();
};
}
