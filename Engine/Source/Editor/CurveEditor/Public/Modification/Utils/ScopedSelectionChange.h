// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSelection.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "ScopedChangeBase.h"
#include "Selection/ScopedSelectionChangeEventSuppression.h"

#define UE_API CURVEEDITOR_API

namespace UE::CurveEditor
{
enum class EScopedSelectionChangeFlags : uint8
{
	None,

	/** Revert the change if FScopedSelectionChange or surrounding FScopedTransaction becomes cancelled. */
	RevertOnCancel = 1 << 0,

	/**
	 * While the scope is constructed, also construct a FScopedSelectionChangeEventSuppression.
	 *
	 * If set, then the curve editor's FCurveEditorSelection::OnSelectionChanged will not be broadcast during the scope. Once the scope ends, and a
	 * change has been made to selection, FCurveEditorSelection::OnSelectionChanged is invoked.
	 */
	SuppressOnSelectionChangedEvent = 1 << 1,

	Default = RevertOnCancel | SuppressOnSelectionChangedEvent
};
ENUM_CLASS_FLAGS(EScopedSelectionChangeFlags);

/**
 * Saves the current selection state and on destruction compares it against the then current selection.
 * If the selection has changed when the scope ends, then
 * - it appends an undo-able action to the curve editor transaction manager,
 * - conditionally reverts the change if you called FScopedTransaction::Cancel on the surrounding transaction,
 * - conditionally broadcasts FCurveEditorSelection::OnSelectionChanged if you set EScopedSelectionChangeFlags::SuppressOnSelectionChangedEvent.
 *
 * You can safely use this within the scope of a FScopedTransaction. In that case, the command will be added as sub-transaction to the parent
 * transaction.
 *
* The following is operations are supported:
 * - Combination with FScopedKeyChange (@see FScopedKeyChange for detailed explanation).
 * - Nesting within FScopedTransaction; if the outer FScopedTransaction is cancelled, FScopedSelectionChange can detect this (@see EScopedSelectionChangeFlags).
 * - Nested FScopedSelectionChange
 */
class FScopedSelectionChange : public FScopedChangeBase
{
public:

	[[nodiscard]] UE_API explicit FScopedSelectionChange(
		TWeakPtr<FCurveEditor> InCurveEditor, EScopedSelectionChangeFlags InFlags = EScopedSelectionChangeFlags::Default
		);
	[[nodiscard]] UE_API explicit FScopedSelectionChange(
		TWeakPtr<FCurveEditor> InCurveEditor, const FText& InDescription,
		EScopedSelectionChangeFlags = EScopedSelectionChangeFlags::Default
		);

	UE_API ~FScopedSelectionChange();

private:

	/** Flags that affect the behaviour of this change. */
	const EScopedSelectionChangeFlags Flags;

	/** Suppresses FCurveEditorSelection::OnSelectionChanged until the scope ends. Only set if the flags requested it. */
	TOptional<FScopedSelectionChangeEventSuppression> SelectionChangedSuppressor;

	/** The selection the editor had when the transaction was started.  */
	FCurveEditorSelection OriginalSelection;
};
}

#undef UE_API
