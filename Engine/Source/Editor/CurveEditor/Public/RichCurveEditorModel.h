// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "IBufferedCurveModel.h"
#include "Math/Range.h"
#include "Misc/Attribute.h"
#include "Misc/OptionalFwd.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class IBufferedCurveModel;
class UObject;
struct FCurveAttributes;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyHandle;
struct FKeyPosition;
struct FRichCurve;

class FRichCurveEditorModel : public FCurveModel
{
public:
	UE_API FRichCurveEditorModel(UObject* InOwner);

	UE_API virtual const void* GetCurve() const override;

	UE_API virtual void Modify() override;

	UE_API virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	UE_API virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override;
	UE_API virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;

	UE_API virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override;
	UE_API virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;

	UE_API virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	UE_API virtual void GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	UE_API virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;

	UE_API virtual void GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const override;
	UE_API virtual void SetCurveAttributes(const FCurveAttributes& InCurveAttributes) override;
	UE_API virtual void GetTimeRange(double& MinTime, double& MaxTime) const override;
	UE_API virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	UE_API virtual int32 GetNumKeys() const override;
	UE_API virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override;
	UE_API virtual TPair<ERichCurveInterpMode, ERichCurveTangentMode> GetInterpolationMode(const double& InTime, ERichCurveInterpMode DefaultInterpolationMode, ERichCurveTangentMode DefaultTangentMode) const override;

	UE_API virtual bool Evaluate(double ProspectiveTime, double& OutValue) const override;
	UE_API virtual void AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;
	UE_API virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override;

	UE_API virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;

	UE_API virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;

	virtual UObject* GetOwningObject() const override { return WeakOwner.Get(); }

	// Set a range to clamp key input values
	void SetClampInputRange(TAttribute<TRange<double>> InClampInputRange) { ClampInputRange = InClampInputRange; }

	// Check for whether this rich curve is valid. This is required mostly in the case of undo/redo, where 
	// curves can potentially become invalid underneath this model before owners get their undo/redo callbacks
	virtual bool IsValid() const = 0;

	// Get the rich curve we are operating on
	virtual FRichCurve& GetRichCurve() = 0;
	virtual const FRichCurve& GetReadOnlyRichCurve() const = 0;

private:

	TWeakObjectPtr<> WeakOwner;

	TAttribute<TRange<double>> ClampInputRange;
};

// Rich curve model operating on a raw curve ptr
class FRichCurveEditorModelRaw : public FRichCurveEditorModel
{
public:
	UE_API FRichCurveEditorModelRaw(FRichCurve* InRichCurve, UObject* InOwner);

	UE_API virtual bool IsReadOnly() const override;

	UE_API void SetIsReadOnly(TAttribute<bool> InReadOnlyAttribute);

	// FRichCurveEditorModel interface
	virtual bool IsValid() const override { return RichCurve != nullptr; }
	UE_API virtual FRichCurve& GetRichCurve() override;
	UE_API virtual const FRichCurve& GetReadOnlyRichCurve() const override;

private:
	FRichCurve* RichCurve;
	TAttribute<bool> ReadOnlyAttribute;
};

#undef UE_API
