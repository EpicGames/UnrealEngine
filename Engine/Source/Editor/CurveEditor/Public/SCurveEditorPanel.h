// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "WidgetFocusUtils.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "Filters/PromotedFilterCommandBinder.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FExtender;
class FTabManager;
class FToolBarBuilder;
class FMenuBuilder;
class FUICommandList;
class IDetailsView;
class IGraphEditorView;
class ITimeSliderController;
class SCurveEditorToolProperties;
class SCurveEditorView;
class SCurveEditorViewContainer;
class SCurveEditorFilterPanel;
class SCurveKeyDetailPanel;
class SScrollBox;
class SWidget;
class UCurveEditorFilterBase;
struct FCurveEditorDelayedDrag;
struct FCurveEditorEditObjectContainer;
struct FCurveEditorToolID;
struct FKeyEvent;
namespace UE::CurveEditor { class FPromotedFilterContainer; }
namespace UE::CurveEditor { class FSharedCurveInfoModel; }

/**
 * Curve editor widget that reflects the state of an FCurveEditor
 */
class SCurveEditorPanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCurveEditorPanel)
		: _GridLineTint(FLinearColor(0.1f, 0.1f, 0.1f, 1.f))
		, _MinimumViewPanelHeight(300.0f)
	{}

		/** Color to draw grid lines */
		SLATE_ATTRIBUTE(FLinearColor, GridLineTint)

		/** Tab Manager which owns this panel. */
		SLATE_ARGUMENT(TSharedPtr<FTabManager>, TabManager)

		/** Optional Time Slider Controller which allows us to synchronize with an externally controlled Time Slider */
		SLATE_ARGUMENT(TSharedPtr<ITimeSliderController>, ExternalTimeSliderController)

		/** If specified, causes the time snap adjustment UI controls to be disabled and specifies the tooltip to be used. Can be used to disable time snap controls when externally controlled. */
		SLATE_ATTRIBUTE(FText, DisabledTimeSnapTooltip)

		/** Widget slot for the tree content */
		SLATE_NAMED_SLOT(FArguments, TreeContent)

		/** The minimum height for the panel which contains the curve editor views. */
		SLATE_ARGUMENT(float, MinimumViewPanelHeight)

	SLATE_END_ARGS()

	UE_API SCurveEditorPanel();
	UE_API ~SCurveEditorPanel();

	/**
	 * Construct a new curve editor panel widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor);

	/**
	 * Access the combined command list for this curve editor and panel widget
	 */
	TSharedPtr<FUICommandList> GetCommands() const
	{
		return CommandList;
	}

	/**
	 * Access the details view used for editing selected keys
	 */
	TSharedPtr<class SCurveKeyDetailPanel> GetKeyDetailsView() const
	{
		return KeyDetailsView;
	}

	/**
	 * Access the filter panel
	 */
	TSharedPtr<class SCurveEditorFilterPanel> GetFilterPanel() const
	{
		return FilterPanel;
	}

	/**
	 * Access the tool properties panel
	 */
	TSharedPtr<class SCurveEditorToolProperties> GetToolPropertiesPanel() const
	{
		return ToolPropertiesPanel;
	}

	UE_API void AddView(TSharedRef<SCurveEditorView> ViewToAdd);

	UE_API void RemoveView(TSharedRef<SCurveEditorView> ViewToRemove);

	/** This returns an extender which is pre-configured with the standard set of Toolbar Icons. Implementers of SCurveEditorPanel should use this
	* to generate the icons and then add any additional context-specific icons (such as save buttons in the Asset Editor) to ensure that the Curve
	* Editor has a consistent set (and order) of icons across all usages.
	*/
	UE_API TSharedPtr<FExtender> GetToolbarExtender();

	/** Access the cached geometry of the outer scroll panel that contains this panel's views */
	UE_API const FGeometry& GetScrollPanelGeometry() const;

	/** Access the cached geometry of container housing all this panel's views */
	UE_API const FGeometry& GetViewContainerGeometry() const;

	/** Get all the views stored in this panel. */
	UE_API TArrayView<const TSharedPtr<SCurveEditorView>> GetViews() const;

	/** Get the grid line tint to be used for views on panel */
	FLinearColor GetGridLineTint() const { return GridLineTintAttribute.Get(); }

	/** Scroll this panel's view scroll box vertically by the specified amount */
	UE_API void ScrollBy(float Amount);

	/**
	 * Find all the views that the specified curve is being displayed on
	 * @note: Returns an in-place iterator to this curve's view mapping. Adding or removing curves from views *will* invalidate this iterator.
	 *
	 * @param InCurveID The identifier of the curve to find views for
	 * @return An iterator to all the views that this cuvrve is displayed within.
	 */
	TMultiMap<FCurveModelID, TSharedRef<SCurveEditorView>>::TConstKeyIterator FindViews(TRetainedRef<FCurveModelID> InCurveID)
	{
		return CurveViews.CreateConstKeyIterator(InCurveID.Get());
	}

	/**
	 * Remove the specified curve from all views it is currently displayed on.
	 */
	UE_API void RemoveCurveFromViews(FCurveModelID InCurveID);

	/** Get the last set View Mode for this UI. Utility function for the UI. */
	ECurveEditorViewID GetViewMode() const { return DefaultViewID; }

	/** Undo occurred, invalidate or update internal structures */
	UE_API void PostUndo();

	/** Reset Stored Min/Max's*/
	UE_API void ResetMinMaxes();

	/** Update the axis snapping based on the settings. */
	UE_API void UpdateAxisSnapping();

	/** Delegate for when the chosen filter class has changed */
	FSimpleDelegate OnFilterClassChanged;
	UE_API void FilterClassChanged();

	/** Enable/disable pending focus */
	UE_API void EnablePendingFocusOnHovering(const bool InEnabled);
	
	/** Broadcasts after the curve views have been rebuilt (by RebuildCurveViews). */
	FSimpleMulticastDelegate& OnPostRebuildCurveViews() { return OnPostRebuildCurveViewsDelegate; }

private:
	// SWidget Interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// ~SWidget Interface

	/*~ Keyboard interaction */
	virtual bool SupportsKeyboardFocus() const override { return true; }
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
    UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** @return Extender combining all ICurveEditorModule::GetAllToolBarMenuExtenders and the owning FCurveEditor's ICurveEditorExtensions. */
	UE_API TSharedPtr<FExtender> CombineEditorExtensions() const;
	/** Adds all base curve editor UI elements to ToolbarBuilder. */
	UE_API void BuildToolbar(FToolBarBuilder& InToolBarBuilder, TSharedPtr<FExtender> InBaseExtender);

	UE_API TSharedRef<SWidget> MakeTimeSnapMenu();
	UE_API FText GetTimeSnapMenuTooltip() const;

	UE_API TSharedRef<SWidget> MakeGridSpacingMenu();
	UE_API TSharedRef<SWidget> MakeAxisSnapMenu();

	UE_API bool IsInlineEditPanelEditable() const;
	UE_API EVisibility ShouldInstructionOverlayBeVisible() const;

private:
	
	UE_API TSharedRef<SWidget> MakeToolsComboMenu(TSharedPtr<FExtender> InExtender);
	UE_API FText GetCurrentToolLabel() const;
	UE_API FText GetCurrentToolDescription() const;
	UE_API FSlateIcon GetCurrentToolIcon() const;
	
private:
	
	// Tangent mode toolbar combo button
	UE_API TSharedRef<SWidget> MakeTangentModeMenu();
	UE_API FText GetTangentModeLabel() const;
	UE_API FText GetTangentModeTooltip() const;
	UE_API FSlateIcon GetTangentModeIcon() const;
	
	enum class ETangentModeComboState : uint8
	{
		// All keys in selection share the following mode:
		Constant,
		Linear,
		CubicAuto,
		CubicSmartAuto,
		CubicUser,
		CubicBreak,

		// The keys are mixed
		Mixed,
		// There are no keys in view so nothing can be set.
		NoKeys
	};
	UE_API ETangentModeComboState DetermineTangentMode() const;
	
private:

	/**
	 * Get the visibility for the value splitter control
	 */
	UE_API EVisibility GetSplitterVisibility() const;

	/*~ Event bindings */
	UE_API void UpdateTime();
	UE_API void UpdateEditBox();

	/** Creates the drop-down list you see when changing Curve View options. */
	UE_API TSharedRef<SWidget> MakeCurveEditorCurveViewOptionsMenu(TSharedPtr<FExtender> InExtender);
	/** Creates the drop-down for changing curves on a whole. */
	UE_API TSharedRef<SWidget> MakeCurvesMenu(TSharedPtr<FExtender> InExtender);
	UE_API void AddPreInfinityToMenu(FMenuBuilder& InMenuBuilder);
	UE_API void AddPostInfinityToMenu(FMenuBuilder& InMenuBuilder);
	UE_API FSlateIcon GetCurveExtrapolationPreIcon() const;
	UE_API FSlateIcon GetCurveExtrapolationPostIcon() const;

	/** Creates the Curve Editor Filter UI and pre-populates it with the specified class. */
	UE_API void ShowCurveFilterUI(TSubclassOf<UCurveEditorFilterBase> FilterClass);
	
private:

	/**
	 * Bind command mappings for this widget
	 */
	void BindCommands();
	
	/** Assign new attributes to the currently selected keys*/
	void HandleSetKeyAttributesCommand(FKeyAttributes InKeyAttributes, FText InDescription);
	/** Sets the key attributes on all selected keys. */
	void SetKeyAttributesOnSelection(const FKeyAttributes& InKeyAttributes, const FText& InDescription) const;
	/** Sets the key attributes on all keys. */
	void SetKeyAttributesOnAllCurves(const FKeyAttributes& InKeyAttributes, const FText& InDescription) const;

	/**
	 * Assign new curve attributes to all visible curves
	 */
	void SetCurveAttributes(FCurveAttributes CurveAttributes, FText Description);

	/** Compare all the currently selected keys' interp modes against the specified interp mode */
	bool CompareCommonInterpolationMode(ERichCurveInterpMode InterpMode) const;

	/** Compare all the currently selected keys' tangent modes against the specified tangent mode */
	bool CompareCommonTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode) const;

	/** Compare all the currently selected keys' tangent modes against the specified tangent mode */
	bool CompareCommonTangentWeightMode(ERichCurveInterpMode InterpMode, ERichCurveTangentWeightMode TangentWeightMode) const;

	/** Compare all the visible curves' pre-extrapolation modes against the specified extrapolation mode */
	bool CompareCommonPreExtrapolationMode(ERichCurveExtrapolation PreExtrapolationMode) const;

	/** Compare all the visible curves' post-extrapolation modes against the specified extrapolation mode */
	bool CompareCommonPostExtrapolationMode(ERichCurveExtrapolation PostExtrapolationMode) const;

	/**
	 * Toggle weighted tangents on the current selection
	 */
	UE_API void ToggleWeightedTangents();

	/**
	 * Check whether we can toggle weighted tangents on the current selection
	 */
	UE_API bool CanToggleWeightedTangents() const;

	/**
	 * Check whether or not we can set a key interpolation on the current selection. If no keys are selected, you can't set an interpolation!
	 */
	UE_API bool CanSetKeyInterpolation() const;

	/** Sets the axis snapping to the specified value. Only supports X, Y and None. */
	UE_API void SetAxisSnapping(ECurveEditorSnapAxis);

	/** Get a reference to the curve editor this panel represents. */
	TSharedPtr<FCurveEditor> GetCurveEditor() const { return CurveEditor; }

	float GetColumnFillCoefficient(int32 ColumnIndex) const
	{
		ensure(ColumnIndex == 0 || ColumnIndex == 1);
		return ColumnFillCoefficients[ColumnIndex];
	}

	/** Called when a column fill percentage is changed by a splitter slot. */
	UE_API void OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex);

	UE_API void OnSplitterFinishedResizing();
	
private:

	/**
	 * Create a new view for a model of the specified type, and add the curve to the view
	 *
	 * @param CurveModelID The ID of the curve we're creating a view for
	 * @param ViewTypeID   The ID of the view we'd like to create
	 * @param bPinned      Whether the view should be pinned or not
	 * @return A new view or nullptr if one could not be created
	 */
	UE_API TSharedPtr<SCurveEditorView> CreateViewOfType(FCurveModelID CurveModelID, ECurveEditorViewID ViewTypeID, bool bPinned);

private:

	/** The curve editor pointer */
	TSharedPtr<FCurveEditor> CurveEditor;

	/** Map from curve model ID to the views that it is on */
	TMultiMap<FCurveModelID, TSharedRef<SCurveEditorView>> CurveViews;

	/** Set of externally added views */
	TSet<TSharedRef<SCurveEditorView>> ExternalViews;

	/** (Optional) the current drag operation */
	TOptional<FCurveEditorDelayedDrag> DragOperation;

	/** Keeps track about information that is equal for all selected keys, or all keys if none are selected. */
	TPimplPtr<UE::CurveEditor::FSharedCurveInfoModel> SharedCurveInfo; 

	/** Attribute used for retrieving the desired grid line color */
	TAttribute<FLinearColor> GridLineTintAttribute;

	/** Attribute used for retrieving the tooltip for when the Time Snap control is disabled. Specifying this causes the Time Snap Adjustment to be disabled. */
	TAttribute<FText> DisabledTimeSnapTooltipAttribute;

	/** Edit panel */
	TSharedPtr<SCurveKeyDetailPanel> KeyDetailsView;

	/* Filter panel */
	TSharedPtr<SCurveEditorFilterPanel> FilterPanel;

	/** Tool options panel */
	TSharedPtr<SCurveEditorToolProperties> ToolPropertiesPanel;

	/** Map of edit UI widgets for each curve in the current selection set */
	TMap<FCurveModelID, TSharedPtr<SWidget>> CurveToEditUI;

	/** Command list for widget specific command bindings */
	TSharedPtr<FUICommandList> CommandList;

	/** Binds commands from FPromotedFilterContainer so the filters surfaced from SCurveEditorFilterPanel show up in the toolbar. */
	TUniquePtr<UE::CurveEditor::FPromotedFilterCommandBinder> ToolbarPromotedFilterBinder;

	/** Cached serial number from the curve editor selection. Used to update edit UIs when the selection changes. */
	uint32 CachedSelectionSerialNumber;

private:

	/** Sets the View Mode for the UI to the specified mode. This will destroy and re-create all views, but leave additional pinned views unmodified. */
	UE_API void SetViewMode(const ECurveEditorViewID NewViewMode);

	/** Compare if our current view mode matches the specified one. Utility function for the UI. */
	UE_API bool CompareViewMode(const ECurveEditorViewID InViewMode) const;

	/** Rebuild the Curve Views layout to match the currently specified View Mode. */
	UE_API void RebuildCurveViews();

	/** Reconstructs the properties widget on tool switch */
	UE_API void OnCurveEditorToolChanged(FCurveEditorToolID InToolId);

	/** Last Output Min and Max values for the views*/
	double LastOutputMin = DBL_MAX;
	double LastOutputMax = DBL_MIN;

	/** The last set View Mode for this UI. */
	ECurveEditorViewID DefaultViewID;

	/** Broadcasts after the curve views have been rebuilt (by RebuildCurveViews). */
	FSimpleMulticastDelegate OnPostRebuildCurveViewsDelegate;

	TMultiMap<ECurveEditorViewID, TSharedRef<SCurveEditorView>> FreeViewsByType;

	/** The scrool box that all curve views live inside */
	TSharedPtr<SScrollBox> ScrollBox;

	/** The container for all our vertically laid out curve views. */
	TSharedPtr<SCurveEditorViewContainer> CurveViewsContainer;

private:

	/** Pending focus handler */
	FPendingWidgetFocus PendingFocus;
	
	/** Whether to explicitly refresh the views for this panel */
	bool bNeedsRefresh;

	/** Serial number cached from FCurveEditor::GetActiveCurvesSerialNumber() on tick */
	uint32 CachedActiveCurvesSerialNumber;

	/** A copy of the View Geometry used to represent the View portion of the Curve Editor. */
	FGeometry CachedViewGeometry;

	/** The fill coefficients of each column in the grid. */
	float ColumnFillCoefficients[2];

	TSharedPtr<class SSplitter> TreeViewSplitter;

	/** Container of objects that are being used to edit keys on the curve editor */
	TUniquePtr<FCurveEditorEditObjectContainer> EditObjects;

	TWeakPtr<FTabManager> WeakTabManager;
};

#undef UE_API
