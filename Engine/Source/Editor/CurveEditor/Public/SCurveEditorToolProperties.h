// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorToolExtension.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class IStructureDetailsView;
struct FPropertyChangedEvent;

/**
 * Widget to show curve editor tool extension properties
 */
class SCurveEditorToolProperties : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorToolProperties)
	{}

	SLATE_END_ARGS()

	SCurveEditorToolProperties() : ToolId(FCurveEditorToolID::Unset()) {}

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor, FCurveEditorToolID ToolId);

	/**
	 * Called when the curve editor tool changes
	 * Sets the new tool ID and rebuilds the details view
	 */
	UE_API void OnToolChanged(FCurveEditorToolID NewToolId);

	/** The details view for tool properties */
	TSharedPtr<class IStructureDetailsView> GetStructureDetailsView() const { return DetailsView; }

private:

	/**
	 * Rebuilds details view
	 * Called either by the tool itself when it's properties change or when the curve editor tool changes
	 */
	UE_API void RebuildProperties();

	/**
	 * Called when the user enters in a new value for one of the properties
	 * Calls OnToolOptionsUpdated on the current tool
	 */
	UE_API void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

private:
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	TSharedPtr<IStructureDetailsView> DetailsView;
	FCurveEditorToolID ToolId;
};

#undef UE_API
