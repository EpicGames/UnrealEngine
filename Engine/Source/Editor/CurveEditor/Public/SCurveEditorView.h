// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SortedMap.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorTypes.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Math/TransformCalculus2D.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/Less.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "CurveEditorSettings.h"
#include "CurveDrawInfo.h"
#include "CurveEditorCurveDrawParamsCache.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FSlateRect;
class FText;
class SCurveEditorPanel;
class FCurveModel;
class FCurveEditorAxis;
class SRetainerWidget;
class SCurveEditorView;

enum class ECurveEditorAxisOrientation : uint8;
namespace UE::CurveEditor { class FCurveDrawParamsCache; }

/**
 * Identifier for a specific axis on a view. These identifiers are transient and should not be stored externally.
 */
struct FCurveEditorViewAxisID
{
	FCurveEditorViewAxisID()
		: Index(std::numeric_limits<uint8>::max())
	{}
	explicit FCurveEditorViewAxisID(uint8 InIndex)
		: Index(InIndex)
	{
		check(InIndex < std::numeric_limits<uint8>::max());
	}

	explicit operator bool() const
	{
		return Index != std::numeric_limits<uint8>::max();
	}

	friend uint32 GetTypeHash(FCurveEditorViewAxisID In)
	{
		return GetTypeHash(In.Index);
	}

	friend bool operator==(FCurveEditorViewAxisID A, FCurveEditorViewAxisID B)
	{
		return A.Index == B.Index;
	}

	friend bool operator!=(FCurveEditorViewAxisID A, FCurveEditorViewAxisID B)
	{
		return A.Index != B.Index;
	}

	friend bool operator<(FCurveEditorViewAxisID A, FCurveEditorViewAxisID B)
	{
		return A.Index < B.Index;
	}

private:

	operator int32() const
	{
		check(Index != std::numeric_limits<uint8>::max());
		return Index;
	}

	FCurveEditorViewAxisID& operator=(int32 InIndex)
	{
		check(InIndex >= 0 && InIndex < 255);
		Index = static_cast<uint8>(InIndex);
		return *this;
	}

	friend SCurveEditorView;
	uint8 Index;
};



/**
 * This is the base widget type for all views that exist on a curve editor panel. A view may contain 0 or more curves (stored in CurveInfoByID).
 * SCurveEditorViews are directly housed within a single SCurveEditorViewContainer which arranges each view vertically in order.
 *
 * View types:
 *		A view may have a centrally registered ID (See FCurveEditorViewRegistry and FCurveModel::GetSupportedViews) which allows any external curve model type to express support for any other view type.
 *		3 built-in views are provided:
 *			Absolute:   Shows 1 or more curves on a single 2D-scrollable grid using a single zoomable view-space.
 *			Normalized: Shows 1 or more curves on a single grid scrollable horizontally with all curves normalized to the extents of the view
 *			Stacked:    Shows 1 or more curves on separate fixed-height grids scrollable horizontally each curve normalized to the extents of its own grid
 *		Unregistered curve views may still be added to the view by calling SCurveEditorPanel::AddView
 *
 * Space Transformations:
 *		Views must define a valid 2-dimensional space (ViewSpace) that is used to convert from a pixel position to the virtual view space.
 *		Views may additionally implement per-curve transformations from view space to curve space, to allow for specific layouts/scales of curves within them.
 *		Per-curve transforms are stored inside FCurveInfo::ViewToCurveTransform
 *
 * Sizing:
 *		Views may be size in one of 2 ways:
 *			* Auto sized (bAutoSize=true): views derive their height from the widget's desired size;
 *			* Stretched (bAutoSize=false): views will be stretched to the size of the parent panel with a sensible minimum height.
 */
class SCurveEditorView : public SCompoundWidget
{
public:

	/**
	 * Default constructor
	 */
	UE_API SCurveEditorView();
	UE_API ~SCurveEditorView();

	/**
	 * Get the default screen space ultility that defines this view's input, output and pixel metrics
	 */
	UE_API FCurveEditorScreenSpace GetViewSpace() const;

	/**
	 * Get the screen space ultility that defines this view's input, output and pixel metrics for the specified axis combination
	 */
	UE_API FCurveEditorScreenSpace GetViewSpace(const FName& InHorizontalAxis, const FName& InVerticalAxis) const;

	/**
	 * Get the screen space ultility that defines the specified curves's input, output and pixel metrics.
	 * The resulting struct defines the transformation from this view's widget pixel space to the curve input/output space
	 */
	UE_API FCurveEditorScreenSpace GetCurveSpace(FCurveModelID CurveID) const;

	/** @return The transform used to translate view space (absolute key coordinates) to the curve space (e.g. 0-1 range in normalized view). */
	UE_API FTransform2d GetViewToCurveTransform(FCurveModelID CurveID) const;

	/**
	 * Retrieve the horizontal screen-space information for the specified axis
	 */
	UE_API FCurveEditorScreenSpaceH GetHorizontalAxisSpace(FCurveEditorViewAxisID ID) const;

	/**
	 * Retrieve the vertical screen-space information for the specified axis
	 */
	UE_API FCurveEditorScreenSpaceV GetVerticalAxisSpace(FCurveEditorViewAxisID ID) const;

	/**
	 * Retrueve the axis ID assigned to the specified curve and orientation
	 */
	UE_API FCurveEditorViewAxisID GetAxisForCurve(FCurveModelID CurveID, ECurveEditorAxisOrientation AxisOrientation) const;

	/**
	 * Retrueve the axis associated with the specified ID and orientation
	 */
	UE_API TSharedPtr<FCurveEditorAxis> GetAxis(FCurveEditorViewAxisID ID, ECurveEditorAxisOrientation AxisOrientation) const;

	/**
	 * Check whether this view should auto size to fit its FixedHeight or child slot content
	 */
	bool ShouldAutoSize() const
	{
		return bAutoSize;
	}

	/**
	 * Check whether this view can be interacted with. This should be checked by any tool or edit mode attempting to manipulated curves on this view.
	 */
	bool IsInteractive() const
	{
		return bInteractive;
	}

	/**
	 * Check whether this view has space for any more curves
	 */
	bool HasCapacity() const
	{
		return MaximumCapacity == 0 || CurveInfoByID.Num() < MaximumCapacity;
	}

	/**
	 * Retrieve the number of curves that exist on this view
	 */
	int32 NumCurves() const
	{
		return CurveInfoByID.Num();
	}

	/**
	 * Retrieve this view's input bounds, accounting for any offsets or padding from the outer container
	 */
	UE_API void GetInputBounds(double& OutInputMin, double& OutInputMax) const;

	/**
	 * Retrieve this view's minimum output bound in view space
	 */
	double GetOutputMin() const
	{
		return OutputMin;
	}

	/**
	 * Retrieve this view's maximum output bound in view space
	 */
	double GetOutputMax() const
	{
		return OutputMax;
	}

	/**
	 * Set this view's output bounds
	 */
	UE_API void SetOutputBounds(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID = FCurveEditorViewAxisID());

	/**
	 * Set this view's input bounds
	 */
	UE_API void SetInputBounds(double InInputMin, double InInputMax, FCurveEditorViewAxisID AxisID = FCurveEditorViewAxisID());

	/**
	 * Zoom this view in or out around its center point
	 *
	 * @param Amount       The amount to zoom by horizontally and vertically as a factor of the current size
	 */
	UE_API void Zoom(const FVector2D& Amount);

	/**
	 * Zoom this view in or out around the specified point
	 *
	 * @param Amount       The amount to zoom by horizontally and vertically as a factor of the current size
	 * @param TimeOrigin   The time origin to zoom around
	 * @param ValueOrigin  The value origin to zoom around
	 */
	UE_API void ZoomAround(const FVector2D& Amount, double InputOrigin, double OutputOrigin);

	/** This should be called every tick by an owning widget, to see if the cache is valid, which will then recreate it and invalidate widget*/
	UE_API virtual void CheckCacheAndInvalidateIfNeeded();

	UE_DEPRECATED(5.3, "Use UpdateViewToTransformCurves(double InputMin, double InputMax) instead.")
	virtual void UpdateViewToTransformCurves() {}

	/** Function to make sure to update the view to the transform curves, we need to do this before we cache*/
	virtual void UpdateViewToTransformCurves(double InputMin, double InputMax) {};

	/** Frame the view vertially by the input and output bounds, peformaing any custom clipping as needed */
	UE_API virtual void FrameVertical(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID = FCurveEditorViewAxisID());

	/** Frame the view horizontall by the input and output bounds, peformaing any custom clipping as needed */
	UE_API virtual void FrameHorizontal(double InInputMin, double InInputMax, FCurveEditorViewAxisID AxisID = FCurveEditorViewAxisID());

	/** Returns the cure editor associated with this view, or nullptr if the view is not specific to a curve editor */
	UE_API TSharedPtr<FCurveEditor> GetCurveEditor() const;

	/** Invoked when we detect that a curve needs to be redrawn, i.e. when FCurveModel::HasChangedAndResetTest returns true. */
	UE::CurveEditor::FOnCurveHasChangedExternally& OnCurveHasChangedExternally();

public:

	/**
	 * Retrieve all the curve points that overlap the specified rectangle in widget space
	 *
	 * @param WidgetRectangle  The rectangle to hit test against. May not hit points that would exist outside of the view's visible bounds.
	 * @param OutPoints        (required) pointer to an array to populate with overlapping points
	 * @return Whether any points were within the widget range
	 */
	virtual bool GetPointsWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const
	{ return false; }

	/**
	 * Retrieve all the curve points, if any of the interpolating points overlap the specified rectangle in widget space
	 *
	 * @param WidgetRectangle  The rectangle to hit test against. May not hit points that would exist outside of the view's visible bounds.
	 * @param OutPoints        (required) pointer to an array to populate with overlapping points
	 * @return Whether any points were within the widget range
	 */
	virtual bool GetCurveWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const
	{ return false; }

	/**
	 * Tries to retrieve all curve points that overlap the rectangle.
	 * If no points are found, selects the overlapping curves instead.
	 *
	 * @param WidgetRectangle  The rectangle to hit test against. May not hit points that would exist outside of the view's visible bounds.
	 * @param OutPoints        (required) pointer to an array to populate with overlapping points
	 * 
	 * @return Whether anything was selected.
	 */
	bool GetPointsThenCurveWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const
	{
		return GetPointsWithinWidgetRange(WidgetRectangle, OutPoints) || GetCurveWithinWidgetRange(WidgetRectangle, OutPoints);
	}

	/**
	 * Retrieve the id of the hovered curve
	 */
	virtual TOptional<FCurveModelID> GetHoveredCurve() const { return TOptional<FCurveModelID>(); }

	/**
	 * Bind UI commands for this view
	 */
	virtual void BindCommands()
	{}

	/**
	 * Invoked when a curve has been added or removed from this list.
	 * @note: Care should be taken here to not impede performance as this will get called each time a curve is added/removed
	 */
	virtual void OnCurveListChanged()
	{}

	/**
	 * Should the tools respect the global snapping rules for Time (X axis) input when manipulating a curve in this view. Can be set to false to ignore the snap setting.
	 */
	virtual bool IsTimeSnapEnabled() const
	{
		return true;
	}

	/**
	 * Should the tools respect the global snapping rules for Value (Y axis) input when manipulating a curve in this view. Can be set to false to ignore the snap setting.
	 */
	virtual bool IsValueSnapEnabled() const
	{
		return true;
	}

	virtual void GetGridLinesX(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels = nullptr) const {}
	virtual void GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels = nullptr) const {}

	/** Request a new render from the retainer widget */
	UE_API void RefreshRetainer();

	/** Update the custom axes if necessary */
	UE_API void UpdateCustomAxes();

protected:

	/** Inits this view with the curve editor and sets up required callbacks. */
	UE_API void InitCurveEditorReference(const TSharedRef<FCurveEditor>& InCurveEditor);
	
	/** Gets info about the curves being drawn. Converts actual curves into an abstract series of lines/points/handles/etc. */
	UE_DEPRECATED(5.6, "Instead use the CurveDrawParamsCache, use FCurveDrawParamsCache::GetCurveDrawParamsSynchronous to get Curve Draw Parms as with this function.")
	UE_API void GetCurveDrawParams(TArray<FCurveDrawParams>& OutDrawParams);

	/** Gets curve draw params  get the cache by calling a single curve */
	UE_DEPRECATED(5.6, "Instead use GetCurveDrawParamsCache, use FCurveDrawParamsCache::GetCurveDrawParamSynchronous to get Curve Draw Parm as with this function.")
	UE_API void GetCurveDrawParam(TSharedPtr<FCurveEditor>& CurveEditor, const FCurveModelID& ModelID, FCurveModel* CurveModel, FCurveDrawParams& OutDrawParam) const;

	/** Update all the curve view transforms from curve models */
	UE_API void UpdateCurveViewTransformsFromModels();

	// ~SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Only the curve editor panel can add curves to views to ensure internal consistency between the views and SCurveEditorPanel::CurveViews */
	friend struct FCurveEditorPanelViewTracker;

	/** Add a curve to this view */
	UE_API void AddCurve(FCurveModelID CurveID);

	/** Add a curve from this view */
	UE_API void RemoveCurve(FCurveModelID CurveID);

public:

	/** (Default: false) When true, this view has been pinned to the top of the view container. Only takes affect from the next call to SCurveEditorPanel::RebuildCurveViews. */
	uint8 bPinned : 1;
	/** (Default: true) When true, this view should accept interactive operations such as drags and tool interaction */
	uint8 bInteractive : 1;
	/** (Default: false) When true, this view has fixed vertical bounds that should never be changed by zooming or panning */
	uint8 bFixedOutputBounds : 1;
	/** (Default: true) See class notes - defines whether this view should size to its desired size (true) or stretch to the height of the panel (false). Only takes affect from the next call to SCurveEditorPanel::RebuildCurveViews. */
	uint8 bAutoSize : 1;
	/** (Default: false) Defines whether this view should remain on the panel UI even if it does not represent any curves. */
	uint8 bAllowEmpty : 1;
	/** (Default: false) When true, custom axes need to be rebuilt before use. */
	uint8 bAllowModelViewTransforms : 1;
	/** (Default: false) When true, custom axes need to be rebuilt before use. */
	uint8 bUpdateModelViewTransforms : 1;
	/** (Default: true) When true, this view has models that need the default grid lines drawing. */
	uint8 bNeedsDefaultGridLinesH : 1;
	/** (Default: true) When true, this view has models that need the default grid lines drawing. */
	uint8 bNeedsDefaultGridLinesV : 1;

	/** (Default: 0) Should be assigned on Construction of derived types. Defines a custom sort bias to use when sorting the stack of views (before sorting by pinned/unpinned state) */
	int8 SortBias = 0;

	/** This view's type identifier. Assigned by the curve editor panel after construction. */
	ECurveEditorViewID ViewTypeID = ECurveEditorViewID::Invalid;

	/** Transient integer that is assigned each time the view container order is changed, to guarantee relative ordering of views when the list changes */
	int32 RelativeOrder = TNumericLimits<int32>::Max();

	/** The maximum number of curves allowed on this view, or 0 for no limit */
	int32 MaximumCapacity = 0;

	/** (Optional) Attribute that defines a fixed height for this view. Overrides ChildSlot's desired size when set */
	TAttribute<float> FixedHeight;

protected:

	/** The curve editor associated with this view. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** This view's minimum visible output value */
	double OutputMin = 0.0;

	/** This view's maximum visible output value */
	double OutputMax = 1.0;

	struct FCurveInfo
	{
		/** The linear index of the curve within this view determined by the order curves were added. */
		int32 CurveIndex;
		FTransform2d ViewToCurveTransform;
		FCurveEditorViewAxisID HorizontalAxis;
		FCurveEditorViewAxisID VerticalAxis;
	};

	/** Map from curve identifier to specific info pertaining to that curve for this view.
	 * @note: Should only be added to or removed from in AddCurve/RemoveCurve. Derived types must only change the FCurveInfo contained within this map.
	 */
	TSortedMap<FCurveModelID, FCurveInfo, TInlineAllocator<1>> CurveInfoByID;

	struct FAxisInfo
	{
		TSharedPtr<FCurveEditorAxis> Axis;
		double Min = 0.0;
		double Max = 1.0;
		int32 UseCount = 0;
	};

	TArray<FAxisInfo> CustomHorizontalAxes;
	TArray<FAxisInfo> CustomVerticalAxes;

	FAxisInfo& GetHorizontalAxisInfo(FCurveEditorViewAxisID ID)
	{
		check(ID);
		return CustomHorizontalAxes[ID];
	}
	FAxisInfo& GetVerticalAxisInfo(FCurveEditorViewAxisID ID)
	{
		check(ID);
		return CustomVerticalAxes[ID];
	}

	/** Flag enum signifying how the curve cache has changed since it was last generated
	Note for a data change it may only effect certain data(curves) not every drawn curve*/	
	enum class UE_DEPRECATED(5.6, "Instead use the CurveDrawParamsCache,  call FCurveDrawParamsCache::ECurveCacheFlags to get a corresponding type.") ECurveCacheFlags : uint8
	{
		CheckCurves = 0,       // The cache may be valid need to check each curve to see if they are still valid
		All = 1 << 0,		   // Get all
	};

	/** Curve cache flags that change based upon data or view getting modified*/
	UE_DEPRECATED(5.6, "Instead use the CurveDrawParamsCache, call FCurveDrawParamsCache::GetCurveCacheFlags to get current flags. Use FCurveDrawParamsCache::Update to update them.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ECurveCacheFlags CurveCacheFlags;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Curve draw parameters that are re-generated on tick if the cache has changed. We generate them once and then they're used in multiple places per frame. */
	UE_DEPRECATED(5.5, "Instead use the CurveDrawParamsCache, call GetCurveDrawParamsSynchronous.")
	TArray<FCurveDrawParams> CachedDrawParams;

	/** Cache for curve draw params */
	TSharedPtr<UE::CurveEditor::FCurveDrawParamsCache> CurveDrawParamsCache;

	/** Set of Cached values we need to check each tick to see if we need to redo cache*/
	struct UE_DEPRECATED(5.6, "Instead use the CurveDrawParamsCache, call FCurveDrawParamsCache::FCachedCurveEditorData to get a corresponding type.") FCachedValuesToCheck
	{
		/** Serial number cached from FCurveEditor::GetActiveCurvesSerialNumber() on tick */
		uint32 CachedActiveCurvesSerialNumber;

		/** Serial number bached from CurveEditorSelecstion::GetSerialNumber */
		uint32 CachedSelectionSerialNumber;

		/** Cached Tangent Visibility*/
		ECurveEditorTangentVisibility CachedTangentVisibility;

		/** Cached input and output min max values to see if we need to recalc curves, though we need to poll it's safer*/
		double CachedInputMin, CachedInputMax, CachedOutputMin, CachedOutputMax;

		/** Cached Geometry Size*/
		FVector2D CachedGeometrySize;
	};

	UE_DEPRECATED(5.6, "Instead use the CurveDrawParamsCache, use FCurveDrawParamsCache::GetCurveEditorData to get cached values. Use FCurveDrawParamsCache::Update to update them.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCachedValuesToCheck CachedValues;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Possible pointer to a retainer widget that we may need to force update*/
	TSharedPtr<SRetainerWidget> RetainerWidget;

public:
	void SetRetainerWidget(TSharedPtr<SRetainerWidget>& InWidget)
	{
		RetainerWidget = InWidget;
	}
};

#undef UE_API
