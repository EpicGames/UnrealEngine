// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
// Forward Declare
class IPropertyRowGenerator;
class SWidget;

/**
 * Inline details panel that lets you edit the Time and Value of a generic FCurveEditor Key
 */
class SCurveKeyDetailPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveKeyDetailPanel)
	{}

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor);
	TSharedPtr<IPropertyRowGenerator> GetPropertyRowGenerator() const { return PropertyRowGenerator; }

private:
	UE_API void PropertyRowsRefreshed();
	UE_API void ConstructChildLayout(TSharedPtr<SWidget> TimeWidget, TSharedPtr<SWidget> ValueWidget);

private:
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	TSharedPtr<SWidget> TempTimeWidget;
	TSharedPtr<SWidget> TempValueWidget;
};

#undef UE_API
