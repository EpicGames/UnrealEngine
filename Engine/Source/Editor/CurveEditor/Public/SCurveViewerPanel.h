// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveDrawInfo.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Rendering/RenderingCommon.h"
#include "SCurveEditorView.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;

/**
 * Curve viewer widget that reflects the state of an FCurveEditor
 */
class SCurveViewerPanel : public SCurveEditorView
{
	SLATE_BEGIN_ARGS(SCurveViewerPanel) {}

	SLATE_ATTRIBUTE(float, CurveThickness)

	SLATE_END_ARGS()

	/**
	 * Construct a new curve editor panel widget
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor);

	/**
	 * Access the draw parameters that this curve editor has cached for this frame
	 */
	const TArray<FCurveDrawParams>& GetCachedDrawParams() const
	{
		return CachedDrawParams;
	}

private:

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

public:

	/**
	 * Draw curve data
	 */
	UE_API void DrawCurves(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const;

private:
	/** Curve draw parameters that are re-generated on tick */
	TArray<FCurveDrawParams> CachedDrawParams;

	/** Thickness of the displayed curve */
	TAttribute<float> CurveThickness;
};

#undef UE_API
