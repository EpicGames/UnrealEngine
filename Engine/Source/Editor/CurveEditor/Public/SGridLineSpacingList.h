// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Events.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"

/** A widget which creates a list of pre-determined numbers and a spot to enter a custom number. Similar to SNumericList but without the dropdown menu. */
class SGridLineSpacingList : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnValueChanged, TOptional<float> );

public:
	/** Represents a named numeric value for display in the drop down menu. */ 
	class FNamedValue
	{
	public:
		/** Creates a new FNamedValue 
		  * @InValue The numeric value to be assigned
		  * @InName The display text for the value in the UI 
		  * @InDescription The description of the value used in tooltips or wherever a longer description is needed. */
		FNamedValue( TOptional<float> InValue, FText InName, FText InDescription )
		{
			Value = InValue;
			Name = InName;
			Description = InDescription;
		}

		TOptional<float> GetValue() const
		{
			return Value;
		}

		FText GetName() const
		{
			return Name;
		}

		FText GetDescription() const
		{
			return Description;
		}

	private:
		TOptional<float> Value;
		FText Name;
		FText Description;
	};

public:
	SLATE_BEGIN_ARGS( SGridLineSpacingList )
		: _DropDownValues()
		, _MinDesiredValueWidth( 40)
		, _bShowNamedValue(true)
		{}

		/** The values which are used to populate the drop down menu. */
		SLATE_ARGUMENT( TArray<FNamedValue>, DropDownValues )
		/** Controls the minimum width for the text box portion of the control. */
		SLATE_ATTRIBUTE( float, MinDesiredValueWidth )
		/** The value displayed by the control. */
		SLATE_ATTRIBUTE( TOptional<float>, Value )
		/** If enabled, attempt to show the original name specified in the Drop Down values for a given entry instead of just a straight FText::AsNumber(NumericType). */
		SLATE_ATTRIBUTE(bool, bShowNamedValue)
		/** An optional header to prepend to the generated list. */
		SLATE_ATTRIBUTE( FText, HeaderText)
		/** The callback for when the value changes. */
		SLATE_EVENT( FOnValueChanged, OnValueChanged )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		DropDownValues = InArgs._DropDownValues;
		Value = InArgs._Value;
		OnValueChanged = InArgs._OnValueChanged;

		FMenuBuilder MenuBuilder(true, nullptr);

		// Allow an optional header text because it looks nicer and helps give context to the widget.
		if (InArgs._HeaderText.IsSet())
		{
			MenuBuilder.BeginSection("Header", InArgs._HeaderText.Get());
		}

		for (const FNamedValue& NamedValue : DropDownValues)
		{
			AddMenuEntry(MenuBuilder, NamedValue);
		}

		// Add the custom entry box at the end. 
		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			.Padding(FMargin(8, 0, 0, 0))
			[
				SNew(SEditableTextBox)
				.RevertTextOnEscape(true)
				.SelectAllTextWhenFocused(true)
				.Text(this, &SGridLineSpacingList::GetValueText)
				.OnTextCommitted(this, &SGridLineSpacingList::ValueTextCommitted)
				.MinDesiredWidth(InArgs._MinDesiredValueWidth)
			],
			NSLOCTEXT("SNumericList", "CustomNumberDisplayLabel", "Custom")
		);

		ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
	}

private:
	void AddMenuEntry(FMenuBuilder& MenuBuilder, const FNamedValue& Info)
	{
		MenuBuilder.AddMenuEntry(
			Info.GetName(),
			Info.GetDescription(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SGridLineSpacingList::SetValue, Info.GetValue()),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SGridLineSpacingList::IsSameValue, Info.GetValue())
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	bool IsSameValue(TOptional<float> InValue) const
	{
		return Value.Get() == InValue;
	}

	/** Get the value text for displaying in the custom entry box. We use a textbox here so we can show actual names from the original list if possible. */
	FText GetValueText() const
	{
		if (bShowNamedValue.Get())
		{
			for ( FNamedValue DropDownValue : DropDownValues )
			{
				if (Value.Get() == DropDownValue.GetValue())
				{
					return DropDownValue.GetName();
				}
			}
		}

		return FText::FromString(LexToSanitizedString(Value.Get() ? Value.Get().GetValue() : 0.0f));
	}

	void ValueTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
	{
		if (InNewText.IsNumeric())
		{
			TOptional<float> NewValue = 0.0f;
			TTypeFromString<float>::FromString(NewValue.GetValue(), *InNewText.ToString());
			OnValueChanged.ExecuteIfBound(NewValue);
		}
	}

	void SetValue( TOptional<float> InValue )
	{
		FSlateApplication::Get().ClearKeyboardFocus( EFocusCause::Cleared );
		OnValueChanged.ExecuteIfBound(InValue);
	}

private:
	TArray<FNamedValue> DropDownValues;
	TAttribute<bool> bShowNamedValue;
	TAttribute<TOptional<float>> Value;
	FOnValueChanged OnValueChanged;
};
