// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

namespace UE::CurveEditor
{
/**
 * Prevents FCurveEditorSelection::OnSelectionChanged from being broadcast while the constructed scope is open.
 * Once all scopes end, and if changes to selection have been made, FCurveEditorSelection::OnSelectionChanged is broadcast exactly once.
 * Nesting FScopedSelectionChangeEventSuppression is supported.
 * 
 * This is an optimization if you are going to make multiple changes to selection and want the delegate to be broadcast after you're done.
 *
 * Example usage:
 * TSharedRef<FCurveEditor> MyCurveEditor = ...;
 * {
 *		FScopedSelectionChangeEventSuppression SuppressChangeEvents(MyCurveEditor);
 *		MyCurveEditor->Selection.Toggle(FooCurveId, ECurvePointType::Key, FooKeyHandle); // Normally this would invoke OnSelectionChanged. 
 *		MyCurveEditor->Selection.Toggle(BarCurveId, ECurvePointType::Key, BarKeyHandle); // So would this
 *		{
 *			FScopedSelectionChangeEventSuppression AnotherChange(MyCurveEditor);
 *			MyCurveEditor->Selection.Toggle(BazCurveId, ECurvePointType::Key, BazKeyHandle);
 *			//~ FCurveEditorSelection::OnSelectionChanged not invoked due to the outer scope.
 *		}
 *		
 *		// ~FScopedSelectionChangeEventSuppression will now invoke FCurveEditorSelection::OnSelectionChanged once exactly.
 * }
 */
class FScopedSelectionChangeEventSuppression : public FNoncopyable
{
public:

	UE_API explicit FScopedSelectionChangeEventSuppression(const TSharedRef<FCurveEditor>& InCurveEditor);
	UE_API ~FScopedSelectionChangeEventSuppression();

private:

	/** Handles the case of curve editor being destroyed during the scope, which should not happen but we handle for safety. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;
};
}

#undef UE_API
