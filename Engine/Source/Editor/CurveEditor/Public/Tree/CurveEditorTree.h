// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/ForEach.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

#define UE_API CURVEEDITOR_API

enum class ECurveEditorTreeFilterType : uint32;

class FCurveEditor;
class FCurveEditorTree;
struct FCurveEditorTreeFilter;
struct ICurveEditorTreeItem;

/** Enumeration of bitmask values specifying how a specific tree item is interpreted by the current set of filters */
enum class ECurveEditorTreeFilterState : uint8
{
	/** The item did not match any filter, and neither did any of its parents or children */
	NoMatch        = 0x00,

	/** One of this item's parents matched a filter (ie it resides within a matched item) */
	ImplicitChild  = (1<<0),
	
	/** One of this item's descendant children matched a filter (ie it is a parent of a matched item) */
	ImplicitParent = (1<<1),
	
	/** This item itself matched one or more of the filters */
	Match          = (1<<2),

	/** This item in the tree should be expanded according to one or more of the filters */
	Expand         = (1<<3),

	MatchBitMask   = (ImplicitParent | Match | ImplicitChild),
};
ENUM_CLASS_FLAGS(ECurveEditorTreeFilterState);

/**
 * Scoped guard that prevents the broadcast of tree events for the duration of its lifetime. Will trigger necessary events after the last remaining guard has been destroyed.
 */
struct FScopedCurveEditorTreeEventGuard
{
	UE_API explicit FScopedCurveEditorTreeEventGuard(FCurveEditorTree* InTree);
	UE_API ~FScopedCurveEditorTreeEventGuard();

	UE_API FScopedCurveEditorTreeEventGuard(FScopedCurveEditorTreeEventGuard&& RHS);
	UE_API FScopedCurveEditorTreeEventGuard& operator=(FScopedCurveEditorTreeEventGuard&& RHS);

private:
	/** OnItemsChanged.SerialNumber cached on construction. Used to detect whether OnItemsChanged should be broadcast. */
	uint32 CachedItemSerialNumber;
	/** OnSelectionChanged.SerialNumber cached on construction. Used to detect whether OnSelectionChanged should be broadcast. */
	uint32 CachedSelectionSerialNumber;
	/** OnFiltersChanged.SerialNumber cached on construction. Used to detect whether OnFiltersChanged should be broadcast. */
	uint32 CachedFiltersSerialNumber;
	/** Pointer back to the owning tree */
	FCurveEditorTree* Tree;
};

/**
 * Generic multicast delegate that guards against re-entrancy for the curve editor tree
 */
class FCurveEditorTreeDelegate : public FSimpleMulticastDelegate
{
private:

	friend FCurveEditorTree;
	friend FScopedCurveEditorTreeEventGuard;

	/* Broadcast this delegate provided it is not re-entrant */
	void Broadcast();

	/** Whether this delegate is currently broadcasting */
	bool bBroadcasting = false;

	/** Serial number that is incremented any time this delegate should be broadcast */
	uint32 SerialNumber = 0;
};

/**
 * Struct that represents an event for when the tree has been changed.
 * This type carefully only allows FScopedCurveEditorTreeEventGuard to broadcast the event, and makes special checks for re-entrancy
 */
struct FCurveEditorTreeEvents
{
	/** Event that is broadcast when the tree items container has changed */
	FCurveEditorTreeDelegate OnItemsChanged;

	/** Event that is broadcast when the selection has changed */
	FCurveEditorTreeDelegate OnSelectionChanged;

	/** Event that is broadcast when any kind of filtering has changed (ie active state, filters being added/removed etc) */
	FCurveEditorTreeDelegate OnFiltersChanged;

private:

	friend FScopedCurveEditorTreeEventGuard;

	/** Counter that is incremented for each living instance of FScopedCurveEditorTreeEventGuard */
	uint32 UpdateGuardCounter = 0;

};

/**
 * Container specifying a linear set of child identifiers and 
 */
struct FSortedCurveEditorTreeItems
{
	/** (default: false) Whether the child ID array needs re-sorting or not */
	bool bRequiresSort = false;

	/** Sorted list of child IDs */
	TArray<FCurveEditorTreeItemID> ChildIDs;
};

/**
 * Concrete type used as a tree item for the curve editor. No need to derive from this type - custom behaviour is implemented through ICurveEditorTreeItem.
 * Implemented in this way to ensure that all hierarchical information can be reasoned about within the curve editor itself, and allow for mixing of tree item types from any usage domain.
 */
struct FCurveEditorTreeItem
{
	/** @return This item's unique identifier within the tree */
	FCurveEditorTreeItemID GetID() const
	{
		return ThisID;
	}

	/** @return This parent's unique identifier within the tree, or FCurveEditorTreeItemID::Invalid for root items */
	FCurveEditorTreeItemID GetParentID() const
	{
		return ParentID;
	}

	/**
	 * Access the sorted list of children for this item
	 *
	 * @return An array view to this item's children
	 */
	TArrayView<const FCurveEditorTreeItemID> GetChildren() const
	{
		return Children.ChildIDs;
	}

	/**
	*Get optional unique path name for the tree editor item
	*/
	const TOptional<FString>& GetUniquePathName() const { return UniquePathName; } 

	/**
	*Set optional unique path name for the tree editor item
	*/
	void SetUniquePathName(const TOptional<FString>& InName) { UniquePathName = InName; }

	/**
	 * Access the user-specified implementation for this tree item
	 * @return A strong pointer to the implementation or null if it has expired, or was never assigned
	 */
	TSharedPtr<ICurveEditorTreeItem> GetItem() const
	{
		return StrongItemImpl.IsValid() ? StrongItemImpl : WeakItemImpl.Pin();
	}

	/**
	 * Overwrite this item's implementation with an externally held implementation to this tree item. Does not hold a strong reference.
	 */
	void SetWeakItem(TWeakPtr<ICurveEditorTreeItem> InItem)
	{
		WeakItemImpl = InItem;
		StrongItemImpl = nullptr;
	}

	/**
	 * Overwrite this item's implementation, holding a strong reference to it for the lifetime of this tree item.
	 */
	void SetStrongItem(TSharedPtr<ICurveEditorTreeItem> InItem)
	{
		WeakItemImpl = nullptr;
		StrongItemImpl = InItem;
	}

	/**
	 * Get all the curves currently represented by this tree item. Items may not be created until the tree item has been selected
	 */
	TArrayView<const FCurveModelID> GetCurves() const
	{
		return Curves;
	}

	/**
	 * Retrieve all the curves for this tree item, creating them through ICurveEditorTreeItem::CreateCurveModels if there are none
	 */
	UE_API TArrayView<const FCurveModelID> GetOrCreateCurves(FCurveEditor* CurveEditor);

	/**
	 * Destroy any previously constructed curve models that this tree item owns
	 */
	UE_API void DestroyCurves(FCurveEditor* CurveEditor);

	/**
	 * Destroy any previously constructed unpinned curve models that this tree item owns
	 */
	UE_API void DestroyUnpinnedCurves(FCurveEditor* CurveEditor);

private:

	friend FCurveEditorTree;

	/** This item's ID */
	FCurveEditorTreeItemID ThisID;
	/** This parent's ID or FCurveEditorTreeItemID::Invalid() for root nodes */
	FCurveEditorTreeItemID ParentID;
	/** Optional Unique Path Name*/
	TOptional<FString> UniquePathName;
	/** A weak pointer to an externally held implementation. Mutually exclusive to StrongItemImpl. */
	TWeakPtr<ICurveEditorTreeItem> WeakItemImpl;
	/** A strong pointer to an implementation for this tree item. Mutually exclusive to WeakItemImpl. */
	TSharedPtr<ICurveEditorTreeItem> StrongItemImpl;
	/** All the curves currently added to the curve editor from this tree item. */
	TArray<FCurveModelID, TInlineAllocator<1>> Curves;
	/** This item's sorted children. */
	FSortedCurveEditorTreeItems Children;
};


/**
 * Sparse map of filter states specifying items that have matched a filter
 */
struct FCurveEditorFilterStates
{
	/**
	 * Reset all the filter states currently being tracked (does not affect IsActive
	 */
	void Reset()
	{
		FilterStates.Reset();
		NumMatched = 0;
		NumMatchedImplicitly = 0;
	}

	int32 GetNumMatched() const
	{
		return NumMatched;
	}

	int32 GetNumMatchedImplicitly() const
	{
		return NumMatchedImplicitly;
	}

	/**
	 * Retrieve the filter state for a specific tree item ID
	 * @return The item's filter state, or ECurveEditorTreeFilterState::Match if filters are not currently active.
	 */
	ECurveEditorTreeFilterState Get(FCurveEditorTreeItemID ItemID) const
	{
		if (!bIsActive)
		{
			// If not active, everything is treated as having matched the (non-existent) filters
			return ECurveEditorTreeFilterState::Match;
		}

		const ECurveEditorTreeFilterState* State = FilterStates.Find(ItemID);
		return State ? *State : ECurveEditorTreeFilterState::NoMatch;
	}

	template <typename CallableT>
	void ForEachItemState(CallableT Callable) const
	{
		Algo::ForEach(FilterStates, Callable);
	}

	/**
	 * Assign a new filter state to an item
	 */
	void SetFilterState(FCurveEditorTreeItemID ItemID, ECurveEditorTreeFilterState NewState)
	{
		const ECurveEditorTreeFilterState* Existing = FilterStates.Find(ItemID);
		if (Existing)
		{
			if ((*Existing & ECurveEditorTreeFilterState::Match) != ECurveEditorTreeFilterState::NoMatch)
			{
				--NumMatched;
			}
			else if ((*Existing & ECurveEditorTreeFilterState::MatchBitMask) != ECurveEditorTreeFilterState::NoMatch)
			{
				--NumMatchedImplicitly;
			}
		}

		if ((NewState & ECurveEditorTreeFilterState::MatchBitMask) == ECurveEditorTreeFilterState::NoMatch)
		{
			FilterStates.Remove(ItemID);
		}
		else
		{
			FilterStates.Add(ItemID, NewState);

			if ((NewState & ECurveEditorTreeFilterState::Match) != ECurveEditorTreeFilterState::NoMatch)
			{
				++NumMatched;
			}
			else
			{
				++NumMatchedImplicitly;
			}
		}
	}

	/**
	 * Check whether filters are active or not
	 */
	bool IsActive() const
	{
		return bIsActive;
	}

	/**
	 * Activate the filters so that they begin to take effect
	 */
	void Activate()
	{
		bIsActive = true;
	}

	/**
	 * Deactivate the filters so that they no longer take effect
	 */
	void Deactivate()
	{
		bIsActive = false;
	}

private:

	/** Whether filters should be active or not */
	bool bIsActive = false;

	/** The total number of nodes that have matched all the filters */
	int32 NumMatched = 0;

	/** The total number of nodes that have not matched the filters but have a parent or child that did */
	int32 NumMatchedImplicitly = 0;

	/** Filter state map. Items with no implicit or explicit filter state are not present */
	TMap<FCurveEditorTreeItemID, ECurveEditorTreeFilterState> FilterStates;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCurveEditorToggleExpansionState, bool);
DECLARE_MULTICAST_DELEGATE(FOnCurveEditorExpandAllNodes);
DECLARE_MULTICAST_DELEGATE(FOnCurveEditorCollapseAllNodes);

/** 
 * Complete implementation of a curve editor tree. Only really defines the hierarchy and selection states for tree items.
 */
class FCurveEditorTree
{
public:
	
	/** Defines a predicate for sorting curve editor tree item implementations. */
	UE_DEPRECATED(5.8, "Use FCurveTreeItemSortPredicate instead.")
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FTreeItemSortPredicate, const ICurveEditorTreeItem* /* ItemA */, const ICurveEditorTreeItem* /* ItemB */);
	/** Returns whether ItemA < ItemB. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FCurveTreeItemSortPredicate, const FCurveEditorTreeItem& /* ItemA */, const FCurveEditorTreeItem& /* ItemB */);

	/** Structure containing all the events for this tree */
	FCurveEditorTreeEvents Events;

	UE_API FCurveEditorTree();

	/**
	 * Retrieve an item from its ID, assuming it is definitely valid
	 */
	UE_API FCurveEditorTreeItem& GetItem(FCurveEditorTreeItemID ItemID);

	/**
	 * Retrieve an item from its ID, assuming it is definitely valid
	 */
	UE_API const FCurveEditorTreeItem& GetItem(FCurveEditorTreeItemID ItemID) const;

	/**
	 * Retrieve an item from its ID or nullptr if the ID is not valid
	 */
	UE_API FCurveEditorTreeItem* FindItem(FCurveEditorTreeItemID ItemID);

	/**
	 * Retrieve an item from its ID or nullptr if the ID is not valid
	 */
	UE_API const FCurveEditorTreeItem* FindItem(FCurveEditorTreeItemID ItemID) const;

	/** 
	 * Retrieve this curve editor's root items irrespective of filter state
	 */
	UE_API const TArray<FCurveEditorTreeItemID>& GetRootItems() const;

	/** 
	 * Retrieve all the items stored in this tree irrespective of filter state
	 */
	UE_API const TMap<FCurveEditorTreeItemID, FCurveEditorTreeItem>& GetAllItems() const;

	/**
	 * Add a new empty item to the tree
	 *
	 * @param ParentID The ID of the desired parent for the new item, or FCurveEditorTreeItemID::Invalid for root nodes
	 */
	UE_API FCurveEditorTreeItem* AddItem(FCurveEditorTreeItemID ParentID);

	/**
	 * Remove an item and all its children from this tree, destroying any curves it may have created.
	 *
	 * @param ItemID The ID of the item to remove
	 * @param CurveEditor (required) Pointer to the curve editor that owns this tree to remove curves from
	 */
	UE_API void RemoveItem(FCurveEditorTreeItemID ItemID, FCurveEditor* CurveEditor);

	/**
	 * Run all the filters on this tree, updating filter state for all tree items
	 */
	UE_API void RunFilters();

	/**
	 * Add a new filter to this tree. Does not run the filter (and thus update any tree views) until RunFilters is called.
	 *
	 * @param NewFilter The new filter to add to this tree
	 */
	UE_API void AddFilter(TWeakPtr<FCurveEditorTreeFilter> NewFilter);

	/**
	 * Remove an existing filter from this tree. Does not re-run the filters (and thus update any tree views) until RunFilters is called.
	 *
	 * @param FilterToRemove The filter to remove from this tree
	 */
	UE_API void RemoveFilter(TWeakPtr<FCurveEditorTreeFilter> FilterToRemove);

	/**
	 * Direct access to all current filters (potentially including expired ones)
	 */
	UE_API TArrayView<const TWeakPtr<FCurveEditorTreeFilter>> GetFilters() const;

	/**
	 * Clear all filters from the tree
	 */
	UE_API void ClearFilters();

	/**
	 * Attempt to locate a filter by its type
	 *
	 * @param Type The type of the filter to find
	 * @return A pointer to the filter, or nullptr if one could not be found
	 */
	UE_API const FCurveEditorTreeFilter* FindFilterByType(ECurveEditorTreeFilterType Type) const;

	/**
	 * Sets a predicate which will be used to sort tree items after they're been marked as needing sort.
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API void SetSortPredicate(FTreeItemSortPredicate InSortPredicate);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Sets a predicate which will be used to sort tree items after they're been marked as needing sort. */
	UE_API void SetSortPredicate(FCurveTreeItemSortPredicate InSortPredicate);

	 /**
	  * Sorts all tree items which have been marked for sorting if the sort predicate has been set.
	  */
	UE_API void SortTreeItems();

	/**
	 * Inform this tree that the specified tree item IDs have been directly selected on the UI.
	 * @note: This populates both implicit and explicit selection state for the supplied items and any children/parents
	 */
	UE_API void SetDirectSelection(TArray<FCurveEditorTreeItemID>&& TreeItems, FCurveEditor* InCurveEditor);

	/**
	 * Removes tree items from the current selection.
	 */
	UE_API void RemoveFromSelection(TArrayView<const FCurveEditorTreeItemID> TreeItems, FCurveEditor* InCurveEditor);

	/**
	 * Access the selection state for this tree. Items that are neither implicitly or explicitly selected are not present in the map.
	 */
	UE_API const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& GetSelection() const;

	/**
	 * Check a specific tree item's selection state
	 */
	UE_API ECurveEditorTreeSelectionState GetSelectionState(FCurveEditorTreeItemID InTreeItemID) const;

	/**
	 * Access the filter state for this tree. Items that are neither implicitly or explicitly filtered-in are not present in the map.
	 */
	UE_API const FCurveEditorFilterStates& GetFilterStates() const;

	/**
	 * Check a specific tree item's filter state
	 */
	UE_API ECurveEditorTreeFilterState GetFilterState(FCurveEditorTreeItemID InTreeItemID) const;

	/**
	 * Retrieve a scoped event guard that will block broadcast of events until the last guard on the stack goes out of scope
	 * Can be used to defer broadcasts in situations where many changes are made to the tree at a time.
	 */
	FScopedCurveEditorTreeEventGuard ScopedEventGuard()
	{
		return FScopedCurveEditorTreeEventGuard(this);
	}

	/**
	 * Compact the memory used by this tree (does not modify any meaningful state)
	 */
	UE_API void Compact();

	/*
	 * Toggle the expansion state of the selected nodes or all nodes if none selected
	 */
	UE_API void ToggleExpansionState(bool bRecursive);

	/** Expand all nodes in the tree */
	UE_API void ExpandAllNodes();

	/** Collapse all nodes in the tree */
	UE_API void CollapseAllNodes();

	FOnCurveEditorToggleExpansionState& GetToggleExpansionState()
	{
		return ToggleExpansionStateDelegate;
	}

	FOnCurveEditorExpandAllNodes& GetOnExpandAllNodes()
	{
		return ExpandAllNodesDelegate;
	}

	FOnCurveEditorCollapseAllNodes& GetOnCollapseAllNodes()
	{
		return CollapseAllNodesDelegate;
	}

	/*
	* Get cached expanded items
	*/
	UE_API TArray<FCurveEditorTreeItemID> GetCachedExpandedItems() const;
	
	/*
	* Set Item expansion state
	*/
	UE_API void SetItemExpansion(FCurveEditorTreeItemID InTreeItemID, bool bInExpansion);

	/**
	Whether or not we are are doign a direct selection, could be used to see why a curve model is being created or destroyed, by direct selection or by sequencer filtering?
	*/
	bool IsDoingDirectSelection() const
	{
		return bIsDoingDirectSelection;
	}

	/**
	Recreate the curve models from the existing selection, this may be needed in case of a setting change.
	*/
	UE_API void RecreateModelsFromExistingSelection(FCurveEditor* CurveEditor);

private:

	// Recursively removes children without removing them from the parent (assuming the parent is also being removed)
	UE_API void RemoveChildrenRecursive(TArray<FCurveEditorTreeItemID>&& Children, FCurveEditor* CurveEditor);

	/**
	 * Run the specified filters over the specified items and their recursive children, storing the results in this instance's FilterStates struct.
	 *
	 * @param FilterPtrs     Array of non-null pointers to filters to use. Items are considered matched if they match any filter in this array.
	 * @param ItemsToFilter  Array item IDs to filter
	 * @param InheritedState The filter state for each item to receive if it does not directly match a filter (either ECurveEditorTreeFilterState::NoMatch or ECurveEditorTreeFilterState::InheritedChild)
	 * @return Raised state flags which should be applied to parents of the filtered items.
	 */
	UE_API ECurveEditorTreeFilterState PerformFilterPass(TArrayView<const FCurveEditorTreeFilter* const> FilterPtrs, TArrayView<const FCurveEditorTreeItemID> ItemsToFilter, ECurveEditorTreeFilterState InheritedState);

	/** 
	 * Recursively sorts the tree item ids using the sort predicate.
	 */
	UE_API void SortTreeItems(FSortedCurveEditorTreeItems& TreeItemIDsToSort);

	/** Incrementing ID for the next tree item to be created */
	FCurveEditorTreeItemID NextTreeItemID;

	/** Map of all tree items by their ID */
	TMap<FCurveEditorTreeItemID, FCurveEditorTreeItem> Items;

	/** Array of all filters that are currently active on this tree */
	TArray<TWeakPtr<FCurveEditorTreeFilter>> WeakFilters;

	/** Hierarchical information for the tree */
	FSortedCurveEditorTreeItems RootItems;
	TMap<FCurveEditorTreeItemID, FSortedCurveEditorTreeItems> ChildItemIDs;

	/** Selection state map. Items with no implicit or explicit selection are not present */
	TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> Selection;

	/** Filter state map. Items with no implicit or explicit filter state are not present */
	FCurveEditorFilterStates FilterStates;

	/** A predicate which will be used to sort tree items after they're been marked as needing sort. */
	FCurveTreeItemSortPredicate SortPredicate;

	/** Delegate for when toggle expansion state is invoked */
	FOnCurveEditorToggleExpansionState ToggleExpansionStateDelegate;

	/** Delegates for expand/collapse all */
	FOnCurveEditorExpandAllNodes ExpandAllNodesDelegate;
	FOnCurveEditorCollapseAllNodes CollapseAllNodesDelegate;

	/** Set of cached expanded items, based on GetTypedHash(FString)*/
	TSet<int32> CachedExpandedItems;

	/** Whether or not we are are doign a direct selection, could be used to see why a curve model is being created or destroyed, by direct selection or by sequencer filtering?*/
	bool bIsDoingDirectSelection = false;
};

#undef UE_API
