// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API CURVEEDITOR_API

class FCurveEditorTree;
struct FCurveEditorFilterStates;
struct FCurveEditorTreeItem;
struct FCurveEditorTreeItemID;

enum class ECurveEditorTreeFilterState : uint8;

enum class ECurveEditorTreeFilterType : uint32
{
	/** Filter is a FCurveEditorTreeTextFilter instance */
	Text,

	CUSTOM_START,


	First = Text,
};

/**
 * Base class for all filters that can be applied to a curve editor tree.
 * Filters are identifyable through their type (see GetType()), which is a pre-registered static value retrieved through RegisterFilterType(). Client types can inspect this for supported types, and static_cast<> for implementation details.
 * Filters also contain a pass, which is used to separate filters into separate passes. Filters applied within the same pass are matched as a boolean OR, filters in different passes are matched as a boolean AND.
 */
struct FCurveEditorTreeFilter
{
	FCurveEditorTreeFilter(ECurveEditorTreeFilterType InFilterType, int32 InFilterPass)
		: FilterType(InFilterType)
		, FilterPass(InFilterPass)
		, bExpandToMatchedItems(true)
	{}

	virtual ~FCurveEditorTreeFilter() = default;

	/**
	 * @return The type of this filter as registered by RegisterFilterType
	 */
	ECurveEditorTreeFilterType GetType() const
	{
		return FilterType;
	}

	/**
	 * @return The pass index that should be used when applying this filter
	 */
	int32 GetFilterPass() const
	{
		return FilterPass;
	}

	/**
	 *  @return If tree paths should be expanded down to matched items
	 */
	bool ShouldExpandOnMatch() const
	{
		return bExpandToMatchedItems;
	}

public:

	/**
	 * Register a new filter type that is passed to ICurveEditorTreeItem::Filter
	 *
	 * @param InTreeItem The tree item to test
	 * @return true if the item matches the filter, false otherwise
	 */
	static UE_API ECurveEditorTreeFilterType RegisterFilterType();

protected:

	/** The static type of this filter as retrieved by RegisterFilterType */
	ECurveEditorTreeFilterType FilterType;

	/** Defines which pass this filter should be applied in */
	int32 FilterPass;

	/** Determines if tree paths should be expanded down to matched items */
	bool bExpandToMatchedItems : 1;
};

/** A specific text token (containing neither spaces nor .) */
struct FCurveEditorTreeTextFilterToken
{
	FString Token;

	/** Match this token against a string */
	bool Match(const TCHAR* InString) const
	{
		return FCString::Stristr(InString, *Token) != nullptr;
	}
};

/** A text filter term containing >= 1 sparate tokens ordered from child to parent */
struct FCurveEditorTreeTextFilterTerm
{
	/**
	 * Struct representing the result of a match operation,
	 *     including any remaining tokens that need to be matched recursively upwards
	 */
	struct FMatchResult
	{
		/** Default constructor to an empty result that did not match */
		FMatchResult()
			: bMatched(false)
		{}

		/** Constructor that creates a successful match with optional remaining tokens */
		FMatchResult(TArrayView<const FCurveEditorTreeTextFilterToken> InRemainingTokens)
			: RemainingTokens(InRemainingTokens)
			, bMatched(true)
		{}

		/** Match another string against the remaining tokens for this result */
		CURVEEDITOR_API FMatchResult Match(FStringView InString) const;

		/** Check whether this result is a partial or total match. Returns false if any match has failed. */
		bool IsAnyMatch() const
		{
			return bMatched;
		}
		/** Check whether this result matched, but still has tokens left to match */
		bool IsPartialMatch() const
		{
			return bMatched && RemainingTokens.Num() > 0;
		}
		/** Check whether this result has completely matched all tokens */
		bool IsTotalMatch() const
		{
			return bMatched && RemainingTokens.Num() == 0;
		}

	private:

		TArrayView<const FCurveEditorTreeTextFilterToken> RemainingTokens;
		bool bMatched;
	};

	/**
	 * Match a string against these tokens
	 */
	CURVEEDITOR_API FMatchResult Match(FStringView InString) const;

	TArray<FCurveEditorTreeTextFilterToken, TInlineAllocator<1>> ChildToParentTokens;
};

/**
 * Built-in text filter of type ECurveEditorTreeFilterType::Text. Filter terms are applied as a case-insensitive boolean OR substring match.
 */
struct FCurveEditorTreeTextFilter : FCurveEditorTreeFilter
{
	/** Original input filter text. */
	FText InputText;

	/** Default pass for text filters. */
	static const int32 DefaultPass = 1000;

	/** Default constructor */
	FCurveEditorTreeTextFilter()
		: FCurveEditorTreeFilter(ECurveEditorTreeFilterType::Text, DefaultPass)
	{}

	/**
	 * Assign a new filter string to this filter, resetting any previous filter terms
	 *
	 * @param FilterString   The new filter string that should be parsed into this filter
	 */
	UE_API void AssignFromText(const FString& FilterString);

	/**
	 * Check whether this filter is empty (ie, has no filter terms)
	 */
	bool IsEmpty() const
	{
		return ChildToParentFilterTerms.Num() == 0;
	}

	/**
	 * Access all the filter terms contained within this filter. Each term comprises an array of text tokens that must match from child to parent.
	 * Valid to call if IsEmpty() == true, but the resulting array view will be empty.
	 */
	TArrayView<const FCurveEditorTreeTextFilterTerm> GetTerms() const
	{
		return ChildToParentFilterTerms;
	}

private:

	/** Arrays of case-insensitive terms to find within tree items. Each term is an array of strings that must be matched from child to parent. */
	TArray<FCurveEditorTreeTextFilterTerm, TInlineAllocator<4>> ChildToParentFilterTerms;
};

#undef UE_API
