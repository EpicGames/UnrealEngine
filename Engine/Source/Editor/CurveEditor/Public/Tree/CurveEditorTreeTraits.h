// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "CurveEditorTypes.h"

template<>
struct TListTypeTraits<FCurveEditorTreeItemID>
{
public:
	struct NullableType : FCurveEditorTreeItemID
	{
		NullableType(TYPE_OF_NULLPTR){}
		NullableType(FCurveEditorTreeItemID Other) : FCurveEditorTreeItemID(Other) {}
	};

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FCurveEditorTreeItemID, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FCurveEditorTreeItemID, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FCurveEditorTreeItemID>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&, TArray<FCurveEditorTreeItemID>&, TSet<FCurveEditorTreeItemID>&, TMap< const U*, FCurveEditorTreeItemID >&) {}

	static bool IsPtrValid(NullableType InPtr)
	{
		return InPtr.IsValid();
	}

	static void ResetPtr(NullableType& InPtr)
	{
		InPtr = nullptr;
	}

	static NullableType MakeNullPtr()
	{
		return nullptr;
	}

	static FCurveEditorTreeItemID NullableItemTypeConvertToItemType(NullableType InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(FCurveEditorTreeItemID InPtr)
	{
		return FString::Printf(TEXT("%d"), InPtr.GetValue());
	}

	class SerializerType{};
};

template <>
struct TIsValidListItem<FCurveEditorTreeItemID>
{
	enum
	{
		Value = true
	};
};

