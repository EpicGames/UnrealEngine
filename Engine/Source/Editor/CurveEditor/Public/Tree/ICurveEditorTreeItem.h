// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#define UE_API CURVEEDITOR_API

enum class ECurveEditorTreeFilterType : uint32;

class FCurveEditor;
class FCurveModel;
class FName;
class ITableRow;
class SWidget;
struct FCurveEditorTreeFilter;
struct FCurveEditorTreeItemID;


/**
 * Optional implementation interface for any tree item to be shown on the curve editor tree.
 */
struct ICurveEditorTreeItem
{
	virtual ~ICurveEditorTreeItem() = default;

	/**
	 * Generate the widget content for the specified column name of the curve editor tree view
	 *
	 * @param InColumnName     The name of the column to generate widget content for. See FColumnNames for valid names.
	 * @param InCurveEditor    Weak pointer to the curve editor instance. Persistent TSharedPtrs should not be held to this pointer.
	 * @param InTreeItemID     The ID of the tree item that this interface is assigned to
	 * @param InTableRow       The table row that will house the widget content
	 * @return (Optional) Widget content for this column, or nullptr if none is necessary for this column.
	 */
	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow) = 0;

	/**
	 * Populate the specified array with curve models that are represented by this tree item
	 *
	 * @param OutCurveModels  Array of curve models to be populated with the curves of this tree item
	 */
	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) = 0;

	/**
	 * Check whether this tree item passes the specified filter. Filter's types can be checked using FCurveEditorTreeFilter::GetType() and static_casted for known filter types.
	 *
	 * @param InFilter        The filter to test against.
	 * @return True if this item is deemed to have passed the filter, false otherwise
	 */
	virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const { return false; }

	/** Structure of pre-defined supported column names for the curve editor outliner */
	struct FColumnNames
	{
		/** Generic label content */
		FName Label;
		/** Right-aligned column for select buttons */
		FName SelectHeader;
		/** Right-aligned column for pin buttons */
		FName PinHeader;

		CURVEEDITOR_API FColumnNames();
	};

	static UE_API const FColumnNames ColumnNames;
};

#undef UE_API
