// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CurveEditorTreeTraits.h"
#include "CurveEditorTypes.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class ITableRow;
class SHeaderRow;
class STableViewBase;
struct FGeometry;
struct FKeyEvent;

class SCurveEditorTree : public STreeView<FCurveEditorTreeItemID>
{

public:

	SLATE_BEGIN_ARGS(SCurveEditorTree)
		: _SelectColumnWidth(24.f)
		{}
		SLATE_ARGUMENT(float, SelectColumnWidth)
		SLATE_EVENT(FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick)
		SLATE_EVENT(FOnTableViewScrolled, OnTreeViewScrolled)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor);

	const TArray<FCurveEditorTreeItemID>& GetSourceItems() const { return RootItems; }

	UE_API TSharedRef<ITableRow> GenerateRow(FCurveEditorTreeItemID ItemID, const TSharedRef<STableViewBase>& OwnerTable);
	
	UE_API void GetTreeItemChildren(FCurveEditorTreeItemID Parent, TArray<FCurveEditorTreeItemID>& OutChildren);

private:

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:


	UE_API void OnTreeSelectionChanged(FCurveEditorTreeItemID, ESelectInfo::Type);

	UE_API void SetItemExpansionRecursive(FCurveEditorTreeItemID Model, bool bInExpansionState);

	UE_API void OnExpansionChanged(FCurveEditorTreeItemID Model, bool bInExpansionState);

	UE_API void RefreshTree();

	UE_API void RefreshTreeWidgetSelection();

	UE_API void ToggleExpansionState(bool bRecursive);

	UE_API void ExpandAllNodes();

	UE_API void CollapseAllNodes();

	UE_API FCurveEditorTreeItemID FindVisibleSelectionProxy(const FCurveEditorTreeItemID InItemID) const;

private:

	bool bFilterWasActive;

	TArray<FCurveEditorTreeItemID> RootItems;

	/** Set of item IDs that were expanded before a filter was applied */
	TSet<FCurveEditorTreeItemID> PreFilterExpandedItems;

	TSharedPtr<FCurveEditor> CurveEditor;

	TSharedPtr<SHeaderRow> HeaderRow;

	bool bUpdatingTreeWidgetSelection;
	bool bUpdatingCurveEditorTreeSelection;

};

#undef UE_API
