// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class STextBlock;

/** Widget that shows the metrics for the current tree filter in the curve editor (in the form "Showing {0} of {1} items ({2} selected)") */
class SCurveEditorTreeFilterStatusBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorTreeFilterStatusBar){}
	SLATE_END_ARGS()

	/** Construct the status bar */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API void ShowStatusBar();
	UE_API void HideStatusBar();
	UE_API void FadeOutStatusBar();

private:

	/** Request the visibility of the clear hyperlink widget based on whether there's a filter active or not */
	UE_API EVisibility GetVisibilityFromFilter() const;

	/** Update the filter text to represent the current filter states in the tree */
	UE_API void UpdateText();

	/** Clear all the filters in the tree - called as a result of the user clicking on the 'clear' hyperlink */
	UE_API void ClearFilters();

private:

	TWeakPtr<FCurveEditor> WeakCurveEditor;
	TSharedPtr<STextBlock> TextBlock;
	double OpacityThrobEndTime = 0;
};

#undef UE_API
