// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CurveEditorTreeTraits.h"
#include "CurveEditorTypes.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class ITableRow;
struct FSlateBrush;

class SCurveEditorTreePin : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorTreePin){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow);

private:

	UE_API FReply TogglePinned();
	UE_API const FSlateBrush* GetPinBrush() const;

	UE_API bool IsPinnedRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor) const;

	UE_API void PinRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor) const;

	UE_API void UnpinRecursive(FCurveEditorTreeItemID InTreeItem, FCurveEditor* CurveEditor, TArray<FCurveEditorTreeItemID>& OutUnpinnedItems) const;

	UE_API EVisibility GetPinVisibility() const;

private:

	TWeakPtr<ITableRow> WeakTableRow;
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	FCurveEditorTreeItemID TreeItemID;
};

#undef UE_API
