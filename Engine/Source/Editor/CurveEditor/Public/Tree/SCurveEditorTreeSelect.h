// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTreeTraits.h"
#include "CurveEditorTypes.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class ITableRow;
struct FSlateBrush;

class SCurveEditorTreeSelect : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveEditorTreeSelect){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow);

private:

	UE_API FReply SelectAll();

	UE_API const FSlateBrush* GetSelectBrush() const;

	UE_API EVisibility GetSelectVisibility() const;

private:

	TWeakPtr<ITableRow> WeakTableRow;
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	FCurveEditorTreeItemID TreeItemID;
};

#undef UE_API
