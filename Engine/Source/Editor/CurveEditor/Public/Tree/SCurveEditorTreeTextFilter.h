// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FText;
class SSearchBox;
struct FCurveEditorTreeTextFilter;
struct FFocusEvent;
struct FGeometry;

class SCurveEditorTreeTextFilter : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCurveEditorTreeTextFilter){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> CurveEditor);
	
	// SWidget Interface
	UE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	// ~SWidget Interface
private:

	UE_API void CreateSearchBox();
	UE_API void OnTreeFilterListChanged();
	UE_API void OnFilterTextChanged(const FText& FilterText);

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<FCurveEditorTreeTextFilter> Filter;
	TWeakPtr<FCurveEditor> WeakCurveEditor;
};

#undef UE_API
