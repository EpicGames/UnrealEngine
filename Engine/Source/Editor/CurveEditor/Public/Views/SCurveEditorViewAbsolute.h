// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Views/SInteractiveCurveEditorView.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;

/**
 * A Normalized curve view supporting one or more curves with their own screen transform that normalizes the vertical curve range to [-1,1]
 */
class SCurveEditorViewAbsolute : public SInteractiveCurveEditorView
{
public:

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

};

#undef UE_API
