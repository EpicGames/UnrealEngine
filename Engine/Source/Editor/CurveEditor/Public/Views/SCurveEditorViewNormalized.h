// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EditorUndoClient.h"
#include "HAL/Platform.h"
#include "Rendering/RenderingCommon.h"
#include "Templates/SharedPointer.h"
#include "Views/SInteractiveCurveEditorView.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
struct FGeometry;

/**
 * A Normalized curve view supporting one or more curves with their own screen transform that normalizes the vertical curve range to [-1,1]
 */
class SCurveEditorViewNormalized : public SInteractiveCurveEditorView, public FSelfRegisteringEditorUndoClient
{
public:

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

	/** Tools should use vertical snapping since grid lines to snap to will usually be visible */
	virtual bool IsValueSnapEnabled() const override { return true; }

	UE_API virtual void UpdateViewToTransformCurves(double InputMin, double InputMax) override;

	UE_API void FrameVertical(double InOutputMin, double InOutputMax, FCurveEditorViewAxisID AxisID = FCurveEditorViewAxisID()) override;

	virtual void OnCurveListChanged() override;

protected:
	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

	// SWidget Interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// ~SWidget Interface

	UE_API virtual void PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API virtual void DrawBufferedCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;

	/** Helper to call UpdateViewToTransformCurves from internal methods */
	UE_API void InternalUpdateViewToTransformCurves();

	/**
	 * Limits OutMin and OutMax to the time controller playback range, i.e. OutMin >= TimeController's min and OutMax <= TimeController's max.
	 * This corresponds green, left and red, right horizontal lines on the frame ruler.
	 */
	UE_API void ClampToTimeController(double& OutMin, double& OutMax) const;

private:
	/** Recomputes a single curve's ViewToCurveTransform from its current value range. */
	void RecomputeViewToCurveTransform(const TSharedRef<FCurveEditor>& CurveEditor, FCurveModelID CurveID, FCurveInfo& CurveInfo);

	/** Curve IDs whose ViewToCurveTransform has been computed at least once. */
	TSet<FCurveModelID> InitializedCurves;
};

#undef UE_API
