// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Rendering/RenderingCommon.h"
#include "Templates/SharedPointer.h"
#include "Views/SInteractiveCurveEditorView.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
struct FGeometry;

/**
 * A Normalized curve view supporting one or more curves with their own screen transform that normalizes the vertical curve range to [-1,1]
 */
class SCurveEditorViewStacked : public SInteractiveCurveEditorView
{
public:

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);
	
	/** Tools should use vertical snapping since grid lines to snap to will usually be visible */
	virtual bool IsValueSnapEnabled() const override { return true; }

	UE_API virtual void UpdateViewToTransformCurves(double InputMin, double InputMax) override;

protected:

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	UE_API virtual void PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual void GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels = nullptr) const override;

	UE_API virtual void DrawViewGrids(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;
	UE_API virtual void DrawLabels(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;
	UE_API virtual void DrawBufferedCurves(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;

	/** Stacked height per curve */
	float StackedHeight;

	/** Stacked padding per curve */
	float StackedPadding;
};

#undef UE_API
