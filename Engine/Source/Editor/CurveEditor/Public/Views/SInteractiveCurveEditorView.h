// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CurveDrawInfo.h"
#include "CurveEditorTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "ICurveEditorDragOperation.h"
#include "ICurveEditorToolExtension.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Rendering/RenderingCommon.h"
#include "SCurveEditorView.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "CurveEditorSettings.h"

#define UE_API CURVEEDITOR_API

class FCurveEditor;
class FCurveModel;
class FMenuBuilder;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class IMenu;
struct FCurveEditorDelayedDrag;
struct FCurveEditorScreenSpace;
struct FCurveEditorToolID;
struct FCurveModelID;
struct FCurvePointHandle;
struct FGeometry;
struct FOptionalSize;
struct FPointerEvent;
struct FKeyAttributes;
struct FKeyHandleSet;

namespace CurveViewConstants
{
	/** The default offset from the top-right corner of curve views for curve labels to be drawn. */
	constexpr float CurveLabelOffsetX = 15.f;
	constexpr float CurveLabelOffsetY = 10.f;

	constexpr FLinearColor BufferedCurveColor = FLinearColor(.4f, .4f, .4f);

	/**
	 * Pre-defined layer offsets for specific curve view elements. Fixed values are used to decouple draw order and layering
	 * Some elements deliberately leave some spare layers as a buffer for slight tweaks to layering within that element
	 */
	namespace ELayerOffset
	{
		enum
		{
			Background     = 0,
			GridLines      = 1,
			GridOverlays   = 2,
			GridLabels     = 3,
			Curves         = 10,
			HoveredCurves  = 15,
			Keys           = 20,
			SelectedKeys   = 30,
			Tools          = 35,
			DragOperations = 40,
			Labels         = 45,
			WidgetContent  = 50,
			Last = Labels
		};
	}
}

/**
 */
class SInteractiveCurveEditorView : public SCurveEditorView
{
public:

	SLATE_BEGIN_ARGS(SInteractiveCurveEditorView)
		: _BackgroundTint(FLinearColor::White)
		, _MaximumCapacity(0)
		, _AutoSize(true)
	{}

		SLATE_ARGUMENT(FLinearColor, BackgroundTint)

		SLATE_ARGUMENT(int32, MaximumCapacity)

		SLATE_ATTRIBUTE(float, FixedHeight)

		SLATE_ARGUMENT(bool, AutoSize)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

	UE_API virtual void GetGridLinesX(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels = nullptr) const override;
	UE_API virtual void GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels = nullptr) const override;

	UE_API virtual void BuildContextMenu(FMenuBuilder& MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurveID);

protected:

	// ~SCurveEditorView Interface
	UE_API virtual bool GetPointsWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const override;
	UE_API virtual bool GetCurveWithinWidgetRange(const FSlateRect& WidgetRectangle, TArray<FCurvePointHandle>* OutPoints) const override;
	UE_API virtual TOptional<FCurveModelID> GetHoveredCurve() const override;

	UE_API virtual FText FormatToolTipCurveName(const FCurveModel& CurveModel) const;
	UE_API virtual FText FormatToolTipTime(const FCurveModel& CurveModel, double EvaluatedTime) const;
	UE_API virtual FText FormatToolTipValue(const FCurveModel& CurveModel, double EvaluatedValue) const;

	UE_API virtual void PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

protected:

	// SWidget Interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnFinishedPointerInput() override;
	UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

public:
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	// ~SWidget Interface

protected:

	UE_API void DrawBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;
	UE_API void DrawGridLines(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;
	UE_API void DrawCurves(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const;
	UE_API void DrawBufferedCurves(TSharedRef<FCurveEditor> CurveEditor, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, ESlateDrawEffect DrawEffects) const;
	UE_API void DrawValueIndicatorLines(TSharedRef<FCurveEditor> InCurveEditor, const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InBaseLayerId) const;

	/** Updates the keys that will be drawn with value indicator lines. */
	UE_API void UpdatedKeysWithValueIndicatorLines(const FCurveEditor& InCurveEditor) const;
	/** Picks which keys will have value indicator lines drawn for them. Returns true if the selection. */
	UE_API bool PickPointsToPlaceValueIndicatorLinesOn(const FCurveModel& InCurveModel, const FKeyHandleSet& InUserSelectedKeys, FKeyHandle& OutMinKey, FKeyHandle& OutMaxKey) const;

	UE_API FSlateColor GetCurveCaptionColor() const;
	UE_API FText GetCurveCaption() const;

private:

	UE_API void HandleDirectKeySelectionByMouse(TSharedPtr<FCurveEditor> CurveEditor, const FPointerEvent& MouseEvent, TOptional<FCurvePointHandle> MouseDownPoint);

	UE_API void CreateContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Updates our distance to the curve neareast to the cursor. */
	UE_API void UpdateCurveProximities(FVector2D MousePixel);

	UE_API TOptional<FCurvePointHandle> HitPoint(FVector2D MousePixel) const;

	UE_API bool IsToolTipEnabled() const;
	UE_API FText GetToolTipCurveName() const;
	UE_API FText GetToolTipTimeText() const;
	UE_API FText GetToolTipValueText() const;

	/**
	 * Returns the proper tangent value so we can keep the curve remain the original shape
	 *
	 * @param InTime		The time we are trying to add a key to
	 * @param InValue		The value we are trying to add a key to
	 * @param CurveToAddTo  The curve we are trying to add a key to
	 * @param DeltaTime		Negative to get the left tangent, positive for right. Remember to use FMath::Abs() when needed
	 * @return				The tangent value relatives to the DeltaTime upon mouse click's position
	 */
	UE_API double GetTangentValue(const double InTime, const double InValue, FCurveModel* CurveToAddTo, double DeltaTime) const;

	/*~ Command binding callbacks */
	UE_API void AddKeyAtScrubTime(bool bUseSelection);
	UE_DEPRECATED(5.8, "Please, use AddKeyAtScrubTime that takes bUseSelection parameter")
	void AddKeyAtScrubTime(TSet<FCurveModelID> ForCurves) { AddKeyAtScrubTime(ForCurves.Num() > 0); }
	UE_API void AddKeyAtMousePosition(TSet<FCurveModelID> ForCurves);
	UE_API void AddKeyAtTime(const TSet<FCurveModelID>& ToCurves, double InTime);
	UE_API void PasteKeys(TSet<FCurveModelID> ToCurves);
	UE_API void ResetKeysToDefaultValues(bool bUseSelection);
	UE_API void OnCurveEditorToolChanged(FCurveEditorToolID InToolId);
	UE_API void OnShowValueIndicatorsChanged();

	/**
	 * Rebind contextual command mappings that rely on the mouse position
	 */
	UE_API void RebindContextualActions(FVector2D InMousePosition);

	/** Copy the curves from this view and set them as the Curve Editor's buffered curve support. */
	UE_API void BufferCurves();
	/** Attempt to apply the previously buffered curves to the currently selected curves. */
	UE_API void ApplyBufferCurves(const bool bSwapBufferCurves);
	/** Check if it's legal to buffer any of our selected curves. */
	UE_API bool CanBufferedCurves() const;
	/** Check if it's legal to apply any of the buffered curves to our currently selected curves. */
	UE_API bool CanApplyBufferedCurves() const;
	/** Returns interpolation mode and tangent mode based on neighbours or default curve editor if no neighbours . */
	UE_API FKeyAttributes GetDefaultKeyAttributesForCurveTime(const FCurveEditor& CurveEditor, const FCurveModel& CurveModel, double EvalTime) const;

protected:

	/** Background tint for this widget */
	FLinearColor BackgroundTint;

private:

	/** (Optional) the current drag operation */
	TOptional<FCurveEditorDelayedDrag> DragOperation;
	/** Whether DragOperation needs to have OnFinishedPointerInput called this tick. */
	bool bHadMouseMovesThisTick = false;

	struct FCachedToolTipData
	{
		FCachedToolTipData() {}

		FText Text;
		FText EvaluatedValue;
		FText EvaluatedTime;
	};

	TOptional<FCachedToolTipData> CachedToolTipData;

	/** Array of curve proximities in slate units that's updated on mouse move. Does not necessarily hold all curves. */
	TArray<TTuple<FCurveModelID, float>> CurveProximities;

	/** Track if we have a context menu active. Used to suppress hover updates as it causes flickers in the CanExecute bindings. */
	TWeakPtr<IMenu> ActiveContextMenu;

	/** Cached location of the mouse relative to this widget each tick. This is so that command bindings related to the mouse cursor can create them at the right time. */
	FVector2D CachedMousePosition;

	/** Cached curve caption, used to determine when to refresh the retainer */
	mutable FText CachedCurveCaption;

	/** Cached curve caption color, used to determine when to refresh the retainer */
	mutable FSlateColor CachedCurveCaptionColor;

	mutable bool bNeedsRefresh = false;

	struct FValueIndicatorLineData
	{
		/** The curve for which the value indicator lines are being drawn. */
		const FCurveModelID HighlightedCurve;

		/** Key handle to the min item through which a value indicator line is being drawn. */
		FKeyHandle MinKey = FKeyHandle::Invalid();
		/** Key handle to the max item through which a value indicator line is being drawn. If the user only has 1 key selected, this is FKeyHandle::Invalid(). */
		FKeyHandle MaxKey = FKeyHandle::Invalid();

		explicit FValueIndicatorLineData(const FCurveModelID& HighlightedCurve, const FKeyHandle& MinKey, const FKeyHandle& MaxKey)
			: HighlightedCurve(HighlightedCurve) , MinKey(MinKey), MaxKey(MaxKey)
		{}
	};
	/**
	 * We draw up to 2 horizontal value indicator line on the keys that a user selects.
	 * This makes it easier for the user to see a selected key's value in relation to other keys.
	 *
	 * Requirements:
	 * - Only draw value indicator line if 1 curve is selected. More curves selected = no line.
	 * - The value indicator lines are drawn through the min and max value keys in the selected range.
	 * - If only 1 key is selected, only draw 1 line (obviously).
	 * The per-curve upper limit of value indicator lines that are drawn is configured in UCurveEditorSettings::MaxNumHorizontalSelectionLinesPerCurve.
	 *
	 * This is called in OnPaint, which is const. Mutable is a workaround.
	 */
	mutable TOptional<FValueIndicatorLineData> ValueIndicatorLineDrawData;
};

#undef UE_API
